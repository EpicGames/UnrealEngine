[CoreRedirects]

+ClassRedirects=(OldName="/Script/DynamicMaterial.DMMaterialValueTexture2D",NewName="/Script/DynamicMaterial.DMMaterialValueTexture")
+ClassRedirects=(OldName="/Script/DynamicMaterial.DMMaterialValueTextureCube",NewName="/Script/DynamicMaterial.DMMaterialValueTexture")
+ClassRedirects=(OldName="/Script/DynamicMaterial.DMMaterialValueVolumeTexture",NewName="/Script/DynamicMaterial.DMMaterialValueTexture")
+ClassRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModelBase",NewName="/Script/DynamicMaterial.DynamicMaterialModel")
+ClassRedirects=(OldName="/Script/DynamicMaterialEditor.DynamicMaterialModel",NewName="/Script/DynamicMaterial.DynamicMaterialModel")

+EnumRedirects=(OldName="/Script/DynamicMaterial.EDMValueType",NewName="/Script/DynamicMaterial.EDMValueType",ValueChanges=(("VT_Texture_Any","VT_Texture"),("VT_Texture2D_Object","VT_Texture"),("VT_TextureCube_Object","VT_Texture"),("VT_VolumeTexture_Object","VT_Texture")))

+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_Effect_HSV.MF_DM_Effect_HSV",NewName="/DynamicMaterial/MaterialFunctions/Effects/Color/MF_DM_Effect_Base_HSV.MF_DM_Effect_Base_HSV")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_Effect_Base_HSV.MF_DM_Effect_Base_HSV",NewName="/DynamicMaterial/MaterialFunctions/Effects/Color/MF_DM_Effect_Base_HSV.MF_DM_Effect_Base_HSV")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_UVPanner.MF_DM_UVPanner",NewName="/DynamicMaterial/MaterialFunctions/Effects/UV/MF_DM_Effect_UV_Panner.MF_DM_Effect_UV_Panner")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_Effect_UV_Panner.MF_DM_Effect_UV_Panner",NewName="/DynamicMaterial/MaterialFunctions/Effects/UV/MF_DM_Effect_UV_Panner.MF_DM_Effect_UV_Panner")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_Effect_Base_Levels.MF_DM_Effect_Base_Levels",NewName="/DynamicMaterial/MaterialFunctions/Effects/Color/MF_DM_Effect_Base_Levels.MF_DM_Effect_Base_Levels")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_Effect_Mask_Level.MF_DM_Effect_Mask_Level",NewName="/DynamicMaterial/MaterialFunctions/Effects/Alpha/MF_DM_Effect_Mask_Level.MF_DM_Effect_Mask_Level")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_UV_Distort.MF_DM_UV_Distort",NewName="/DynamicMaterial/MaterialFunctions/Effects/UV/MF_DM_UV_Distort.MF_DM_UV_Distort")
+PackageRedirects=(OldName="/DynamicMaterial/MaterialFunctions/Effects/MF_DM_Effect_UV_Pixelate.MF_DM_Effect_UV_Pixelate",NewName="/DynamicMaterial/MaterialFunctions/Effects/UV/MF_DM_Effect_UV_Pixelate.MF_DM_Effect_UV_Pixelate")

+PropertyRedirects=(OldName="/Script/DynamicMaterial.DMRenderTargetTextRenderer.TextScale",NewName="/Script/DynamicMaterial.DMRenderTargetTextRenderer.RenderScale")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalOpacity",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalOpacityValue")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalTiling",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalTilingValue")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalMetallic",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalMetallicParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalSpecular",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalSpecularParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalRoughness",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalRoughnessParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalAnisotropy",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalAnisotropyParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalWorldPositionOffset",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalWorldPositionOffsetParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalAmbientOcclusion",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalAmbientOcclusionParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalRefraction",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalRefractionParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalPixelDepthOffset",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalPixelDepthOffsetParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalOffset",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalOffsetParameter")
+PackageRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalRotation",NewName="/Script/DynamicMaterial.DynamicMaterialModel.GlobalRotationParameter")

+PropertyRedirects=(OldName="/Script/DynamicMaterial.DMDefaultMaterialPropertySlotValue.Type",NewName="/Script/DynamicMaterial.DMDefaultMaterialPropertySlotValue.DefaultType")
+PropertyRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialInstance.MaterialModel",NewName="/Script/DynamicMaterial.DynamicMaterialInstance.MaterialModelBase")
+FunctionRedirects=(OldName="/Script/DynamicMaterial.DynamicMaterialModel.RemoveValueByName",NewName="/Script/DynamicMaterial.DynamicMaterialModel.RemoveValueByParameterName")

+PropertyRedirects=(OldName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.bTwoSidedFlag",NewName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.bTwoSided")
+PropertyRedirects=(OldName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.bPixelAnimationFlag",NewName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.bHasPixelAnimation")
+FunctionRedirects=(OldName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.IsPixelAnimationFlagSet",NewName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.GetHasPixelAnimation")
+FunctionRedirects=(OldName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.SetPixelAnimationFlag",NewName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.SetHasPixelAnimation")
+FunctionRedirects=(OldName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.IsTwoSidedFlagSet",NewName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.GetIsTwoSided")
+FunctionRedirects=(OldName="/Script/DynamicMateThatrialEditor.DynamicMaterialModelEditorOnlyData.SetTwoSidedFlag",NewName="/Script/DynamicMaterialEditor.DynamicMaterialModelEditorOnlyData.SetIsTwoSided")