{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Material Designer",
	"Description": "Compact dynamic material creator and editor, similar in style to other DDCs.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DynamicMaterial",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DynamicMaterialTextureSet",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DynamicMaterialEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DynamicMaterialTextureSetEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DynamicMaterialShaders",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "CustomDetailsView",
			"Enabled": true
		}
	]
}