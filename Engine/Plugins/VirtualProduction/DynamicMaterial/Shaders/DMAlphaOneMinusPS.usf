// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D InputTexture;
SamplerState InputSampler;

// Shader to apply 1-x to the alpha channel
void MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0
)
{
	float2 UV = UVAndScreenPos.xy;
	OutColor = InputTexture.Sample(InputSampler, UV);
	OutColor.a = 1.f - OutColor.a;
}
