// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterial : ModuleRules
{
	public DynamicMaterial(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DynamicMaterialTextureSet",
				"Slate",
				"SlateCore",
				"UMG"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Json",
				"JsonUtilities",
				"RenderCore"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd"
			});
		}

		ShortName = "DynMat";
	}
}
