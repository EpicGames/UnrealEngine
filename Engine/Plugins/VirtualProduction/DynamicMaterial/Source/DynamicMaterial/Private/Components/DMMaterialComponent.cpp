// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialComponent.h"
#include "DMComponentPath.h"
#include "DynamicMaterialModule.h"
#include "HAL/PlatformTime.h"

#if WITH_EDITOR
#include "Styling/SlateIconFinder.h"
#include "Textures/SlateIcon.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialComponent)

#define LOCTEXT_NAMESPACE "DMMaterialComponent"

double UDMMaterialComponent::MinCleanTime = FPlatformTime::Seconds();
const double UDMMaterialComponent::MinTimeBeforeClean = 0.2; // 5 fps

UDMMaterialComponent::UDMMaterialComponent()
#if WITH_EDITORONLY_DATA
	: ComponentState(EDMComponentLifetimeState::Created)
	, bComponentDirty(true)
#endif
{
}

UObject* UDMMaterialComponent::GetOuterSafe() const
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return nullptr;
	}

	UObject* Outer = GetOuter();

	if (!Outer || !Outer->IsValidLowLevelFast())
	{
		return nullptr;
	}

	return Outer;
}

bool UDMMaterialComponent::IsComponentValid() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS 
	const bool bValid = IsThisNotNull(this,"UDMMaterialComponent::IsComponentValid")
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		&& IsValidChecked(this)
		&& !HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed);

	return bValid;
}

UDMMaterialComponent* UDMMaterialComponent::GetComponentByPath(const FString& InPath) const
{
	FDMComponentPath Path(InPath);
	return GetComponentByPath(Path);
}

UDMMaterialComponent* UDMMaterialComponent::GetComponentByPath(FDMComponentPath& InPath) const
{
	if (InPath.IsLeaf())
	{
		return const_cast<UDMMaterialComponent*>(this);
	}

	// Fetches the first component of the path and removes it from the path
	const FDMComponentPathSegment FirstComponent = InPath.GetFirstSegment();

	if (UDMMaterialComponent* SubComponent = GetSubComponentByPath(InPath, FirstComponent))
	{
		return SubComponent->GetComponentByPath(InPath);
	}

	return nullptr;
}

UDMMaterialComponent* UDMMaterialComponent::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	// We have no subobjects by default
	return nullptr;
}

void UDMMaterialComponent::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

#if WITH_EDITOR
	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::RefreshDetailView))
	{
		if (UDMMaterialComponent* ParentComponent = GetParentComponent())
		{
			ParentComponent->Update(InSource, InUpdateType);
		}
	}

	OnUpdate.Broadcast(this, InSource, InUpdateType);
#endif
}

#if WITH_EDITOR
FString UDMMaterialComponent::GetComponentPath() const
{
	TArray<FString> ComponentPaths;
	GetComponentPathInternal(ComponentPaths);

	FString OutPath = "";

	for (const FString& ComponentPath : ComponentPaths)
	{
		if (OutPath.IsEmpty())
		{
			OutPath = ComponentPath;
		}
		else
		{
			OutPath = ComponentPath + TEXT(".") + OutPath;
		}
	}

	return OutPath;
}

FString UDMMaterialComponent::GetComponentPathComponent() const
{
	return GetName();
}

void UDMMaterialComponent::GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const
{
	OutChildComponentPathComponents.Add(GetComponentPathComponent());

	if (UDMMaterialComponent* ParentComponent = GetParentComponent())
	{
		ParentComponent->GetComponentPathInternal(OutChildComponentPathComponents);
	}
}

UDMMaterialComponent* UDMMaterialComponent::GetParentComponent() const
{
	return Cast<UDMMaterialComponent>(GetOuterSafe());
}

UDMMaterialComponent* UDMMaterialComponent::GetTypedParent(UClass* InParentClass, bool bInAllowSubclasses) const
{
	if (UDMMaterialComponent* Parent = GetParentComponent())
	{
		UClass* ParentClass = Parent->GetClass();

		if (ParentClass == InParentClass)
		{
			return Parent;
		}

		if (bInAllowSubclasses && ParentClass->IsChildOf(InParentClass))
		{
			return Parent;
		}

		return Parent->GetTypedParent(InParentClass, bInAllowSubclasses);
	}

	return nullptr;
}

FText UDMMaterialComponent::GetComponentDescription() const
{
	return GetClass()->GetDisplayNameText();
}

FSlateIcon UDMMaterialComponent::GetComponentIcon() const
{
	FSlateIcon Icon = FSlateIconFinder::FindIconForClass(GetClass());

	if (Icon.IsSet())
	{
		return Icon;
	}

	// Fall back to a default icon.
	return FSlateIconFinder::FindIconForClass(UDMMaterialComponent::StaticClass());
}

bool UDMMaterialComponent::CanClean()
{
	return (FPlatformTime::Seconds() >= MinCleanTime);
}

void UDMMaterialComponent::PreventClean(double InDelayFor)
{
	MinCleanTime = FMath::Max(MinCleanTime, FPlatformTime::Seconds() + InDelayFor);
}

bool UDMMaterialComponent::NeedsClean()
{
	if (!IsComponentValid())
	{
		return false;
	}

	return bComponentDirty;
}

void UDMMaterialComponent::DoClean()
{
	bComponentDirty = false;

	if (!IsComponentValid())
	{
		return;
	}

	static const double VeryShortTime = 0.0001;
	
	// Make sure we don't spam updates on a single tick.
	PreventClean(VeryShortTime);
}

void UDMMaterialComponent::SetComponentState(EDMComponentLifetimeState InNewState)
{
	if (ComponentState == InNewState)
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	ComponentState = InNewState;
	OnComponentStateChange(InNewState);
}

void UDMMaterialComponent::PostLoad()
{
	Super::PostLoad();

	SetFlags(RF_Transactional);

	ComponentState = EDMComponentLifetimeState::Added;
	MarkComponentDirty();
}

void UDMMaterialComponent::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	SetFlags(RF_Transactional);
	ComponentState = EDMComponentLifetimeState::Added;
	MarkComponentDirty();
}

bool UDMMaterialComponent::Modify(bool bInAlwaysMarkDirty /*= true*/)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	MarkComponentDirty();

	return bSaved;
}

void UDMMaterialComponent::OnComponentStateChange(EDMComponentLifetimeState InNewState)
{
	if (!IsComponentValid())
	{
		return;
	}

	switch (InNewState)
	{
		case EDMComponentLifetimeState::Added:
			OnComponentAdded();
			break;

		case EDMComponentLifetimeState::Removed:
			OnComponentRemoved();
			break;

		default:
			// Nothing
			break;
	}
}

void UDMMaterialComponent::OnComponentAdded()
{
	OnAdded.Broadcast(this, EDMComponentLifetimeState::Added);
}

void UDMMaterialComponent::OnComponentRemoved()
{
	OnAdded.Broadcast(this, EDMComponentLifetimeState::Removed);
}
#endif // WITH_EDITOR

void UDMMaterialComponent::MarkComponentDirty()
{
#if WITH_EDITOR
	bComponentDirty = true;
	PreventClean();
#endif
}

#undef LOCTEXT_NAMESPACE
