// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialComponentDynamic.h"

#include "DMComponentPath.h"
#include "DynamicMaterialModule.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialComponentDynamic)

const FString UDMMaterialComponentDynamic::ParentValuePathToken = TEXT("ParentValue");

UDMMaterialComponentDynamic::UDMMaterialComponentDynamic()
	: ParentComponentName(NAME_None)
	, ParentComponent(nullptr)
{
}

UDynamicMaterialModelDynamic* UDMMaterialComponentDynamic::GetMaterialModelDynamic() const
{
	return Cast<UDynamicMaterialModelDynamic>(GetOuterSafe());
}

FName UDMMaterialComponentDynamic::GetParentComponentName() const
{
	return ParentComponentName;
}

UDMMaterialComponent* UDMMaterialComponentDynamic::GeResolvedParentComponent() const
{
	if (!ParentComponent && !ParentComponentName.IsNone())
	{
		const_cast<UDMMaterialComponentDynamic*>(this)->ResolveParentComponent();
	}

	return ParentComponent;
}

#if WITH_EDITOR
void UDMMaterialComponentDynamic::PostEditorDuplicate(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	Super::PostEditorDuplicate(InMaterialModelDynamic->GetParentModel(), nullptr);

	if (GetOuter() != InMaterialModelDynamic)
	{
		Rename(nullptr, InMaterialModelDynamic, UE::DynamicMaterial::RenameFlags);
	}
}

void UDMMaterialComponentDynamic::BeginDestroy()
{
	Super::BeginDestroy();

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	UDynamicMaterialModelDynamic* MaterialModelDynamic = GetMaterialModelDynamic();

	if (!MaterialModelDynamic)
	{
		return;
	}

	MaterialModelDynamic->RemoveComponentDynamic(this);
}
#endif

void UDMMaterialComponentDynamic::ResolveParentComponent()
{
	if (ParentComponentName.IsNone())
	{
		return;
	}

	UDynamicMaterialModelDynamic* MaterialModelDynamic = GetMaterialModelDynamic();

	if (!MaterialModelDynamic)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MaterialModelDynamic->GetParentModel();

	if (!MaterialModel)
	{
		return;
	}

	ParentComponent = FindObjectFast<UDMMaterialComponent>(MaterialModel, ParentComponentName);
}

UDMMaterialComponent* UDMMaterialComponentDynamic::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == ParentValuePathToken)
	{
		return GeResolvedParentComponent();
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}
