// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialLinkedComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialLinkedComponent)

#if WITH_EDITOR
UDMMaterialComponent* UDMMaterialLinkedComponent::GetParentComponent() const
{
	return ParentComponent.Get();
}

void UDMMaterialLinkedComponent::SetParentComponent(UDMMaterialComponent* InParentComponent)
{
	ParentComponent = InParentComponent;
}

void UDMMaterialLinkedComponent::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	ParentComponent = InParent;
}
#endif
