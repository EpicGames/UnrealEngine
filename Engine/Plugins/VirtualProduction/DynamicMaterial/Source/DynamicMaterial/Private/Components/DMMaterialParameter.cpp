// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialParameter.h"
#include "Model/DynamicMaterialModel.h"

#if WITH_EDITOR
#include "DynamicMaterialModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialParameter)

UDMMaterialParameter::UDMMaterialParameter()
{
	ParameterName = NAME_None;
}

UDynamicMaterialModel* UDMMaterialParameter::GetMaterialModel() const
{
	return Cast<UDynamicMaterialModel>(GetOuterSafe());
}

#if WITH_EDITOR
void UDMMaterialParameter::RenameParameter(FName InBaseParameterName)
{
	if (ParameterName == InBaseParameterName)
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = GetMaterialModel();
	check(MaterialModel);

#if WITH_EDITOR
	if (GUndo)
	{
		MaterialModel->Modify();
	}
#endif

	MaterialModel->RenameParameter(this, InBaseParameterName);
}

void UDMMaterialParameter::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (GetOuter() != InMaterialModel)
	{
		Rename(nullptr, InMaterialModel, UE::DynamicMaterial::RenameFlags);
	}
}

void UDMMaterialParameter::OnComponentRemoved()
{
	if (ParameterName.IsNone() == false)
	{
		if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
		{
#if WITH_EDITOR
			if (GUndo)
			{
				MaterialModel->Modify();
			}
#endif

			MaterialModel->FreeParameter(this);
		}
	}

	Super::OnComponentRemoved();
}

void UDMMaterialParameter::BeginDestroy()
{
	if (FDynamicMaterialModule::AreUObjectsSafe() && ParameterName.IsNone() == false)
	{
		UDynamicMaterialModel* const MaterialModel = GetMaterialModel();

		if (MaterialModel && !MaterialModel->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
#if WITH_EDITOR
			if (GUndo)
			{
				MaterialModel->Modify();
			}
#endif

			MaterialModel->FreeParameter(this);
		}
	}

	Super::BeginDestroy();
}
#endif
