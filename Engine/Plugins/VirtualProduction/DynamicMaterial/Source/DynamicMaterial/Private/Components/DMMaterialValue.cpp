// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/DMMaterialValue.h"
#include "Components/DMMaterialParameter.h"
#include "DMComponentPath.h"
#include "Model/DynamicMaterialModel.h"
#include "Materials/Material.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "DMValueDefinition.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValue)

#define LOCTEXT_NAMESPACE "DMMaterialValue"

TMap<EDMValueType, TStrongObjectPtr<UClass>> UDMMaterialValue::TypeClasses = {};

const FString UDMMaterialValue::ParameterPathToken = FString(TEXT("Parameter"));

#if WITH_EDITOR
const FName UDMMaterialValue::ValueName = "Value";

UDMMaterialValue* UDMMaterialValue::CreateMaterialValue(UDynamicMaterialModel* InMaterialModel, const FString& InName,
	TSubclassOf<UDMMaterialValue> InValueClass, bool bInLocal)
{
	check(InMaterialModel);
 
	UDMMaterialValue* NewValue = NewObject<UDMMaterialValue>(InMaterialModel, InValueClass, NAME_None, RF_Transactional);
	check(NewValue);
	NewValue->bLocal = bInLocal;
	NewValue->ResetDefaultValue();
	NewValue->ApplyDefaultValue();
 
	if (InName.IsEmpty() == false)
	{
		if (GUndo)
		{
			InMaterialModel->Modify();
		}

		NewValue->Parameter = InMaterialModel->CreateUniqueParameter(*InName);
		NewValue->Parameter->SetParentComponent(NewValue);
		NewValue->CachedParameterName = *InName;
	}
 
	return NewValue;
}
#endif // WITH_EDITOR

UDMMaterialValue::UDMMaterialValue()
	: UDMMaterialValue(EDMValueType::VT_None)
{
}

UDMMaterialValue::UDMMaterialValue(EDMValueType InType)
	: Type(InType)
	, bLocal(false)
	, Parameter(nullptr)
	, CachedParameterName(NAME_None)
#if WITH_EDITORONLY_DATA
	, bExposeParameter(false)
#endif
{
#if WITH_EDITORONLY_DATA
	EditableProperties.Add(ValueName);
#endif
}

UDynamicMaterialModel* UDMMaterialValue::GetMaterialModel() const
{
	return Cast<UDynamicMaterialModel>(GetOuterSafe());
}

#if WITH_EDITOR
FText UDMMaterialValue::GetTypeName() const
{
	return UDMValueDefinitionLibrary::GetValueDefinition(Type).GetDisplayName();
}
 
FText UDMMaterialValue::GetDescription() const
{
	return FText::Format(
		LOCTEXT("ValueDescriptionTemplate", "{0} ({1})"),
		FText::FromName(GetMaterialParameterName()),
		GetTypeName()
	);
}

void UDMMaterialValue::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	// -> represents a child property. Child property resets are handled by individual class implementations.
	if (InPropertyHandle->GetPropertyPath().Find(TEXT("->")) == INDEX_NONE)
	{
		ApplyDefaultValue();
	}
}

void UDMMaterialValue::PostCDOContruct()
{
	Super::PostCDOContruct();

	if (Type != EDMValueType::VT_None)
	{
		TypeClasses.Emplace(Type, TStrongObjectPtr<UClass>(GetClass()));
	}
}

void UDMMaterialValue::PostLoad()
{
	Super::PostLoad();

	if (Parameter)
	{
		if (GUndo)
		{
			Parameter->Modify();
		}

		Parameter->SetParentComponent(this);
	}

	if (bLocal)
	{
		if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
		{
			MaterialModel->AddRuntimeComponentReference(this);
		}
	}
}

void UDMMaterialValue::PostEditImport()
{
	Super::PostEditImport();

	if (Parameter)
	{
		if (GUndo)
		{
			Parameter->Modify();
		}

		Parameter->SetParentComponent(this);
	}
}
#endif // WITH_EDITOR

FName UDMMaterialValue::GetMaterialParameterName() const
{
	if (Parameter)
	{
		return Parameter->GetParameterName();
	}

	if (!CachedParameterName.IsNone())
	{
		return CachedParameterName;
	}

	return GetFName();
}

UDMMaterialComponent* UDMMaterialValue::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == ParameterPathToken)
	{
		return Parameter;
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}
 
#if WITH_EDITOR
bool UDMMaterialValue::SetParameterName(FName InBaseName)
{
	if (Parameter && Parameter->GetParameterName() == InBaseName)
	{
		return false;
	}

	if (!IsComponentValid())
	{
		return false;
	}
 
	UDynamicMaterialModel* MaterialModel = GetMaterialModel();
	check(MaterialModel);

	if (GUndo && IsValid(Parameter))
	{
		Parameter->Modify();
		MaterialModel->Modify();
	}
 
	if (InBaseName.IsNone())
	{
		if (Parameter)
		{
			Parameter->SetParentComponent(nullptr);
			MaterialModel->FreeParameter(Parameter);
			Parameter = nullptr;
		}
	}
	else if (Parameter)
	{
		Parameter->RenameParameter(InBaseName);
	}
	else
	{
		Parameter = MaterialModel->CreateUniqueParameter(InBaseName);
		Parameter->SetParentComponent(this);
	}

	UpdateCachedParameterName(/* Reset Name */ false);
 
	return true;
}

EDMMaterialParameterGroup UDMMaterialValue::GetParameterGroup() const
{
	if (GetMaterialParameterName().ToString().StartsWith("Global"))
	{
		return EDMMaterialParameterGroup::Global;
	}

	if (bExposeParameter)
	{
		return EDMMaterialParameterGroup::Property;
	}

	return EDMMaterialParameterGroup::NotExposed;
}

void UDMMaterialValue::SetShouldExposeParameter(bool bInExpose)
{
	if (bExposeParameter == bInExpose)
	{
		return;
	}

	bExposeParameter = bInExpose;
	
	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialValue::OnComponentAdded()
{
	Super::OnComponentAdded();

	if (bLocal)
	{
		if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
		{
			MaterialModel->AddRuntimeComponentReference(this);
		}

		CachedParameterName = NAME_None;
		UpdateCachedParameterName(/* Reset Name */ true);
	}
}

void UDMMaterialValue::OnComponentRemoved()
{
	if (Parameter)
	{
		if (GUndo)
		{
			Parameter->Modify();
		}

		Parameter->SetComponentState(EDMComponentLifetimeState::Removed);
	}

	if (bLocal)
	{
		if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
		{
			MaterialModel->RemoveRuntimeComponentReference(this);
		}

		CachedParameterName = NAME_None;
	}
 
	Super::OnComponentRemoved();
}
#endif // WITH_EDITOR

TSharedPtr<FJsonValue> UDMMaterialValue::JsonSerialize() const
{
	return nullptr;
}

bool UDMMaterialValue::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	return false;
}

#if WITH_EDITOR
void UDMMaterialValue::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	if (GetOuter() == InMaterialModel)
	{
		Super::PostEditorDuplicate(InMaterialModel, InParent);
		UpdateCachedParameterName(/* Reset Name */ false);
		return;
	}

	FName OldParameterName = NAME_None;

	if (UDMMaterialParameter* Param = GetParameter())
	{
		// Reset this to null as it holds a copy of the parameter from the copied-from object.
		// This will not be in the model's parameter list and will share the same name as the old parameter.
		// Just null the reference and create a new parameter.
		if (InMaterialModel->ConditionalFreeParameter(Param))
		{
			OldParameterName = Param->GetParameterName();
			Parameter = nullptr;
		}
	}

	Super::PostEditorDuplicate(InMaterialModel, InParent);

	Rename(nullptr, InMaterialModel, UE::DynamicMaterial::RenameFlags);

	if (!OldParameterName.IsNone())
	{
		SetParameterName(OldParameterName);
	}

	UpdateCachedParameterName(/* Reset Name */ false);
}

bool UDMMaterialValue::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	if (Parameter)
	{
		Parameter->Modify(bInAlwaysMarkDirty);
	}

	return bSaved;
}

void UDMMaterialValue::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}
 
	MarkComponentDirty();
 
	OnValueChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate); // Just in case - Undos are not meant to be quick and easy.
}
 
void UDMMaterialValue::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (!IsComponentValid())
	{
		return;
	}

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName.IsNone())
	{
		return;
	}

	for (const FName& EditableProperty : EditableProperties)
	{
		if (EditableProperty == MemberPropertyName)
		{
			const EDMUpdateType UpdateType = (EditableProperty == ValueName ? EDMUpdateType::Value : EDMUpdateType::Structure)
				| EDMUpdateType::AllowParentUpdate;

			OnValueChanged(UpdateType);
			return;
		}
	}
}
#endif // WITH_EDITOR
 
void UDMMaterialValue::OnValueChanged(EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	Update(this, InUpdateType);

#if WITH_EDITOR
	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::AllowParentUpdate) && ParentComponent)
	{
		ParentComponent->Update(this, InUpdateType);
	}
#endif
}

#if WITH_EDITOR
FName UDMMaterialValue::GenerateAutomaticParameterName() const
{
	return *GetComponentPath();
}

void UDMMaterialValue::UpdateCachedParameterName(bool bInResetName)
{
	if (Parameter)
	{
		CachedParameterName = Parameter->GetParameterName();
	}
	else if (bInResetName || CachedParameterName.IsNone())
	{
		CachedParameterName = GenerateAutomaticParameterName();
	}
}
#endif

void UDMMaterialValue::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

#if WITH_EDITOR
	if (HasComponentBeenRemoved())
	{
		return;
	}

	MarkComponentDirty();

	if (InUpdateType == EDMUpdateType::Structure)
	{
		UpdateCachedParameterName(/* Reset Name */ false);
	}
#endif

	Super::Update(InSource, InUpdateType);

	if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		MaterialModel->OnValueUpdated(this, InUpdateType);
	}
}

#if WITH_EDITOR
int32 UDMMaterialValue::GetInnateMaskOutput(int32 OutputChannels) const
{
	return INDEX_NONE;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
