// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialValueDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"

#if WITH_EDITOR
#include "Materials/Material.h"
#include "PropertyHandle.h"
#include "UObject/Package.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueDynamic)

#if WITH_EDITOR
UDMMaterialValueDynamic* UDMMaterialValueDynamic::CreateValueDynamic(TSubclassOf<UDMMaterialValueDynamic> InInstanceValueClass, 
	UDynamicMaterialModelDynamic* InMaterialModelDynamic, UDMMaterialValue* InParentValue)
{
	if (!IsValid(InMaterialModelDynamic))
	{
		return nullptr;
	}

	if (!IsValid(InParentValue))
	{
		return nullptr;
	}

	UClass* InstanceValueClass = InInstanceValueClass.Get();

	if (!InInstanceValueClass)
	{
		return nullptr;
	}

	UDMMaterialValueDynamic* NewValueDynamic = NewObject<UDMMaterialValueDynamic>(InMaterialModelDynamic, InstanceValueClass, NAME_None, RF_Transactional);
	NewValueDynamic->ParentComponent = InParentValue;
	NewValueDynamic->ParentComponentName = InParentValue->GetFName();
	NewValueDynamic->ApplyDefaultValue();

	InMaterialModelDynamic->AddComponentDynamic(NewValueDynamic);

	return NewValueDynamic;
}

void UDMMaterialValueDynamic::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (InPropertyHandle->GetPropertyPath().Find(TEXT("->")) == INDEX_NONE)
	{
		ApplyDefaultValue();
	}
}
#endif

UDMMaterialValue* UDMMaterialValueDynamic::GetParentValue() const
{
	return Cast<UDMMaterialValue>(GeResolvedParentComponent());
}

void UDMMaterialValueDynamic::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

#if WITH_EDITOR
	if (HasComponentBeenRemoved())
	{
		return;
	}

	MarkComponentDirty();
#endif

	Super::Update(InSource, InUpdateType);

	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = GetMaterialModelDynamic())
	{
		MaterialModelDynamic->OnValueUpdated(this);
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueDynamic::JsonSerialize() const
{
	return nullptr;
}

bool UDMMaterialValueDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	return false;
}

#if WITH_EDITOR
void UDMMaterialValueDynamic::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	MarkComponentDirty();

	OnValueChanged();
}

void UDMMaterialValueDynamic::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (!IsComponentValid())
	{
		return;
	}

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == UDMMaterialValue::ValueName)
	{
		OnValueChanged();
	}
}
#endif

void UDMMaterialValueDynamic::OnValueChanged()
{
	if (!IsComponentValid())
	{
		return;
	}

	Update(this, EDMUpdateType::Value);
}

#if WITH_EDITOR
void UDMMaterialValueDynamic::OnComponentAdded()
{
	Super::OnComponentAdded();

	OnValueChanged();
}
#endif
