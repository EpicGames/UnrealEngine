// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMRenderTargetRenderer.h"

#include "Components/MaterialValues/DMMaterialValueRenderTarget.h"
#include "Misc/CoreDelegates.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMRenderTargetRenderer)

#define LOCTEXT_NAMESPACE "DMRenderTargetRenderer"

UDMRenderTargetRenderer* UDMRenderTargetRenderer::CreateRenderTargetRenderer(TSubclassOf<UDMRenderTargetRenderer> InRendererClass, 
	UDMMaterialValueRenderTarget* InRenderTargetValue)
{
	UClass* Class = InRendererClass.Get();

	check(Class);
	check(!Class->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	check(InRenderTargetValue);

	UDMRenderTargetRenderer* Renderer = NewObject<UDMRenderTargetRenderer>(InRenderTargetValue, Class, NAME_None, RF_Transactional);
	InRenderTargetValue->SetRenderer(Renderer);

	return Renderer;
}

UDMMaterialValueRenderTarget* UDMRenderTargetRenderer::GetRenderTargetValue() const
{
	return Cast<UDMMaterialValueRenderTarget>(GetOuterSafe());
}

void UDMRenderTargetRenderer::UpdateRenderTarget()
{
	if (EndOfFrameDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(EndOfFrameDelegateHandle);
		EndOfFrameDelegateHandle.Reset();
	}

	if (bUpdating)
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdating, true);

	UpdateRenderTarget_Internal();
}

void UDMRenderTargetRenderer::AsyncUpdateRenderTarget()
{
	if (bUpdating)
	{
		return;
	}

	if (!EndOfFrameDelegateHandle.IsValid())
	{
		EndOfFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &UDMRenderTargetRenderer::UpdateRenderTarget);
	}
}

void UDMRenderTargetRenderer::FlushUpdateRenderTarget()
{
	if (EndOfFrameDelegateHandle.IsValid())
	{
		UpdateRenderTarget();
	}
}

TSharedPtr<FJsonValue> UDMRenderTargetRenderer::JsonSerialize() const
{
	return nullptr;
}

bool UDMRenderTargetRenderer::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	return false;
}

#if WITH_EDITOR
FText UDMRenderTargetRenderer::GetComponentDescription() const
{
	return LOCTEXT("Renderer", "Renderer");
}
#endif

void UDMRenderTargetRenderer::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

#if WITH_EDITOR
	if (HasComponentBeenRemoved())
	{
		return;
	}

	MarkComponentDirty();
#endif

	if (UDMMaterialValueRenderTarget* RenderTarget = GetRenderTargetValue())
	{
		RenderTarget->Update(InSource, InUpdateType);
	}

	Super::Update(InSource, InUpdateType);
}

void UDMRenderTargetRenderer::PostLoad()
{
	Super::PostLoad();

	if (UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue())
	{
		RenderTargetValue->EnsureRenderTarget();
	}
}

#undef LOCTEXT_NAMESPACE
