// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMTextureUV.h"
#include "Components/DMMaterialParameter.h"
#include "Components/DMTextureUVDynamic.h"
#include "DMComponentPath.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Model/DynamicMaterialModel.h"
#include "Serialization/CustomVersion.h"

#if WITH_EDITOR
#include "Model/IDMMaterialBuildUtilsInterface.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureUV)

namespace UE::DynamicMaterial::Private
{
	TMap<int32, FName> BaseParameterNames = {
		{ParamID::PivotX,   FName(TEXT("Pivot.X"))},
		{ParamID::PivotY,   FName(TEXT("Pivot.Y"))},
		{ParamID::TilingX,  FName(TEXT("Tiling.X"))},
		{ParamID::TilingY,  FName(TEXT("Tiling.Y"))},
		{ParamID::Rotation, FName(TEXT("Rotation"))},
		{ParamID::OffsetX,  FName(TEXT("Offset.X"))},
		{ParamID::OffsetY,  FName(TEXT("Offset.Y"))},
	};
}

enum class EDMTextureUVVersion : int32
{
	Initial_Pre_20221102 = 0,
	Version_22021102 = 1,
	ScaleToTiling = 2,
	LatestVersion = ScaleToTiling
};

const FGuid UDMTextureUV::GUID(0xFCF57AFB, 0x50764284, 0xB9A9E659, 0xFFA02D33);
FCustomVersionRegistration GRegisterDMTextureUVVersion(UDMTextureUV::GUID, static_cast<int32>(EDMTextureUVVersion::LatestVersion), TEXT("DMTextureUV"));

const FString UDMTextureUV::OffsetXPathToken  = FString(TEXT("OffsetX"));
const FString UDMTextureUV::OffsetYPathToken  = FString(TEXT("OffsetY"));
const FString UDMTextureUV::PivotXPathToken   = FString(TEXT("PivotX"));
const FString UDMTextureUV::PivotYPathToken   = FString(TEXT("PivotY"));
const FString UDMTextureUV::RotationPathToken = FString(TEXT("Rotation"));
const FString UDMTextureUV::TilingXPathToken   = FString(TEXT("Tiling"));
const FString UDMTextureUV::TilingYPathToken   = FString(TEXT("TilingY"));

const FName UDMTextureUV::NAME_Offset     = GET_MEMBER_NAME_CHECKED(UDMTextureUV, Offset);
const FName UDMTextureUV::NAME_Pivot      = GET_MEMBER_NAME_CHECKED(UDMTextureUV, Pivot);
const FName UDMTextureUV::NAME_Rotation   = GET_MEMBER_NAME_CHECKED(UDMTextureUV, Rotation);
const FName UDMTextureUV::NAME_Tiling     = GET_MEMBER_NAME_CHECKED(UDMTextureUV, Tiling);

#if WITH_EDITOR
const FName UDMTextureUV::NAME_UVSource   = GET_MEMBER_NAME_CHECKED(UDMTextureUV, UVSource);
const FName UDMTextureUV::NAME_bMirrorOnX = GET_MEMBER_NAME_CHECKED(UDMTextureUV, bMirrorOnX);
const FName UDMTextureUV::NAME_bMirrorOnY = GET_MEMBER_NAME_CHECKED(UDMTextureUV, bMirrorOnY);

const TMap<FName, bool> UDMTextureUV::TextureProperties = {
	{NAME_UVSource,   false},
	{NAME_Offset,     true},
	{NAME_Pivot,      true},
	{NAME_Rotation,   true},
	{NAME_Tiling,     true},
	{NAME_bMirrorOnX, false},
	{NAME_bMirrorOnY, false}
};
#endif

UDMTextureUV::UDMTextureUV()
{
#if WITH_EDITOR
	EditableProperties.Add(NAME_Offset);
	EditableProperties.Add(NAME_Pivot);
	EditableProperties.Add(NAME_Rotation);
	EditableProperties.Add(NAME_Tiling);
	EditableProperties.Add(NAME_bMirrorOnX);
	EditableProperties.Add(NAME_bMirrorOnY);
#endif
}

#if WITH_EDITORONLY_DATA
void UDMTextureUV::SetUVSource(EDMUVSource InUVSource)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (UVSource == InUVSource)
	{
		return;
	}

	UVSource = InUVSource;

	OnTextureUVChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}
#endif

void UDMTextureUV::SetOffset(const FVector2D& InOffset)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Offset.Equals(InOffset))
	{
		return;
	}

	Offset = InOffset;

	OnTextureUVChanged(EDMUpdateType::Value);
}

void UDMTextureUV::SetPivot(const FVector2D& InPivot)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Pivot.Equals(InPivot))
	{
		return;
	}

	Pivot = InPivot;

	OnTextureUVChanged(EDMUpdateType::Value);
}

void UDMTextureUV::SetRotation(float InRotation)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (FMath::IsNearlyEqual(Rotation, InRotation))
	{
		return;
	}

	Rotation = InRotation;

	OnTextureUVChanged(EDMUpdateType::Value);
}

void UDMTextureUV::SetTiling(const FVector2D& InTiling)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Tiling.Equals(InTiling))
	{
		return;
	}

	Tiling = InTiling;

	OnTextureUVChanged(EDMUpdateType::Value);
}

#if WITH_EDITORONLY_DATA
void UDMTextureUV::SetMirrorOnX(bool bInMirrorOnX)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (bMirrorOnX == bInMirrorOnX)
	{
		return;
	}

	bMirrorOnX = bInMirrorOnX;

	OnTextureUVChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}

void UDMTextureUV::SetMirrorOnY(bool bInMirrorOnY)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (bMirrorOnY == bInMirrorOnY)
	{
		return;
	}

	bMirrorOnY = bInMirrorOnY;

	OnTextureUVChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}
#endif

TArray<UDMMaterialParameter*> UDMTextureUV::GetParameters() const
{
	TArray<UDMMaterialParameter*> Parameters;
	Parameters.Reserve(MaterialParameters.Num());

	for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& Pair : MaterialParameters)
	{
		Parameters.Add(Pair.Value.Get());
	}

	return Parameters;
}

UDMMaterialParameter* UDMTextureUV::GetMaterialParameter(FName InPropertyName, int32 InComponent) const
{
	if (const TObjectPtr<UDMMaterialParameter>* ParameterPtr = MaterialParameters.Find(PropertyComponentToParamId(InPropertyName, InComponent)))
	{
		return ParameterPtr->Get();
	}

	return nullptr;
}

FName UDMTextureUV::GetMaterialParameterName(FName InPropertyName, int32 InComponent) const
{
	if (UDMMaterialParameter* Parameter = GetMaterialParameter(InPropertyName, InComponent))
	{
		return Parameter->GetParameterName();
	}

	if (const FName* CachedNamePtr = CachedParameterNames.Find(PropertyComponentToParamId(InPropertyName, InComponent)))
	{
		return *CachedNamePtr;
	}

	using namespace UE::DynamicMaterial::Private;

	const int32 ParamId = PropertyComponentToParamId(InPropertyName, InComponent);

	if (const FName* NamePtr = BaseParameterNames.Find(ParamId))
	{
		return *NamePtr;
	}

	return TEXT("Error");
}

#if WITH_EDITOR
bool UDMTextureUV::SetMaterialParameterName(FName InPropertyName, int32 InComponent, FName InNewName)
{
	if (!IsComponentValid())
	{
		return false;
	}

	const int32 ParamId = PropertyComponentToParamId(InPropertyName, InComponent);

	if (ParamId == UE::DynamicMaterial::ParamID::Invalid)
	{
		return false;
	}

	UDMMaterialParameter* Parameter = GetMaterialParameter(InPropertyName, InComponent);

	if (Parameter && Parameter->GetParameterName() == InNewName)
	{
		return false;
	}

	UDynamicMaterialModel* MaterialModel = GetMaterialModel();
	check(MaterialModel);

	if (GUndo && IsValid(Parameter))
	{
		Parameter->Modify();
		MaterialModel->Modify();
	}

	if (InNewName.IsNone())
	{
		if (Parameter)
		{
			Parameter->SetParentComponent(nullptr);
			MaterialModel->FreeParameter(Parameter);
			Parameter = nullptr;
			MaterialParameters.Remove(ParamId);
		}
	}
	else if (Parameter)
	{
		Parameter->RenameParameter(InNewName);
	}
	else
	{
		Parameter = MaterialModel->CreateUniqueParameter(InNewName);
		Parameter->SetParentComponent(this);
		MaterialParameters.Add(ParamId, Parameter);
	}

	UpdateCachedParameterName(InPropertyName, InComponent);

	return true;
}

EDMMaterialParameterGroup UDMTextureUV::GetParameterGroup(FName InPropertyName, int32 InComponent) const
{
	return GetShouldExposeParameter(InPropertyName, InComponent)
		? EDMMaterialParameterGroup::Property
		: EDMMaterialParameterGroup::NotExposed;
}

bool UDMTextureUV::GetShouldExposeParameter(FName InPropertyName, int32 InComponent) const
{
	return ExposedParameters.Contains(PropertyComponentToParamId(InPropertyName, InComponent));
}

void UDMTextureUV::SetShouldExposeParameter(FName InPropertyName, int32 InComponent, bool bInExpose)
{
	const int32 ParamId = PropertyComponentToParamId(InPropertyName, InComponent);

	if (bInExpose)
	{
		ExposedParameters.Add(ParamId);
	}
	else
	{
		ExposedParameters.Remove(ParamId);
	}

	Update(this, EDMUpdateType::Structure);
}

void UDMTextureUV::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	if (GetOuter() == InMaterialModel)
	{
		Super::PostEditorDuplicate(InMaterialModel, InParent);
		UpdateCachedParameterNames(/* Reset names */ false);
		return;
	}

	TMap<int32, FName> OldParameterNames;
	OldParameterNames.Reserve(MaterialParameters.Num());

	// Reset these to null as they holds a copy of the parameter from the copied-from object.
	// This will not be in the model's parameter list and will share the same name as the old parameter.
	// Just null the reference and create a new parameter.
	for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& ParameterPair : MaterialParameters)
	{
		OldParameterNames.Add(ParameterPair.Key, ParameterPair.Value->GetParameterName());
	}

	MaterialParameters.Empty();

	Super::PostEditorDuplicate(InMaterialModel, InParent);

	Rename(nullptr, InMaterialModel, UE::DynamicMaterial::RenameFlags);

	UpdateCachedParameterNames(/* Reset names */ false);

	using namespace UE::DynamicMaterial;

	for (const TPair<int32, FName>& OldParameterPair : OldParameterNames)
	{
		if (!OldParameterPair.Value.IsNone())
		{
			switch (OldParameterPair.Key)
			{
				case ParamID::OffsetX:
					SetMaterialParameterName(UDMTextureUV::NAME_Offset, 0, OldParameterPair.Value);
					break;

				case ParamID::OffsetY:
					SetMaterialParameterName(UDMTextureUV::NAME_Offset, 1, OldParameterPair.Value);
					break;

				case ParamID::Rotation:
					SetMaterialParameterName(UDMTextureUV::NAME_Rotation, 0, OldParameterPair.Value);
					break;

				case ParamID::PivotX:
					SetMaterialParameterName(UDMTextureUV::NAME_Pivot, 0, OldParameterPair.Value);
					break;

				case ParamID::PivotY:
					SetMaterialParameterName(UDMTextureUV::NAME_Pivot, 1, OldParameterPair.Value);
					break;

				case ParamID::TilingX:
					SetMaterialParameterName(UDMTextureUV::NAME_Tiling, 0, OldParameterPair.Value);
					break;

				case ParamID::TilingY:
					SetMaterialParameterName(UDMTextureUV::NAME_Tiling, 1, OldParameterPair.Value);
					break;
			}
		}
	}
}
#endif

void UDMTextureUV::SetMIDParameters(UMaterialInstanceDynamic* InMID)
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);

	auto UpdateMID = [InMID](FName InParamName, float InValue)
		{
			if (FMath::IsNearlyEqual(InValue, InMID->K2_GetScalarParameterValue(InParamName)) == false)
			{
				InMID->SetScalarParameterValue(InParamName, InValue);
			}
		};

	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Offset,   0), GetOffset().X);
	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Offset,   1), GetOffset().Y);
	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Rotation, 0), GetRotation());
	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Pivot,    0), GetPivot().X);
	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Pivot,    1), GetPivot().Y);
	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Tiling,   0), GetTiling().X);
	UpdateMID(GetMaterialParameterName(UDMTextureUV::NAME_Tiling,   1), GetTiling().Y);
}

#if WITH_EDITOR
UDMTextureUVDynamic* UDMTextureUV::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	return UDMTextureUVDynamic::CreateTextureUVDynamic(InMaterialModelDynamic, this);
}

bool UDMTextureUV::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& Pair : MaterialParameters)
	{
		Pair.Value->Modify(bInAlwaysMarkDirty);
	}

	return bSaved;
}
#endif

void UDMTextureUV::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

#if WITH_EDITOR
	if (HasComponentBeenRemoved())
	{
		return;
	}

	MarkComponentDirty();

	if (InUpdateType == EDMUpdateType::Structure)
	{
		UpdateCachedParameterNames(/* Reset Names */ false);
	}
#endif

	Super::Update(InSource, InUpdateType);

	if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		MaterialModel->OnTextureUVUpdated(this);
	}
}

int32 UDMTextureUV::PropertyComponentToParamId(FName InPropertyName, int32 InComponent)
{
	using namespace UE::DynamicMaterial;

	if (InPropertyName == NAME_Offset)
	{
		switch (InComponent)
		{
			case 0:
				return ParamID::OffsetX;

			case 1:
				return ParamID::OffsetY;
		}
	}
	else if (InPropertyName == NAME_Pivot)
	{
		switch (InComponent)
		{
			case 0:
				return ParamID::PivotX;

			case 1:
				return ParamID::PivotY;
		}
	}
	else if (InPropertyName == NAME_Rotation)
	{
		switch (InComponent)
		{
			case 0:
				return ParamID::Rotation;
		}
	}
	else if (InPropertyName == NAME_Tiling)
	{
		switch (InComponent)
		{
			case 0:
				return ParamID::TilingX;

			case 1:
				return ParamID::TilingY;
		}
	}

	return ParamID::Invalid;
}

UDynamicMaterialModel* UDMTextureUV::GetMaterialModel() const
{
	return Cast<UDynamicMaterialModel>(GetOuterSafe());
}

UDMMaterialComponent* UDMTextureUV::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	auto GetParameter = [this, &InPath, &InPathSegment](int32 InParamId) -> UDMMaterialComponent*
		{
			if (const TObjectPtr<UDMMaterialParameter>* ParameterPtr = MaterialParameters.Find(InParamId))
			{
				return *ParameterPtr;
			}

			return UDMMaterialLinkedComponent::GetSubComponentByPath(InPath, InPathSegment);
		};

	using namespace UE::DynamicMaterial;

	if (InPathSegment.GetToken() == OffsetXPathToken)
	{
		return GetParameter(ParamID::OffsetX);
	}

	if (InPathSegment.GetToken() == OffsetYPathToken)
	{
		return GetParameter(ParamID::OffsetY);
	}

	if (InPathSegment.GetToken() == PivotXPathToken)
	{
		return GetParameter(ParamID::PivotX);
	}

	if (InPathSegment.GetToken() == PivotYPathToken)
	{
		return GetParameter(ParamID::PivotY);
	}

	if (InPathSegment.GetToken() == RotationPathToken)
	{
		return GetParameter(ParamID::Rotation);
	}

	if (InPathSegment.GetToken() == TilingXPathToken)
	{
		return GetParameter(ParamID::TilingX);
	}

	if (InPathSegment.GetToken() == TilingYPathToken)
	{
		return GetParameter(ParamID::TilingY);
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

#if WITH_EDITOR
void UDMTextureUV::GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const
{
	using namespace UE::DynamicMaterial::Private;

	// Replace parameter object names with the base parameter name
	if (OutChildComponentPathComponents.IsEmpty() == false)
	{
		FString& LastPathComponent = OutChildComponentPathComponents.Last();

		for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& MaterialParameter : MaterialParameters)
		{
			if (LastPathComponent == MaterialParameter.Value->GetComponentPathComponent())
			{
				LastPathComponent = BaseParameterNames[MaterialParameter.Key].ToString();
				break;
			}
		}
	}

	Super::GetComponentPathInternal(OutChildComponentPathComponents);
}

void UDMTextureUV::RemoveParameterNames()
{
	if (!IsComponentValid())
	{
		return;
	}

	CachedParameterNames.Empty();

	UDynamicMaterialModel* MaterialModel = GetMaterialModel();
	check(MaterialModel);

	if (GUndo)
	{
		MaterialModel->Modify();
	}

	for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& Pair : MaterialParameters)
	{
		if (GUndo)
		{
			Pair.Value->Modify();
		}

		MaterialModel->FreeParameter(Pair.Value);
	}
}
#endif

void UDMTextureUV::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMTextureUV* OtherTextureUV = CastChecked<UDMTextureUV>(InOther);
	OtherTextureUV->SetOffset(Offset);
	OtherTextureUV->SetTiling(Tiling);
	OtherTextureUV->SetPivot(Pivot);
	OtherTextureUV->SetRotation(Rotation);
}

void UDMTextureUV::OnTextureUVChanged(EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	Update(this, InUpdateType);

#if WITH_EDITOR
	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::AllowParentUpdate) && ParentComponent)
	{
		ParentComponent->Update(this, InUpdateType);
	}
#endif
}

#if WITH_EDITOR
FName UDMTextureUV::GenerateAutomaticPathComponent(FName InPropertyName, int32 InComponent) const
{
	using namespace UE::DynamicMaterial::Private;

	if (const FName* NamePtr = BaseParameterNames.Find(PropertyComponentToParamId(InPropertyName, InComponent)))
	{
		return *NamePtr;
	}

	return TEXT("Error");
}

FName UDMTextureUV::GenerateAutomaticParameterName(FName InPropertyName, int32 InComponent) const
{
	return *(GetComponentPath() + TEXT(".") + GenerateAutomaticPathComponent(InPropertyName, InComponent).ToString());
}

void UDMTextureUV::UpdateCachedParameterName(FName InPropertyName, int32 InComponent)
{
	const int32 ParamId = PropertyComponentToParamId(InPropertyName, InComponent);

	if (ParamId == UE::DynamicMaterial::ParamID::Invalid)
	{
		return;
	}

	if (const TObjectPtr<UDMMaterialParameter>* ParameterPtr = MaterialParameters.Find(ParamId))
	{
		CachedParameterNames.FindOrAdd(ParamId) = (*ParameterPtr)->GetParameterName();
	}
	else if (!CachedParameterNames.Contains(ParamId))
	{
		CachedParameterNames.Add(ParamId, GenerateAutomaticParameterName(InPropertyName, InComponent));
	}
}

void UDMTextureUV::UpdateCachedParameterNames(bool bInResetNames)
{
	if (bInResetNames)
	{
		CachedParameterNames.Empty(7);
	}

	UpdateCachedParameterName(UDMTextureUV::NAME_Offset, 0);
	UpdateCachedParameterName(UDMTextureUV::NAME_Offset, 1);
	UpdateCachedParameterName(UDMTextureUV::NAME_Rotation, 0);
	UpdateCachedParameterName(UDMTextureUV::NAME_Pivot, 0);
	UpdateCachedParameterName(UDMTextureUV::NAME_Pivot, 1);
	UpdateCachedParameterName(UDMTextureUV::NAME_Tiling, 0);
	UpdateCachedParameterName(UDMTextureUV::NAME_Tiling, 1);
}

void UDMTextureUV::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	UpdateCachedParameterNames(/* Reset Names */ true);

	if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		MaterialModel->AddRuntimeComponentReference(this);
	}
	
	Super::OnComponentAdded();
}

void UDMTextureUV::OnComponentRemoved()
{
	RemoveParameterNames();

	if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		MaterialModel->RemoveRuntimeComponentReference(this);
	}

	Super::OnComponentRemoved();
}

void UDMTextureUV::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (!IsComponentValid())
	{
		return;
	}

	if (!InPropertyChangedEvent.MemberProperty)
	{
		return;
	}

	if (InPropertyChangedEvent.MemberProperty->GetFName() == NAME_Offset
		|| InPropertyChangedEvent.MemberProperty->GetFName() == NAME_Pivot
		|| InPropertyChangedEvent.MemberProperty->GetFName() == NAME_Rotation
		|| InPropertyChangedEvent.MemberProperty->GetFName() == NAME_Tiling)
	{
		OnTextureUVChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
	}
	else if (InPropertyChangedEvent.MemberProperty->GetFName() == NAME_UVSource
		|| InPropertyChangedEvent.MemberProperty->GetFName() == NAME_bMirrorOnX
		|| InPropertyChangedEvent.MemberProperty->GetFName() == NAME_bMirrorOnY)
	{
		OnTextureUVChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
	}
}

void UDMTextureUV::PreEditUndo()
{
	Super::PreEditUndo();

	UVSource_PreUndo = UVSource;
	bMirrorOnX_PreUndo = bMirrorOnX;
	bMirrorOnY_PreUndo = bMirrorOnY;
}

void UDMTextureUV::PostEditUndo()
{
	Super::PostEditUndo();

	if (UVSource != UVSource_PreUndo
		|| bMirrorOnX != bMirrorOnX_PreUndo
		|| bMirrorOnY != bMirrorOnY_PreUndo)
	{
		OnTextureUVChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
	}
	else
	{
		OnTextureUVChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
	}
}

void UDMTextureUV::PostLoad()
{
	Super::PostLoad();

	if (!IsComponentValid())
	{
		return;
	}

	if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		MaterialModel->AddRuntimeComponentReference(this);
	}

	UpdateCachedParameterNames(/* Reset Names */ false);

	for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& Pair : MaterialParameters)
	{
		Pair.Value->SetParentComponent(this);
	}

	/*
	 * @TODO GetLinkerCustomVersion() isn't used here to trigger these updates because it always returns 
	 * the latest version regardless of what was saved to the archive.
	 * Inside the function, it is unable to find a Loader and thus fails in this way.
	 */

	if (bNeedsPostLoadStructureUpdate)
	{
		OnTextureUVChanged(EDMUpdateType::Structure);
	}
	else if (bNeedsPostLoadValueUpdate)
	{
		OnTextureUVChanged(EDMUpdateType::Value);
	}

	bNeedsPostLoadStructureUpdate = false;
	bNeedsPostLoadValueUpdate = false;
}

void UDMTextureUV::PostEditImport()
{
	Super::PostEditImport();

	if (!IsComponentValid())
	{
		return;
	}

	UpdateCachedParameterNames(/* Reset Names */ false);

	for (const TPair<int32, TObjectPtr<UDMMaterialParameter>>& Pair : MaterialParameters)
	{
		Pair.Value->SetParentComponent(this);
	}
}

UDMTextureUV* UDMTextureUV::CreateTextureUV(UObject* InOuter)
{
	UDMTextureUV* NewTextureUV = NewObject<UDMTextureUV>(InOuter, NAME_None, RF_Transactional);
	check(NewTextureUV);

	return NewTextureUV;
}

FString UDMTextureUV::GetComponentPathComponent() const
{
	return TEXT("UV");
}
#endif

void UDMTextureUV::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(UDMTextureUV::GUID);

	Super::Serialize(Ar);

	// @See UDMTextureUV::PostLoad

	int32 TextureUVVersion = Ar.CustomVer(UDMTextureUV::GUID);

	while (TextureUVVersion != static_cast<int32>(EDMTextureUVVersion::LatestVersion))
	{
		switch (TextureUVVersion)
		{
			case INDEX_NONE:
			case static_cast<int32>(EDMTextureUVVersion::Initial_Pre_20221102):
				Offset.X *= -1;
				Rotation *= -360.f;
				Tiling = FVector2D(1.f, 1.f) / Tiling;
				++TextureUVVersion;
#if WITH_EDITORONLY_DATA
				bNeedsPostLoadValueUpdate = true;
#endif
				break;

			case static_cast<int32>(EDMTextureUVVersion::Version_22021102):
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Tiling.X = (Scale.X != 0.f) ? (1.f / Scale.X) : 1.f;
				Tiling.Y = (Scale.Y != 0.f) ? (1.f / Scale.Y) : 1.f;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				++TextureUVVersion;
				break;

			case static_cast<int32>(EDMTextureUVVersion::LatestVersion):
				// Do nothing
				break;

			default:
				TextureUVVersion = static_cast<int32>(EDMTextureUVVersion::LatestVersion);
				break;
		}
	}
}
