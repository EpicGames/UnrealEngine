// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMTextureUVDynamic.h"

#include "Components/DMMaterialParameter.h"
#include "Components/DMTextureUV.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureUVDynamic)

#if WITH_EDITOR
UDMTextureUVDynamic* UDMTextureUVDynamic::CreateTextureUVDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic, 
	UDMTextureUV* InParentTextureUV)
{
	if (!IsValid(InParentTextureUV))
	{
		return nullptr;
	}

	UDMTextureUVDynamic* NewTextureUVDynamic = NewObject<UDMTextureUVDynamic>(InMaterialModelDynamic, UDMTextureUVDynamic::StaticClass(), NAME_None, RF_Transactional);
	NewTextureUVDynamic->ParentComponent = InParentTextureUV;
	NewTextureUVDynamic->ParentComponentName = InParentTextureUV->GetFName();

	InMaterialModelDynamic->AddComponentDynamic(NewTextureUVDynamic);

	return NewTextureUVDynamic;
}
#endif

UDMTextureUV* UDMTextureUVDynamic::GetParentTextureUV() const
{
	return Cast<UDMTextureUV>(GeResolvedParentComponent());
}

void UDMTextureUVDynamic::SetOffset(const FVector2D& InOffset)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Offset.Equals(InOffset))
	{
		return;
	}

	Offset = InOffset;

	OnTextureUVChanged();
}

void UDMTextureUVDynamic::SetPivot(const FVector2D& InPivot)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Pivot.Equals(InPivot))
	{
		return;
	}

	Pivot = InPivot;

	OnTextureUVChanged();
}

void UDMTextureUVDynamic::SetRotation(float InRotation)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (FMath::IsNearlyEqual(Rotation, InRotation))
	{
		return;
	}

	Rotation = InRotation;

	OnTextureUVChanged();
}

void UDMTextureUVDynamic::SetTiling(const FVector2D& InTiling)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Tiling.Equals(InTiling))
	{
		return;
	}

	Tiling = InTiling;

	OnTextureUVChanged();
}

void UDMTextureUVDynamic::SetMIDParameters(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMTextureUV* ParentTextureUV = GetParentTextureUV();

	if (!ParentTextureUV)
	{
		return;
	}

	TArray<UDMMaterialParameter*> MaterialParameters = ParentTextureUV->GetParameters();

	check(InMID);

	auto UpdateMID = [InMID](FName InParamName, float InValue)
		{
			if (FMath::IsNearlyEqual(InValue, InMID->K2_GetScalarParameterValue(InParamName)) == false)
			{
				InMID->SetScalarParameterValue(InParamName, InValue);
			}
		};

	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Offset, 0), GetOffset().X);
	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Offset, 1), GetOffset().Y);
	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Rotation, 0), GetRotation());
	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Pivot, 0), GetPivot().X);
	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Pivot, 1), GetPivot().Y);
	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Tiling, 0), GetTiling().X);
	UpdateMID(ParentTextureUV->GetMaterialParameterName(UDMTextureUV::NAME_Tiling, 1), GetTiling().Y);
}

#if WITH_EDITOR
void UDMTextureUVDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	UDMTextureUV* DestinationTextureUV = Cast<UDMTextureUV>(InDestinationComponent);

	if (!DestinationTextureUV)
	{
		return;
	}

	{
		const FDMUpdateGuard Guard;

		DestinationTextureUV->SetOffset(GetOffset());
		DestinationTextureUV->SetTiling(GetTiling());
		DestinationTextureUV->SetRotation(GetRotation());
		DestinationTextureUV->SetPivot(GetPivot());
	}

	DestinationTextureUV->Update(DestinationTextureUV, EDMUpdateType::Value);
}
#endif

void UDMTextureUVDynamic::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

#if WITH_EDITOR
	if (HasComponentBeenRemoved())
	{
		return;
	}

	MarkComponentDirty();
#endif

	Super::Update(InSource, InUpdateType);

	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = GetMaterialModelDynamic())
	{
		MaterialModelDynamic->OnTextureUVUpdated(this);
	}
}

#if WITH_EDITOR
void UDMTextureUVDynamic::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	MarkComponentDirty();

	OnTextureUVChanged();
}

void UDMTextureUVDynamic::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (!IsComponentValid())
	{
		return;
	}

	if (!InPropertyChangedEvent.MemberProperty)
	{
		return;
	}

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == UDMTextureUV::NAME_Offset
		|| MemberName == UDMTextureUV::NAME_Pivot
		|| MemberName == UDMTextureUV::NAME_Rotation
		|| MemberName == UDMTextureUV::NAME_Tiling)
	{
		OnTextureUVChanged();
	}
}
#endif

void UDMTextureUVDynamic::OnTextureUVChanged()
{
	if (!IsComponentValid())
	{
		return;
	}

	if (FDMUpdateGuard::CanUpdate())
	{
		Update(this, EDMUpdateType::Value);
	}
}

void UDMTextureUVDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMTextureUVDynamic* OtherTextureUVDynamic = CastChecked<UDMTextureUVDynamic>(InOther);
	OtherTextureUVDynamic->SetOffset(Offset);
	OtherTextureUVDynamic->SetTiling(Tiling);
	OtherTextureUVDynamic->SetPivot(Pivot);
	OtherTextureUVDynamic->SetRotation(Rotation);
}

#if WITH_EDITOR
void UDMTextureUVDynamic::OnComponentAdded()
{
	Super::OnComponentAdded();

	OnTextureUVChanged();
}
#endif
