// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueBool.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueBoolDynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueBool)
 
#define LOCTEXT_NAMESPACE "DMMaterialValueBool"

UDMMaterialValueBool::UDMMaterialValueBool()
	: UDMMaterialValue(EDMValueType::VT_Bool)
	, Value(false)
#if WITH_EDITORONLY_DATA
	, bDefaultValue(false)
#endif
{
}

#if WITH_EDITOR
void UDMMaterialValueBool::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionStaticBoolParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionStaticBoolParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(),
		UE_DM_NodeComment_Default
	);

	check(NewExpression);
 
	NewExpression->DefaultValue = Value;
 
	InBuildState->AddValueExpressions(this, {NewExpression});
}
#endif
 
void UDMMaterialValueBool::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	// True dynamic branching is currently being worked on. When it is in, this will become relevant.
	// There is no Jira yet.
	checkNoEntry();
}

#if WITH_EDITOR
bool UDMMaterialValueBool::IsDefaultValue() const
{
	return Value == bDefaultValue;
}

void UDMMaterialValueBool::ApplyDefaultValue()
{
	SetValue(bDefaultValue);
}
 
void UDMMaterialValueBool::ResetDefaultValue()
{
	bDefaultValue = false;
}

UDMMaterialValueDynamic* UDMMaterialValueBool::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueBoolDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueBoolDynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueBool::GetComponentPathComponent() const
{
	return TEXT("Bool");
}

FText UDMMaterialValueBool::GetComponentDescription() const
{
	return LOCTEXT("Bool", "Bool");
}

TSharedPtr<FJsonValue> UDMMaterialValueBool::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueBool::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	bool bValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, bValueJson))
	{
		SetValue(bValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueBool::SetDefaultValue(bool bInDefaultValue)
{
	bDefaultValue = bInDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueBool::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueBool* OtherValue = CastChecked<UDMMaterialValueBool>(InOther);
	OtherValue->SetValue(GetValue());
}
 
void UDMMaterialValueBool::SetValue(bool InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value == InValue)
	{
		return;
	}

	Value = InValue;
 
	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
#undef LOCTEXT_NAMESPACE
