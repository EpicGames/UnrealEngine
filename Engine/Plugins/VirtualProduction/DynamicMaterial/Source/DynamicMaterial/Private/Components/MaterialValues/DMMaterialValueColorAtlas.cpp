// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValues/DMMaterialValueColorAtlas.h"
#include "DMDefs.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueColorAtlasDynamic.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionCurveAtlasRowParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueColorAtlas)

#define LOCTEXT_NAMESPACE "DMMaterialValueColorAtlas"

UDMMaterialValueColorAtlas::UDMMaterialValueColorAtlas()
	: UDMMaterialValue(EDMValueType::VT_ColorAtlas)
	, Value(0)
#if WITH_EDITORONLY_DATA
	, DefaultValue(0)
	, Atlas(nullptr)
	, Curve(nullptr)
#endif
{
#if WITH_EDITORONLY_DATA
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueColorAtlas, Atlas));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueColorAtlas, Curve));
#endif
}

void UDMMaterialValueColorAtlas::SetValue(float InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	InValue = FMath::Clamp(InValue, 0.f, 1.f);

	if (FMath::IsNearlyEqual(Value, InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}

void UDMMaterialValueColorAtlas::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);

	InMID->SetScalarParameterValue(GetMaterialParameterName(), Value);
}

#if WITH_EDITOR
void UDMMaterialValueColorAtlas::SetAtlas(UCurveLinearColorAtlas* InAtlas)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Atlas == InAtlas)
	{
		return;
	}

	Atlas = InAtlas;

	OnValueChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}

void UDMMaterialValueColorAtlas::SetCurve(UCurveLinearColor* InCurve)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Curve == InCurve)
	{
		return;
	}

	Curve = InCurve;

	OnValueChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}

void UDMMaterialValueColorAtlas::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}

	// UMaterialExpressionCurveAtlasRowParameter::Compile returns early with a compile error when Atlas or Curve are null.
	// Letting this compile error propagate through Material Designer build process causes an RHI crash at draw time,
	// so a vector4 constant is added as a placeholder until this object has valid parameters set.
	// See UE-365324.
	if (!Atlas || !Curve)
	{
		UMaterialExpressionConstant4Vector* const VectorParameter = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant4Vector>(UE_DM_NodeComment_Default, InBuildState->GetDynamicMaterial());
		InBuildState->AddValueExpressions(this, {VectorParameter});
		return;
	}

	UMaterialExpressionScalarParameter* AlphaParameter = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionScalarParameter>(
		GetMaterialParameterName(), 
		GetParameterGroup(), 
		UE_DM_NodeComment_Default
	);

	check(AlphaParameter);

	// This is a parameter, but we're treating it as a standard node.
	UMaterialExpressionCurveAtlasRowParameter* AtlasExpression = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionCurveAtlasRowParameter>(UE_DM_NodeComment_Default);
	check(AtlasExpression);

	AtlasExpression->Atlas = Atlas;
	AtlasExpression->Curve = Curve;
	AtlasExpression->DefaultValue = Value;
	AtlasExpression->InputTime.Connect(0, AlphaParameter);

	// Connect the RGB and A channels back together.
	UMaterialExpressionAppendVector* AppendExpression = InBuildState->GetBuildUtils().CreateExpressionAppend(AtlasExpression, 0, AtlasExpression, 4);

	InBuildState->AddValueExpressions(this, {AlphaParameter, AtlasExpression, AppendExpression});
}

void UDMMaterialValueColorAtlas::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}

void UDMMaterialValueColorAtlas::ResetDefaultValue()
{
	DefaultValue = 0.f;
}

UDMMaterialValueDynamic* UDMMaterialValueColorAtlas::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueColorAtlasDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueColorAtlasDynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueColorAtlas::GetComponentPathComponent() const
{
	return TEXT("ColorAtlasAlpha");
}

FText UDMMaterialValueColorAtlas::GetComponentDescription() const
{
	return LOCTEXT("ColorAtlas", "Color Atlas");
}

TSharedPtr<FJsonValue> UDMMaterialValueColorAtlas::JsonSerialize() const
{
	return FDMJsonUtils::Serialize({
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Value)), FDMJsonUtils::Serialize(Value)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Atlas)), FDMJsonUtils::Serialize(Atlas)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Curve)), FDMJsonUtils::Serialize(Curve)}
	});
}

bool UDMMaterialValueColorAtlas::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	if (!FDMJsonUtils::Deserialize(InJsonValue, Data))
	{
		return false;
	}

	bool bSuccess = false;
	EDMUpdateType UpdateType = EDMUpdateType::Value;

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Atlas))))
	{
		UCurveLinearColorAtlas* AtlasJson = nullptr;

		if (FDMJsonUtils::Deserialize(*JsonValue, AtlasJson))
		{
			const FDMUpdateGuard Guard;
			SetAtlas(AtlasJson);
			UpdateType = EDMUpdateType::Structure;
			bSuccess = true;
		}
	}

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Curve))))
	{
		UCurveLinearColor* CurveJson = nullptr;

		if (FDMJsonUtils::Deserialize(*JsonValue, CurveJson))
		{
			const FDMUpdateGuard Guard;
			SetCurve(CurveJson);
			UpdateType = EDMUpdateType::Structure;
			bSuccess = true;
		}
	}

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Value))))
	{
		float ValueJson;

		if (FDMJsonUtils::Deserialize(*JsonValue, ValueJson))
		{
			const FDMUpdateGuard Guard;
			SetValue(ValueJson);
			bSuccess = true;
		}
	}

	if (bSuccess)
	{
		OnValueChanged(UpdateType | EDMUpdateType::AllowParentUpdate);
	}

	return bSuccess;
}

void UDMMaterialValueColorAtlas::SetDefaultValue(float InDefaultValue)
{
	DefaultValue = InDefaultValue;
}

bool UDMMaterialValueColorAtlas::IsDefaultValue() const
{
	return FMath::IsNearlyEqual(Value, DefaultValue);
}
#endif

void UDMMaterialValueColorAtlas::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueColorAtlas* OtherValue = CastChecked<UDMMaterialValueColorAtlas>(InOther);
	OtherValue->SetValue(GetValue());
}

#undef LOCTEXT_NAMESPACE
