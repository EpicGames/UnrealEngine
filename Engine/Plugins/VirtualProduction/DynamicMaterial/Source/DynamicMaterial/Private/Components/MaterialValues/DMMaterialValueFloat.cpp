// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValues/DMMaterialValueFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat)

UDMMaterialValueFloat::UDMMaterialValueFloat()
	: UDMMaterialValueFloat(EDMValueType::VT_None)
{
}

UDMMaterialValueFloat::UDMMaterialValueFloat(EDMValueType InValueType)
	: UDMMaterialValue(InValueType)
	, ValueRange(FFloatInterval(0, 0))
{
}
