// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueFloat1.h"

#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat1Dynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat1)
 
#define LOCTEXT_NAMESPACE "DMMaterialValueFloat1"

UDMMaterialValueFloat1::UDMMaterialValueFloat1()
	: UDMMaterialValueFloat(EDMValueType::VT_Float1)
	, Value(0)
#if WITH_EDITORONLY_DATA
	, DefaultValue(0)
#endif
{
}
 
#if WITH_EDITOR
void UDMMaterialValueFloat1::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionScalarParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionScalarParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(), 
		UE_DM_NodeComment_Default
	);

	check(NewExpression);
 
	NewExpression->DefaultValue = Value;
 
	InBuildState->AddValueExpressions(this, {NewExpression});
}
 
void UDMMaterialValueFloat1::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}
 
void UDMMaterialValueFloat1::ResetDefaultValue()
{
	DefaultValue = 0.f;
}

UDMMaterialValueDynamic* UDMMaterialValueFloat1::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueFloat1Dynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueFloat1Dynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueFloat1::GetComponentPathComponent() const
{
	return TEXT("Scalar");
}

FText UDMMaterialValueFloat1::GetComponentDescription() const
{
	return LOCTEXT("Scalar", "Scalar");
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat1::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat1::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	float ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueFloat1::SetDefaultValue(float InDefaultValue)
{
	DefaultValue = InDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueFloat1::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat1* OtherValue = CastChecked<UDMMaterialValueFloat1>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat1::SetValue(float InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (HasValueRange())
	{
		InValue = FMath::Clamp(InValue, ValueRange.Min, ValueRange.Max);
	}

	if (FMath::IsNearlyEqual(Value, InValue))
	{
		return;
	}
 
	Value = InValue;
 
	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
void UDMMaterialValueFloat1::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	InMID->SetScalarParameterValue(GetMaterialParameterName(), Value);
}
 
#if WITH_EDITOR
bool UDMMaterialValueFloat1::IsDefaultValue() const
{
	return FMath::IsNearlyEqual(Value, DefaultValue);
}
#endif
 
#undef LOCTEXT_NAMESPACE
