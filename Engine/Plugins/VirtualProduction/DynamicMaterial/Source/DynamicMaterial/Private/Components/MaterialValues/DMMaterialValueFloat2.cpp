// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat2Dynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "PropertyHandle.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat2)

#define LOCTEXT_NAMESPACE "DMMaterialValueFloat2"

UDMMaterialValueFloat2::UDMMaterialValueFloat2()
	: UDMMaterialValueFloat(EDMValueType::VT_Float2)
	, Value(FVector2D::ZeroVector)
#if WITH_EDITORONLY_DATA
	, DefaultValue(FVector2D::ZeroVector)
#endif
{
}
 
#if WITH_EDITOR
void UDMMaterialValueFloat2::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionVectorParameter* ValueExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionVectorParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(),
		UE_DM_NodeComment_Default
	);

	check(ValueExpression);
 
	ValueExpression->DefaultValue = FLinearColor(Value.X, Value.Y, 0, 0);

	UMaterialExpressionComponentMask* MaskExpression = InBuildState->GetBuildUtils().CreateExpressionBitMask(ValueExpression, 0, FDMMaterialStageConnectorChannel::TWO_CHANNELS);
	check(MaskExpression);
 
	InBuildState->AddValueExpressions(this, {ValueExpression, MaskExpression});
}

bool UDMMaterialValueFloat2::IsDefaultValue() const
{
	return (FMath::IsNearlyEqual(Value.X, DefaultValue.X)
		&& FMath::IsNearlyEqual(Value.Y, DefaultValue.Y));
}
 
void UDMMaterialValueFloat2::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}

void UDMMaterialValueFloat2::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("X")))
	{
		FVector2D CurrentValue = GetValue();
		CurrentValue.X = GetDefaultValue().X;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Y")))
	{
		FVector2D CurrentValue = GetValue();
		CurrentValue.Y = GetDefaultValue().Y;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat2::ResetDefaultValue()
{
	DefaultValue = FVector2D::ZeroVector;
}

UDMMaterialValueDynamic* UDMMaterialValueFloat2::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueFloat2Dynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueFloat2Dynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueFloat2::GetComponentPathComponent() const
{
	return TEXT("Vector2D");
}

FText UDMMaterialValueFloat2::GetComponentDescription() const
{
	return LOCTEXT("Vector2", "Vector 2");
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat2::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat2::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FVector2D ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueFloat2::SetDefaultValue(const FVector2D& InDefaultValue)
{
	DefaultValue = InDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueFloat2::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat2* OtherValue = Cast<UDMMaterialValueFloat2>(InOther);
	OtherValue->SetValue(GetValue());
}
 
void UDMMaterialValueFloat2::SetValue(const FVector2D& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	FVector2D ValueClamped = InValue;

	if (HasValueRange())
	{
		ValueClamped.X = FMath::Clamp(ValueClamped.X, ValueRange.Min, ValueRange.Max);
		ValueClamped.Y = FMath::Clamp(ValueClamped.Y, ValueRange.Min, ValueRange.Max);
	}

	if (FMath::IsNearlyEqual(Value.X, ValueClamped.X)
		&& FMath::IsNearlyEqual(Value.Y, ValueClamped.Y))
	{
		return;
	}
 
	Value = ValueClamped;
 
	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
#if WITH_EDITOR
int32 UDMMaterialValueFloat2::GetInnateMaskOutput(int32 OutputChannels) const
{
	switch (OutputChannels)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			return 1;
 
		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			return 2;
 
		default:
			return UDMMaterialValue::GetInnateMaskOutput(OutputChannels);
	}
}
#endif
 
void UDMMaterialValueFloat2::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	InMID->SetVectorParameterValue(GetMaterialParameterName(), FLinearColor(Value.X, Value.Y, 0, 0));
}
 
#undef LOCTEXT_NAMESPACE
