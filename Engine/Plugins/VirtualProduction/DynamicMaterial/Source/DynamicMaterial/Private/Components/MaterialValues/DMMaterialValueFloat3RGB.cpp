// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat3RGBDynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "PropertyHandle.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat3RGB)
 
#define LOCTEXT_NAMESPACE "DMMaterialValueFloat3RGB"

UDMMaterialValueFloat3RGB::UDMMaterialValueFloat3RGB()
	: UDMMaterialValueFloat(EDMValueType::VT_Float3_RGB)
	, Value(FLinearColor(0.25, 0.25, 0.25, 1))
#if WITH_EDITORONLY_DATA
	, DefaultValue(FLinearColor(0.25, 0.25, 0.25, 1))
#endif
{
}
 
#if WITH_EDITOR
void UDMMaterialValueFloat3RGB::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionVectorParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionVectorParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(),
		UE_DM_NodeComment_Default
	);

	check(NewExpression);
 
	NewExpression->DefaultValue = FLinearColor(Value.R, Value.G, Value.B, 0);
 
	InBuildState->AddValueExpressions(this, {NewExpression});
}
 
bool UDMMaterialValueFloat3RGB::IsDefaultValue() const
{
	return (FMath::IsNearlyEqual(Value.R, DefaultValue.R)
		&& FMath::IsNearlyEqual(Value.G, DefaultValue.G)
		&& FMath::IsNearlyEqual(Value.B, DefaultValue.B)); 
}
 
void UDMMaterialValueFloat3RGB::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}

void UDMMaterialValueFloat3RGB::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("R")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.R = GetDefaultValue().R;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("G")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.G = GetDefaultValue().G;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("B")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.B = GetDefaultValue().B;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat3RGB::ResetDefaultValue()
{
	DefaultValue = FLinearColor(0, 0, 0, 1);
}

UDMMaterialValueDynamic* UDMMaterialValueFloat3RGB::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueFloat3RGBDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueFloat3RGBDynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueFloat3RGB::GetComponentPathComponent() const
{
	return TEXT("RGB");
}

FText UDMMaterialValueFloat3RGB::GetComponentDescription() const
{
	return LOCTEXT("ColorRGB", "Color (RGB)");
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat3RGB::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat3RGB::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FLinearColor ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueFloat3RGB::SetDefaultValue(const FLinearColor& InDefaultValue)
{
	DefaultValue = InDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueFloat3RGB::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat3RGB* OtherValue = CastChecked<UDMMaterialValueFloat3RGB>(InOther);
	OtherValue->SetValue(GetValue());
}
 
void UDMMaterialValueFloat3RGB::SetValue(const FLinearColor& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	FLinearColor ValueClamped = InValue;
	ValueClamped.A = 1.f;

	if (HasValueRange())
	{
		ValueClamped.R = FMath::Clamp(ValueClamped.R, ValueRange.Min, ValueRange.Max);
		ValueClamped.G = FMath::Clamp(ValueClamped.G, ValueRange.Min, ValueRange.Max);
		ValueClamped.B = FMath::Clamp(ValueClamped.B, ValueRange.Min, ValueRange.Max);
	}

	if (FMath::IsNearlyEqual(Value.R, ValueClamped.R)
		&& FMath::IsNearlyEqual(Value.G, ValueClamped.G)
		&& FMath::IsNearlyEqual(Value.B, ValueClamped.B))
	{
		return;
	}

	Value = ValueClamped;

	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
#if WITH_EDITOR
int32 UDMMaterialValueFloat3RGB::GetInnateMaskOutput(int32 OutputChannels) const
{
	switch (OutputChannels)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			return 1;
 
		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			return 2;
 
		case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
			return 3;
 
		default:
			return UDMMaterialValue::GetInnateMaskOutput(OutputChannels);
	}
}
#endif
 
void UDMMaterialValueFloat3RGB::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	InMID->SetVectorParameterValue(GetMaterialParameterName(), FLinearColor(Value.R, Value.G, Value.B, 0));
}
 
#undef LOCTEXT_NAMESPACE
