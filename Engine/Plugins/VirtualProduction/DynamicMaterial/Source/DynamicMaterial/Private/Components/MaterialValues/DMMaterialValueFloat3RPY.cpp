// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueFloat3RPY.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat3RPYDynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "PropertyHandle.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat3RPY)
 
#define LOCTEXT_NAMESPACE "DMMaterialValueFloat3RPY"

UDMMaterialValueFloat3RPY::UDMMaterialValueFloat3RPY()
	: UDMMaterialValueFloat(EDMValueType::VT_Float3_RPY)
	, Value(FRotator::ZeroRotator)
#if WITH_EDITORONLY_DATA
	, DefaultValue(FRotator::ZeroRotator)
#endif
{
}
 
#if WITH_EDITOR
void UDMMaterialValueFloat3RPY::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionVectorParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionVectorParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(),
		UE_DM_NodeComment_Default
	);

	check(NewExpression);
 
	NewExpression->DefaultValue = FLinearColor(Value.Roll, Value.Yaw, Value.Pitch, 0);
 
	InBuildState->AddValueExpressions(this, {NewExpression});
}

bool UDMMaterialValueFloat3RPY::IsDefaultValue() const
{
	return (FMath::IsNearlyEqual(Value.Roll, DefaultValue.Roll)
		&& FMath::IsNearlyEqual(Value.Pitch, DefaultValue.Pitch)
		&& FMath::IsNearlyEqual(Value.Yaw, DefaultValue.Yaw));
}
 
void UDMMaterialValueFloat3RPY::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}

void UDMMaterialValueFloat3RPY::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("Roll")))
	{
		FRotator CurrentValue = GetValue();
		CurrentValue.Roll = GetDefaultValue().Roll;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Pitch")))
	{
		FRotator CurrentValue = GetValue();
		CurrentValue.Pitch = GetDefaultValue().Pitch;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Yaw")))
	{
		FRotator CurrentValue = GetValue();
		CurrentValue.Yaw = GetDefaultValue().Yaw;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat3RPY::ResetDefaultValue()
{
	DefaultValue = FRotator::ZeroRotator;
}

UDMMaterialValueDynamic* UDMMaterialValueFloat3RPY::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueFloat3RPYDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueFloat3RPYDynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueFloat3RPY::GetComponentPathComponent() const
{
	return TEXT("Rotator");
}

FText UDMMaterialValueFloat3RPY::GetComponentDescription() const
{
	return LOCTEXT("Rotator", "Rotator");
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat3RPY::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat3RPY::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FRotator ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueFloat3RPY::SetDefaultValue(const FRotator& InDefaultValue)
{
	DefaultValue = InDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueFloat3RPY::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat3RPY* OtherValue = CastChecked<UDMMaterialValueFloat3RPY>(InOther);
	OtherValue->SetValue(GetValue());
}
 
void UDMMaterialValueFloat3RPY::SetValue(const FRotator& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	FRotator ValueClamped;
	ValueClamped.Roll = FRotator::NormalizeAxis(InValue.Roll);
	ValueClamped.Pitch = FRotator::NormalizeAxis(InValue.Pitch);
	ValueClamped.Yaw = FRotator::NormalizeAxis(InValue.Yaw);

	if (HasValueRange() && ValueRange.Min >= -180 && ValueRange.Max <= 180)
	{
		ValueClamped.Roll = FMath::Clamp(ValueClamped.Roll, ValueRange.Min, ValueRange.Max);
		ValueClamped.Pitch = FMath::Clamp(ValueClamped.Pitch, ValueRange.Min, ValueRange.Max);
		ValueClamped.Yaw = FMath::Clamp(ValueClamped.Yaw, ValueRange.Min, ValueRange.Max);
	}

	if (FMath::IsNearlyEqual(Value.Roll, ValueClamped.Roll)
		&& FMath::IsNearlyEqual(Value.Pitch, ValueClamped.Pitch)
		&& FMath::IsNearlyEqual(Value.Yaw, ValueClamped.Yaw))
	{
		return;
	}
 
	Value = ValueClamped;
 
	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
#if WITH_EDITOR
int32 UDMMaterialValueFloat3RPY::GetInnateMaskOutput(int32 OutputChannels) const
{
	switch (OutputChannels)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			return 1;
 
		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			return 2;
 
		case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
			return 3;
 
		default:
			return UDMMaterialValue::GetInnateMaskOutput(OutputChannels);
	}
}
#endif
 
void UDMMaterialValueFloat3RPY::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	InMID->SetVectorParameterValue(GetMaterialParameterName(), FLinearColor(Value.Roll, Value.Yaw, Value.Pitch, 0));
}
 
#undef LOCTEXT_NAMESPACE
