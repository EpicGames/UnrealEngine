// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat3XYZDynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "PropertyHandle.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat3XYZ)
 
#define LOCTEXT_NAMESPACE "DMMaterialValueFloat3XYZ"

UDMMaterialValueFloat3XYZ::UDMMaterialValueFloat3XYZ()
	: UDMMaterialValueFloat(EDMValueType::VT_Float3_XYZ)
	, Value(FVector::ZeroVector)
#if WITH_EDITORONLY_DATA
	, DefaultValue(FVector::ZeroVector)
#endif
{
}
 
#if WITH_EDITOR
void UDMMaterialValueFloat3XYZ::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionVectorParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionVectorParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(),
		UE_DM_NodeComment_Default
	);

	check(NewExpression);
 
	NewExpression->DefaultValue = FLinearColor(Value.X, Value.Y, Value.Z, 0);
 
	InBuildState->AddValueExpressions(this, {NewExpression});
}

bool UDMMaterialValueFloat3XYZ::IsDefaultValue() const
{
	return (FMath::IsNearlyEqual(Value.X, DefaultValue.X)
		&& FMath::IsNearlyEqual(Value.Y, DefaultValue.Y)
		&& FMath::IsNearlyEqual(Value.Z, DefaultValue.Z));
}
 
void UDMMaterialValueFloat3XYZ::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}

void UDMMaterialValueFloat3XYZ::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("X")))
	{
		FVector CurrentValue = GetValue();
		CurrentValue.X = GetDefaultValue().X;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Y")))
	{
		FVector CurrentValue = GetValue();
		CurrentValue.Y = GetDefaultValue().Y;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Z")))
	{
		FVector CurrentValue = GetValue();
		CurrentValue.Z = GetDefaultValue().Z;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat3XYZ::ResetDefaultValue()
{
	DefaultValue = FVector::ZeroVector;
}

UDMMaterialValueDynamic* UDMMaterialValueFloat3XYZ::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueFloat3XYZDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueFloat3XYZDynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueFloat3XYZ::GetComponentPathComponent() const
{
	return TEXT("Vector3D");
}

FText UDMMaterialValueFloat3XYZ::GetComponentDescription() const
{
	return LOCTEXT("Vector3", "Vector 3");
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat3XYZ::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat3XYZ::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FVector ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueFloat3XYZ::SetDefaultValue(const FVector& InDefaultValue)
{
	DefaultValue = InDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueFloat3XYZ::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat3XYZ* OtherValue = CastChecked<UDMMaterialValueFloat3XYZ>(InOther);
	OtherValue->SetValue(GetValue());
}
 
void UDMMaterialValueFloat3XYZ::SetValue(const FVector& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	FVector ValueClamped = InValue;

	if (HasValueRange())
	{
		ValueClamped.X = FMath::Clamp(ValueClamped.X, ValueRange.Min, ValueRange.Max);
		ValueClamped.Y = FMath::Clamp(ValueClamped.Y, ValueRange.Min, ValueRange.Max);
		ValueClamped.Z = FMath::Clamp(ValueClamped.Z, ValueRange.Min, ValueRange.Max);
	}

	if (FMath::IsNearlyEqual(Value.X, ValueClamped.X)
		&& FMath::IsNearlyEqual(Value.Y, ValueClamped.Y)
		&& FMath::IsNearlyEqual(Value.Z, ValueClamped.Z))
	{
		return;
	}
 
	Value = ValueClamped;
 
	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
#if WITH_EDITOR
int32 UDMMaterialValueFloat3XYZ::GetInnateMaskOutput(int32 OutputChannels) const
{
	switch (OutputChannels)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			return 1;
 
		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			return 2;
 
		case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
			return 3;
 
		default:
			return UDMMaterialValue::GetInnateMaskOutput(OutputChannels);
	}
}
#endif
 
void UDMMaterialValueFloat3XYZ::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	InMID->SetVectorParameterValue(GetMaterialParameterName(), FLinearColor(Value.X, Value.Y, Value.Z, 0));
}
 
#undef LOCTEXT_NAMESPACE
