// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueFloat4Dynamic.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "PropertyHandle.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat4)
 
#define LOCTEXT_NAMESPACE "DMMaterialValueFloat4"

UDMMaterialValueFloat4::UDMMaterialValueFloat4()
	: UDMMaterialValueFloat(EDMValueType::VT_Float4_RGBA)
	, Value(FLinearColor(0, 0, 0, 1))
#if WITH_EDITORONLY_DATA
	, DefaultValue(FLinearColor(0, 0, 0, 1))
#endif
{
}
 
#if WITH_EDITOR
void UDMMaterialValueFloat4::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}
 
	UMaterialExpressionVectorParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionVectorParameter>(
		GetMaterialParameterName(),
		GetParameterGroup(),
		UE_DM_NodeComment_Default
	);

	check(NewExpression);
 
	NewExpression->DefaultValue = Value;
 
	InBuildState->AddValueExpressions(this, {NewExpression});
}

bool UDMMaterialValueFloat4::IsDefaultValue() const
{
	return (FMath::IsNearlyEqual(Value.R, DefaultValue.R)
		&& FMath::IsNearlyEqual(Value.G, DefaultValue.G)
		&& FMath::IsNearlyEqual(Value.B, DefaultValue.B)
		&& FMath::IsNearlyEqual(Value.A, DefaultValue.A));
}
 
void UDMMaterialValueFloat4::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}
 
void UDMMaterialValueFloat4::ResetDefaultValue()
{
	DefaultValue = FLinearColor(0, 0, 0, 1);
}

void UDMMaterialValueFloat4::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("R")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.R = GetDefaultValue().R;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("G")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.G = GetDefaultValue().G;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("B")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.B = GetDefaultValue().B;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("A")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.A = GetDefaultValue().A;
		SetValue(CurrentValue);
	}
}

UDMMaterialValueDynamic* UDMMaterialValueFloat4::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueFloat4Dynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueFloat4Dynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueFloat4::GetComponentPathComponent() const
{
	return TEXT("RGBA");
}

FText UDMMaterialValueFloat4::GetComponentDescription() const
{
	return LOCTEXT("ColorRGBA", "Color (RGBA)");
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat4::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat4::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FLinearColor ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueFloat4::SetDefaultValue(const FLinearColor& InDefaultValue)
{
	DefaultValue = InDefaultValue;
}
#endif // WITH_EDITOR

void UDMMaterialValueFloat4::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat4* OtherValue = CastChecked<UDMMaterialValueFloat4>(InOther);
	OtherValue->SetValue(GetValue());
}
 
void UDMMaterialValueFloat4::SetValue(const FLinearColor& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	FLinearColor ValueClamped = InValue;

	if (HasValueRange())
	{
		ValueClamped.R = FMath::Clamp(ValueClamped.R, ValueRange.Min, ValueRange.Max);
		ValueClamped.G = FMath::Clamp(ValueClamped.G, ValueRange.Min, ValueRange.Max);
		ValueClamped.B = FMath::Clamp(ValueClamped.B, ValueRange.Min, ValueRange.Max);
		ValueClamped.A = FMath::Clamp(ValueClamped.A, ValueRange.Min, ValueRange.Max);
	}

	if (FMath::IsNearlyEqual(Value.R, ValueClamped.R)
		&& FMath::IsNearlyEqual(Value.G, ValueClamped.G)
		&& FMath::IsNearlyEqual(Value.B, ValueClamped.B)
		&& FMath::IsNearlyEqual(Value.A, ValueClamped.A))
	{
		return;
	}
 
	Value = ValueClamped;
 
	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}
 
#if WITH_EDITOR
int32 UDMMaterialValueFloat4::GetInnateMaskOutput(int32 OutputChannels) const
{
	switch (OutputChannels)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			return 1;
 
		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			return 2;
 
		case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
			return 3;
 
		case FDMMaterialStageConnectorChannel::FOURTH_CHANNEL:
			return 4;
 
		default:
			return UDMMaterialValue::GetInnateMaskOutput(OutputChannels);
	}
}
#endif
 
void UDMMaterialValueFloat4::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);
 
	InMID->SetVectorParameterValue(GetMaterialParameterName(), Value);
}
 
#undef LOCTEXT_NAMESPACE
