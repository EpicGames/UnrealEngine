// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValues/DMMaterialValueRenderTarget.h"
#include "Components/DMRenderTargetRenderer.h"
#include "DMComponentPath.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/CoreDelegates.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueRenderTargetDynamic.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueRenderTarget)

#define LOCTEXT_NAMESPACE "DMMaterialValueRenderTarget"

const FString UDMMaterialValueRenderTarget::RendererPathToken = "Renderer";

UDMMaterialValueRenderTarget::UDMMaterialValueRenderTarget()
	: TextureSize(FIntPoint(512, 512))
	, TextureFormat(ETextureRenderTargetFormat::RTF_RGBA16f)
	, ClearColor(FLinearColor::Black)
	, Renderer(nullptr)
{
#if WITH_EDITOR
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueRenderTarget, TextureSize));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueRenderTarget, TextureFormat));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueRenderTarget, ClearColor));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialValueRenderTarget, Renderer));
#endif
}

UDMMaterialValueRenderTarget::~UDMMaterialValueRenderTarget()
{
	FCoreDelegates::OnEndFrame.Remove(EndOfFrameDelegateHandle);
	EndOfFrameDelegateHandle.Reset();
}

UTextureRenderTarget2D* UDMMaterialValueRenderTarget::GetRenderTarget() const
{
	return Cast<UTextureRenderTarget2D>(GetValue());
}

const FIntPoint& UDMMaterialValueRenderTarget::GetTextureSize() const
{
	return TextureSize;
}

void UDMMaterialValueRenderTarget::SetTextureSize(const FIntPoint& InTextureSize)
{
	if (InTextureSize.X <= 0 || InTextureSize.Y <= 0 || InTextureSize == TextureSize)
	{
		return;
	}

	TextureSize = InTextureSize;

	AsyncCreateRenderTarget();
}

ETextureRenderTargetFormat UDMMaterialValueRenderTarget::GetTextureFormat() const
{
	return TextureFormat;
}

void UDMMaterialValueRenderTarget::SetTextureFormat(ETextureRenderTargetFormat InTextureFormat)
{
	if (InTextureFormat == TextureFormat)
	{
		return;
	}

	TextureFormat = InTextureFormat;

	AsyncCreateRenderTarget();
}

const FLinearColor& UDMMaterialValueRenderTarget::GetClearColor() const
{
	return ClearColor;
}

void UDMMaterialValueRenderTarget::SetClearColor(const FLinearColor& InClearColor)
{
	if (InClearColor == ClearColor)
	{
		return;
	}

	ClearColor = InClearColor;

	AsyncCreateRenderTarget();
}

UDMRenderTargetRenderer* UDMMaterialValueRenderTarget::GetRenderer() const
{
	return Renderer;
}

void UDMMaterialValueRenderTarget::SetRenderer(UDMRenderTargetRenderer* InRenderer)
{
	if (Renderer == InRenderer)
	{
		return;
	}

#if WITH_EDITOR
	if (Renderer)
	{
		Renderer->SetComponentState(EDMComponentLifetimeState::Removed);
	}
#endif

	Renderer = InRenderer;

#if WITH_EDITOR
	if (IsComponentAdded())
	{
		Renderer->SetComponentState(EDMComponentLifetimeState::Added);
	}
#endif
}

void UDMMaterialValueRenderTarget::EnsureRenderTarget(bool bInAsync)
{
	if (IsValid(GetRenderTarget()))
	{
		return;
	}

	if (bInAsync)
	{
		AsyncCreateRenderTarget();
	}
	else
	{
		CreateRenderTarget();
	}
}

void UDMMaterialValueRenderTarget::FlushCreateRenderTarget()
{
	if (EndOfFrameDelegateHandle.IsValid() || !IsValid(GetRenderTarget()))
	{
		CreateRenderTarget();
	}
}

void UDMMaterialValueRenderTarget::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	Super::Update(InSource, InUpdateType);

	if (!IsValid(GetRenderTarget()))
	{
		AsyncCreateRenderTarget();
	}
	else
	{
		UpdateRenderTarget();
	}
}

#if WITH_EDITOR
void UDMMaterialValueRenderTarget::CopyParametersFrom_Implementation(UObject* InOther)
{
	// Intentionally not calling Super.
	// The render target texture should not be copied as it's unique per instance.

	UDMMaterialValueRenderTarget* OtherRenderTarget = CastChecked<UDMMaterialValueRenderTarget>(InOther);
	OtherRenderTarget->SetTextureSize(TextureSize);
	OtherRenderTarget->SetTextureFormat(TextureFormat);
	OtherRenderTarget->SetClearColor(ClearColor);
}

UDMMaterialValueDynamic* UDMMaterialValueRenderTarget::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	return UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueRenderTargetDynamic>(InMaterialModelDynamic, this);
}

FString UDMMaterialValueRenderTarget::GetComponentPathComponent() const
{
	return TEXT("RenderTarget");
}

FText UDMMaterialValueRenderTarget::GetComponentDescription() const
{
	if (IsValid(Renderer))
	{
		return Renderer->GetComponentDescription();
	}

	return LOCTEXT("RenderTarget", "Render Target");
}

TSharedPtr<FJsonValue> UDMMaterialValueRenderTarget::JsonSerialize() const
{
	return FDMJsonUtils::Serialize({
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, TextureSize)), FDMJsonUtils::Serialize(TextureSize)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, TextureFormat)), FDMJsonUtils::Serialize<ETextureRenderTargetFormat>(TextureFormat)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, ClearColor)), FDMJsonUtils::Serialize(ClearColor)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Renderer)), FDMJsonUtils::Serialize(Renderer.Get())}
	});
}

bool UDMMaterialValueRenderTarget::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	if (!FDMJsonUtils::Deserialize(InJsonValue, Data))
	{
		return false;
	}

	bool bSuccess = false;

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, TextureSize))))
	{
		FIntPoint TextureSizeJson = FIntPoint::ZeroValue;

		if (FDMJsonUtils::Deserialize(*JsonValue, TextureSizeJson))
		{
			const FDMUpdateGuard Guard;
			SetTextureSize(TextureSizeJson);
			bSuccess = true;
		}
	}

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, TextureFormat))))
	{
		ETextureRenderTargetFormat TextureFormatJson = RTF_RGBA16f;

		if (FDMJsonUtils::Deserialize(*JsonValue, TextureFormatJson))
		{
			const FDMUpdateGuard Guard;
			SetTextureFormat(TextureFormatJson);
			bSuccess = true;
		}
	}

	if (const TSharedPtr<FJsonValue>* JsonClearColor = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, ClearColor))))
	{
		FLinearColor ClearColorJson = FLinearColor::Black;

		if (FDMJsonUtils::Deserialize(*JsonClearColor, ClearColorJson))
		{
			const FDMUpdateGuard Guard;
			SetClearColor(ClearColorJson);
			bSuccess = true;
		}
	}

	if (const TSharedPtr<FJsonValue>* JsonRenderer = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Renderer))))
	{
		UDMRenderTargetRenderer* RendererJson;

		if (FDMJsonUtils::Deserialize(*JsonRenderer, RendererJson, this))
		{
			const FDMUpdateGuard Guard;
			SetRenderer(RendererJson);
			bSuccess = true;
		}
	}

	if (bSuccess)
	{
		OnValueChanged(EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
	}

	return bSuccess;
}

void UDMMaterialValueRenderTarget::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (IsValid(Renderer))
	{
		Renderer->PostEditorDuplicate(InMaterialModel, this);
	}

	// Make sure we have a unique render target
	AsyncCreateRenderTarget();
}

void UDMMaterialValueRenderTarget::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName TextureSizeName = GET_MEMBER_NAME_CHECKED(UDMMaterialValueRenderTarget, TextureSize);
	static const FName ClearColorName = GET_MEMBER_NAME_CHECKED(UDMMaterialValueRenderTarget, ClearColor);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == TextureSizeName || MemberName == ClearColorName)
	{
		AsyncCreateRenderTarget();
	}
}
#endif

void UDMMaterialValueRenderTarget::PostLoad()
{
	Super::PostLoad();

	if (!IsValid(GetRenderTarget()))
	{
		CreateRenderTarget();
	}
}

void UDMMaterialValueRenderTarget::AsyncCreateRenderTarget()
{
	if (!EndOfFrameDelegateHandle.IsValid())
	{
		EndOfFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &UDMMaterialValueRenderTarget::CreateRenderTarget);
	}
}

void UDMMaterialValueRenderTarget::CreateRenderTarget()
{
	if (EndOfFrameDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(EndOfFrameDelegateHandle);
		EndOfFrameDelegateHandle.Reset();
	}

	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>(this, NAME_None,
		EObjectFlags::RF_Transactional | EObjectFlags::RF_DuplicateTransient | EObjectFlags::RF_TextExportTransient
	);

	check(RenderTarget);
	RenderTarget->RenderTargetFormat = TextureFormat;
	RenderTarget->ClearColor = ClearColor;
	RenderTarget->bAutoGenerateMips = false;
	RenderTarget->bCanCreateUAV = false;
	RenderTarget->InitAutoFormat(TextureSize.X, TextureSize.Y);
	RenderTarget->UpdateResourceImmediate(true);

	SetValue(RenderTarget);
}

void UDMMaterialValueRenderTarget::UpdateRenderTarget()
{
	if (Renderer)
	{
		if (UTextureRenderTarget2D* RenderTarget = GetRenderTarget())
		{
			Renderer->UpdateRenderTarget();
		}
		else
		{
			AsyncCreateRenderTarget();
		}
	}
}

UDMMaterialComponent* UDMMaterialValueRenderTarget::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == RendererPathToken)
	{
		return Renderer;
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

#if WITH_EDITOR
void UDMMaterialValueRenderTarget::OnComponentAdded()
{
	Super::OnComponentAdded();

	if (Renderer)
	{
		Renderer->SetComponentState(EDMComponentLifetimeState::Added);
	}

	AsyncCreateRenderTarget();
}

void UDMMaterialValueRenderTarget::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	if (Renderer)
	{
		Renderer->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}
#endif

#undef LOCTEXT_NAMESPACE
