// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValuesDynamic/DMMaterialValueTextureDynamic.h"
#include "DMDefs.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureCube.h"
#include "Engine/VolumeTexture.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Model/IDMMaterialBuildUtilsInterface.h"
#include "RenderUtils.h"
#include "Utils/DMUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueTexture)

#define LOCTEXT_NAMESPACE "DMMaterialValueTexture"

#if WITH_EDITOR
namespace UE::DynamicMaterial::Private
{
	bool HasAlpha(UTexture* InTexture)
	{
		if (!IsValid(InTexture))
		{
			return false;
		}

		if (InTexture->CompressionNoAlpha)
		{
			return false;
		}

		EPixelFormatChannelFlags ValidTextureChannels = EPixelFormatChannelFlags::None;

		if (UTexture2D* Texture2D = Cast<UTexture2D>(InTexture))
		{
			ValidTextureChannels = GetPixelFormatValidChannels(Texture2D->GetPixelFormat());
		}
		else if (UTextureCube* TextureCube = Cast<UTextureCube>(InTexture))
		{
			ValidTextureChannels = GetPixelFormatValidChannels(TextureCube->GetPixelFormat());
		}
		else if (UVolumeTexture* VolumeTexture = Cast<UVolumeTexture>(InTexture))
		{
			ValidTextureChannels = GetPixelFormatValidChannels(VolumeTexture->GetPixelFormat());
		}
		else
		{
			return false;
		}

		return EnumHasAnyFlags(ValidTextureChannels, EPixelFormatChannelFlags::A);
	}
}

FDMGetDefaultRGBTexture UDMMaterialValueTexture::GetDefaultRGBTexture;
#endif

UDMMaterialValueTexture::UDMMaterialValueTexture()
	: UDMMaterialValue(EDMValueType::VT_Texture)
	, Value(nullptr)
#if WITH_EDITORONLY_DATA
	, DefaultValue(nullptr)
	, OldValue(nullptr)
#endif
{
}

#if WITH_EDITOR
void UDMMaterialValueTexture::GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InBuildState->HasValue(this))
	{
		return;
	}

	UMaterialExpressionTextureObjectParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionTextureObjectParameter>(
		GetMaterialParameterName(), 
		GetParameterGroup(), 
		UE_DM_NodeComment_Default, 
		Value
	);

	check(NewExpression);

	InBuildState->AddValueExpressions(this, {NewExpression});
}
 
UDMMaterialValueTexture* UDMMaterialValueTexture::CreateMaterialValueTexture(UObject* InOuter, UTexture* InTexture)
{
	check(InTexture);
 
	UDMMaterialValueTexture* TextureValue = NewObject<UDMMaterialValueTexture>(InOuter, NAME_None, RF_Transactional);
	TextureValue->SetValue(InTexture);
	return TextureValue;
}
 
void UDMMaterialValueTexture::PreEditChange(FProperty* InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);
 
	if (!IsComponentValid())
	{
		return;
	}

	if (InPropertyAboutToChange->GetFName() == ValueName)
	{
		OldValue = GetValue();
	}
}

bool UDMMaterialValueTexture::IsDefaultValue() const
{
	return Value == DefaultValue;
}

void UDMMaterialValueTexture::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	// Skip parent class because we need to do extra logic.
	Super::Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (!IsComponentValid())
	{
		return;
	}

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName.IsNone())
	{
		return;
	}

	if (MemberPropertyName == ValueName)
	{
		OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
	}
}
#endif

void UDMMaterialValueTexture::PostLoad()
{
	Super::PostLoad();

	Type = EDMValueType::VT_Texture;
}

void UDMMaterialValueTexture::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueTexture* OtherValue = CastChecked<UDMMaterialValueTexture>(InOther);
	OtherValue->SetValue(GetValue());
}

#if WITH_EDITOR
void UDMMaterialValueTexture::ApplyDefaultValue()
{
	SetValue(DefaultValue);
}

void UDMMaterialValueTexture::ResetDefaultValue()
{
	DefaultValue = nullptr;

	if (GetDefaultRGBTexture.IsBound())
	{
		DefaultValue = GetDefaultRGBTexture.Execute();
	}
}

UDMMaterialValueDynamic* UDMMaterialValueTexture::ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDMMaterialValueTextureDynamic* ValueDynamic = UDMMaterialValueDynamic::CreateValueDynamic<UDMMaterialValueTextureDynamic>(InMaterialModelDynamic, this);
	ValueDynamic->SetValue(Value);

	return ValueDynamic;
}

FString UDMMaterialValueTexture::GetComponentPathComponent() const
{
	return TEXT("Texture");
}

FText UDMMaterialValueTexture::GetComponentDescription() const
{
	return LOCTEXT("Texture", "Texture");
}

TSharedPtr<FJsonValue> UDMMaterialValueTexture::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueTexture::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	UTexture* ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}

void UDMMaterialValueTexture::SetDefaultValue(UTexture* InDefaultValue)
{
	DefaultValue = InDefaultValue;
}

bool UDMMaterialValueTexture::HasAlpha() const
{
	return UE::DynamicMaterial::Private::HasAlpha(Value);
}
#endif

void UDMMaterialValueTexture::SetValue(UTexture* InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value == InValue)
	{
		return;
	}

	Value = InValue;

	OnValueChanged(EDMUpdateType::Value | EDMUpdateType::AllowParentUpdate);
}

void UDMMaterialValueTexture::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InMID);

	InMID->SetTextureParameterValue(GetMaterialParameterName(), Value);
}

#undef LOCTEXT_NAMESPACE
