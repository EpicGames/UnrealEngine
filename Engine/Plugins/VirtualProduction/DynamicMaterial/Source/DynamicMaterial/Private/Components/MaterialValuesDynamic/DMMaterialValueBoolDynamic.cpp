// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueBoolDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueBool.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueBoolDynamic)

UDMMaterialValueBoolDynamic::UDMMaterialValueBoolDynamic()
	: UDMMaterialValueDynamic()
	, Value(false)
{
}

#if WITH_EDITOR
bool UDMMaterialValueBoolDynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

bool UDMMaterialValueBoolDynamic::GetDefaultValue() const
{
	if (UDMMaterialValueBool* ParentValue = Cast<UDMMaterialValueBool>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueBool>()->GetDefaultValue();
}

void UDMMaterialValueBoolDynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueBoolDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueBool* DestinationValueBool = Cast<UDMMaterialValueBool>(InDestinationComponent))
	{
		DestinationValueBool->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueBoolDynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueBoolDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	bool ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueBoolDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueBoolDynamic* OtherValue = CastChecked<UDMMaterialValueBoolDynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueBoolDynamic::SetValue(bool InValue)
{
	if (Value == InValue)
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueBoolDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	// True dynamic branching is currently being worked on. When it is in, this will become relevant.
	// There is no Jira yet.
	checkNoEntry();
}
