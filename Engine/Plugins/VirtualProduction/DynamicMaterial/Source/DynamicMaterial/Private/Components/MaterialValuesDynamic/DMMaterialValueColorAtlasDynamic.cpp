// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueColorAtlasDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueColorAtlas.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueColorAtlasDynamic)

UDMMaterialValueColorAtlasDynamic::UDMMaterialValueColorAtlasDynamic()
	: UDMMaterialValueDynamic()
	, Value(0.f)
{
}

#if WITH_EDITOR
bool UDMMaterialValueColorAtlasDynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

float UDMMaterialValueColorAtlasDynamic::GetDefaultValue() const
{
	if (UDMMaterialValueColorAtlas* ParentValue = Cast<UDMMaterialValueColorAtlas>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueColorAtlas>()->GetDefaultValue();
}

void UDMMaterialValueColorAtlasDynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueColorAtlasDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueColorAtlas* DestinationValueColorAtlas = Cast<UDMMaterialValueColorAtlas>(InDestinationComponent))
	{
		DestinationValueColorAtlas->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueColorAtlasDynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueColorAtlasDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	float ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueColorAtlasDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueColorAtlasDynamic* OtherValue = CastChecked<UDMMaterialValueColorAtlasDynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueColorAtlasDynamic::SetValue(float InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (FMath::IsNearlyEqual(Value, InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueColorAtlasDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetScalarParameterValue(ParentValue->GetMaterialParameterName(), Value);
}
