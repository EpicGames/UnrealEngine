// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueFloat1Dynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat1Dynamic)

UDMMaterialValueFloat1Dynamic::UDMMaterialValueFloat1Dynamic()
	: UDMMaterialValueDynamic()
	, Value(0.f)
{
}

#if WITH_EDITOR
bool UDMMaterialValueFloat1Dynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

float UDMMaterialValueFloat1Dynamic::GetDefaultValue() const
{
	if (UDMMaterialValueFloat1* ParentValue = Cast<UDMMaterialValueFloat1>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueFloat1>()->GetDefaultValue();
}

void UDMMaterialValueFloat1Dynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueFloat1Dynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueFloat1* DestinationValueFloat1 = Cast<UDMMaterialValueFloat1>(InDestinationComponent))
	{
		DestinationValueFloat1->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat1Dynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat1Dynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	float ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueFloat1Dynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat1Dynamic* OtherValue = CastChecked<UDMMaterialValueFloat1Dynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat1Dynamic::SetValue(float InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (FMath::IsNearlyEqual(Value, InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueFloat1Dynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetScalarParameterValue(ParentValue->GetMaterialParameterName(), Value);
}
