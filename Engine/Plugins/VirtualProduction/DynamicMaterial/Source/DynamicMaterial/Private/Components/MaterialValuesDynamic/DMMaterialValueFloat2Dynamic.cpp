// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueFloat2Dynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat2Dynamic)

UDMMaterialValueFloat2Dynamic::UDMMaterialValueFloat2Dynamic()
	: UDMMaterialValueDynamic()
	, Value(FVector2D::ZeroVector)
{
}

#if WITH_EDITOR
bool UDMMaterialValueFloat2Dynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

const FVector2D& UDMMaterialValueFloat2Dynamic::GetDefaultValue() const
{
	if (UDMMaterialValueFloat2* ParentValue = Cast<UDMMaterialValueFloat2>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueFloat2>()->GetDefaultValue();
}

void UDMMaterialValueFloat2Dynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueFloat2Dynamic::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("X")))
	{
		FVector2D CurrentValue = GetValue();
		CurrentValue.X = GetDefaultValue().X;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Y")))
	{
		FVector2D CurrentValue = GetValue();
		CurrentValue.Y = GetDefaultValue().Y;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat2Dynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueFloat2* DestinationValueFloat2 = Cast<UDMMaterialValueFloat2>(InDestinationComponent))
	{
		DestinationValueFloat2->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat2Dynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat2Dynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FVector2D ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueFloat2Dynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat2Dynamic* OtherValue = CastChecked<UDMMaterialValueFloat2Dynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat2Dynamic::SetValue(const FVector2D& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value.Equals(InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueFloat2Dynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetVectorParameterValue(
		ParentValue->GetMaterialParameterName(),
		FLinearColor(Value.X, Value.Y, 0.f, 0.f)
	);
}
