// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueFloat3RGBDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat3RGBDynamic)

UDMMaterialValueFloat3RGBDynamic::UDMMaterialValueFloat3RGBDynamic()
	: UDMMaterialValueDynamic()
	, Value(FLinearColor::Black)
{
}

#if WITH_EDITOR
bool UDMMaterialValueFloat3RGBDynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

const FLinearColor& UDMMaterialValueFloat3RGBDynamic::GetDefaultValue() const
{
	if (UDMMaterialValueFloat3RGB* ParentValue = Cast<UDMMaterialValueFloat3RGB>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueFloat3RGB>()->GetDefaultValue();
}

void UDMMaterialValueFloat3RGBDynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueFloat3RGBDynamic::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("R")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.R = GetDefaultValue().R;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("G")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.G = GetDefaultValue().G;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("B")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.B = GetDefaultValue().B;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat3RGBDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueFloat3RGB* DestinationValueFloat3RGB = Cast<UDMMaterialValueFloat3RGB>(InDestinationComponent))
	{
		DestinationValueFloat3RGB->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat3RGBDynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat3RGBDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FLinearColor ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueFloat3RGBDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat3RGBDynamic* OtherValue = CastChecked<UDMMaterialValueFloat3RGBDynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat3RGBDynamic::SetValue(const FLinearColor& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value.Equals(InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueFloat3RGBDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetVectorParameterValue(ParentValue->GetMaterialParameterName(), Value);
}
