// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueFloat3RPYDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RPY.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat3RPYDynamic)

UDMMaterialValueFloat3RPYDynamic::UDMMaterialValueFloat3RPYDynamic()
	: UDMMaterialValueDynamic()
	, Value(FRotator::ZeroRotator)
{
}

#if WITH_EDITOR
bool UDMMaterialValueFloat3RPYDynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

const FRotator& UDMMaterialValueFloat3RPYDynamic::GetDefaultValue() const
{
	if (UDMMaterialValueFloat3RPY* ParentValue = Cast<UDMMaterialValueFloat3RPY>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueFloat3RPY>()->GetDefaultValue();
}

void UDMMaterialValueFloat3RPYDynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueFloat3RPYDynamic::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("Roll")))
	{
		FRotator CurrentValue = GetValue();
		CurrentValue.Roll = GetDefaultValue().Roll;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Pitch")))
	{
		FRotator CurrentValue = GetValue();
		CurrentValue.Pitch = GetDefaultValue().Pitch;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Yaw")))
	{
		FRotator CurrentValue = GetValue();
		CurrentValue.Yaw = GetDefaultValue().Yaw;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat3RPYDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueFloat3RPY* DestinationValueFloat3RPY = Cast<UDMMaterialValueFloat3RPY>(InDestinationComponent))
	{
		DestinationValueFloat3RPY->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat3RPYDynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat3RPYDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FRotator ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueFloat3RPYDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat3RPYDynamic* OtherValue = CastChecked<UDMMaterialValueFloat3RPYDynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat3RPYDynamic::SetValue(const FRotator& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value.Equals(InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueFloat3RPYDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetVectorParameterValue(
		ParentValue->GetMaterialParameterName(),
		FLinearColor(Value.Roll, Value.Pitch, Value.Yaw, 0.f)
	);
}
