// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueFloat3XYZDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat3XYZDynamic)

UDMMaterialValueFloat3XYZDynamic::UDMMaterialValueFloat3XYZDynamic()
	: UDMMaterialValueDynamic()
	, Value(FVector::ZeroVector)
{
}

#if WITH_EDITOR
bool UDMMaterialValueFloat3XYZDynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

const FVector& UDMMaterialValueFloat3XYZDynamic::GetDefaultValue() const
{
	if (UDMMaterialValueFloat3XYZ* ParentValue = Cast<UDMMaterialValueFloat3XYZ>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueFloat3XYZ>()->GetDefaultValue();
}

void UDMMaterialValueFloat3XYZDynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueFloat3XYZDynamic::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("X")))
	{
		FVector CurrentValue = GetValue();
		CurrentValue.X = GetDefaultValue().X;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Y")))
	{
		FVector CurrentValue = GetValue();
		CurrentValue.Y = GetDefaultValue().Y;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("Z")))
	{
		FVector CurrentValue = GetValue();
		CurrentValue.Z = GetDefaultValue().Z;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat3XYZDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueFloat3XYZ* DestinationValueFloat3XYZ = Cast<UDMMaterialValueFloat3XYZ>(InDestinationComponent))
	{
		DestinationValueFloat3XYZ->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat3XYZDynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat3XYZDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FVector ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueFloat3XYZDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat3XYZDynamic* OtherValue = CastChecked<UDMMaterialValueFloat3XYZDynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat3XYZDynamic::SetValue(const FVector& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value.Equals(InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueFloat3XYZDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetVectorParameterValue(
		ParentValue->GetMaterialParameterName(),
		FLinearColor(Value.X, Value.Y, Value.Z, 0.f)
	);
}
