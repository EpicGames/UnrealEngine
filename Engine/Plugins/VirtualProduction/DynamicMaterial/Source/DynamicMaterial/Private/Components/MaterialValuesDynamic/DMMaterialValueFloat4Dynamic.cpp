// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueFloat4Dynamic.h"

#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueFloat4Dynamic)

UDMMaterialValueFloat4Dynamic::UDMMaterialValueFloat4Dynamic()
	: UDMMaterialValueDynamic()
	, Value(FLinearColor::Black)
{
}

#if WITH_EDITOR
bool UDMMaterialValueFloat4Dynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

const FLinearColor& UDMMaterialValueFloat4Dynamic::GetDefaultValue() const
{
	if (UDMMaterialValueFloat4* ParentValue = Cast<UDMMaterialValueFloat4>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueFloat4>()->GetDefaultValue();
}

void UDMMaterialValueFloat4Dynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueFloat4Dynamic::ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const FStringView PropertyPath = InPropertyHandle->GetPropertyPath();
	const int32 IndexOfStructSeparator = PropertyPath.Find(TEXT("->"));

	if (IndexOfStructSeparator == INDEX_NONE)
	{
		Super::ResetToDefault(InPropertyHandle);
		return;
	}

	FStringView LeafName = PropertyPath.RightChop(IndexOfStructSeparator + 2);

	if (LeafName.Equals(TEXT("R")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.R = GetDefaultValue().R;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("G")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.G = GetDefaultValue().G;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("B")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.B = GetDefaultValue().B;
		SetValue(CurrentValue);
	}
	else if (LeafName.Equals(TEXT("A")))
	{
		FLinearColor CurrentValue = GetValue();
		CurrentValue.A = GetDefaultValue().A;
		SetValue(CurrentValue);
	}
}

void UDMMaterialValueFloat4Dynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueFloat4* DestinationValueFloat4 = Cast<UDMMaterialValueFloat4>(InDestinationComponent))
	{
		DestinationValueFloat4->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueFloat4Dynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueFloat4Dynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	FLinearColor ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueFloat4Dynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueFloat4Dynamic* OtherValue = CastChecked<UDMMaterialValueFloat4Dynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueFloat4Dynamic::SetValue(const FLinearColor& InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value.Equals(InValue))
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueFloat4Dynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetVectorParameterValue(ParentValue->GetMaterialParameterName(), Value);
}
