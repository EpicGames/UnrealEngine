// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialValuesDynamic/DMMaterialValueRenderTargetDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueRenderTargetDynamic)

UDMMaterialValueRenderTargetDynamic::UDMMaterialValueRenderTargetDynamic()
{
}

#if WITH_EDITOR
TSharedPtr<FJsonValue> UDMMaterialValueRenderTargetDynamic::JsonSerialize() const
{
	return nullptr;
}

bool UDMMaterialValueRenderTargetDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	return false;
}

bool UDMMaterialValueRenderTargetDynamic::IsDefaultValue() const
{
	return true;
}

void UDMMaterialValueRenderTargetDynamic::ApplyDefaultValue()
{
}

void UDMMaterialValueRenderTargetDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{	
}
#endif

void UDMMaterialValueRenderTargetDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	// Overridden to do nothing
	// The render target texture should not be copied as it's unique per instance.
}

void UDMMaterialValueRenderTargetDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
}
