// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialValuesDynamic/DMMaterialValueTextureDynamic.h"

#include "Components/DMMaterialValue.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Engine/Texture.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialValueTextureDynamic)

UDMMaterialValueTextureDynamic::UDMMaterialValueTextureDynamic()
	: UDMMaterialValueDynamic()
	, Value(nullptr)
{
}

#if WITH_EDITOR
bool UDMMaterialValueTextureDynamic::IsDefaultValue() const
{
	return Value == GetDefaultValue();
}

UTexture* UDMMaterialValueTextureDynamic::GetDefaultValue() const
{
	if (UDMMaterialValueTexture* ParentValue = Cast<UDMMaterialValueTexture>(GetParentValue()))
	{
		return ParentValue->GetValue();
	}

	return GetDefault<UDMMaterialValueTexture>()->GetDefaultValue();
}

void UDMMaterialValueTextureDynamic::ApplyDefaultValue()
{
	SetValue(GetDefaultValue());
}

void UDMMaterialValueTextureDynamic::CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const
{
	if (UDMMaterialValueTexture* DestinationValueTexture = Cast<UDMMaterialValueTexture>(InDestinationComponent))
	{
		DestinationValueTexture->SetValue(GetValue());
	}
}

TSharedPtr<FJsonValue> UDMMaterialValueTextureDynamic::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(Value);
}

bool UDMMaterialValueTextureDynamic::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	UTexture* ValueJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, ValueJson))
	{
		SetValue(ValueJson);
		return true;
	}

	return false;
}
#endif

void UDMMaterialValueTextureDynamic::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialValueTextureDynamic* OtherValue = CastChecked<UDMMaterialValueTextureDynamic>(InOther);
	OtherValue->SetValue(GetValue());
}

void UDMMaterialValueTextureDynamic::SetValue(UTexture* InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value == InValue)
	{
		return;
	}

	Value = InValue;

	OnValueChanged();
}

void UDMMaterialValueTextureDynamic::SetMIDParameter(UMaterialInstanceDynamic* InMID) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialValue* ParentValue = GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	check(InMID);

	InMID->SetTextureParameterValue(ParentValue->GetMaterialParameterName(), Value);
}
