// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RenderTargetRenderers/DMRenderTargetTextRenderer.h"
#include "Components/MaterialValues/DMMaterialValueRenderTarget.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "TextureResource.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
#include "Dom/JsonValue.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMRenderTargetTextRenderer)

#define LOCTEXT_NAMESPACE "DMRenderTargetTextRenderer"

namespace UE::DynamicMaterial::Private
{
	const FIntPoint MinimumTextTextureSize = FIntPoint(5, 10);
	const FIntPoint MaxTexScale = FIntPoint(64, 64);

	UFont* GetDefaultFont()
	{
		static TSoftObjectPtr<UFont> DefaultFont = TSoftObjectPtr<UFont>(FSoftObjectPath(TEXT("/Script/Engine.Font'/Engine/EngineFonts/Roboto.Roboto'")));
		return DefaultFont.LoadSynchronous();
	}
}

#if WITH_EDITOR
struct FDMRenderTargetTextRenderer
{
	static const inline FName FontInfoName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, FontInfo);
	static const inline FName TextName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, Text);
	static const inline FName TextColorName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, TextColor);
	static const inline FName HasHighlightName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, bHasHighlight);
	static const inline FName HighlightColorName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, HighlightColor);
	static const inline FName HasShadowName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, bHasShadow);
	static const inline FName ShadowColorName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, ShadowColor);
	static const inline FName ShadowOffsetName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, ShadowOffset);
	static const inline FName AutoWrapTextName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, bAutoWrapText);
	static const inline FName WrapTextAtName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, WrapTextAt);
	static const inline FName WrappingPolicyName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, WrappingPolicy);
	static const inline FName JustifyName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, Justify);
	static const inline FName TransformPolicyName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, TransformPolicy);
	static const inline FName FlowDirectionName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, FlowDirection);
	static const inline FName ShapingMethodName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, ShapingMethod);
	static const inline FName StrikeBrushName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, StrikeBrush);
	static const inline FName LineHeightName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, LineHeight);
	static const inline FName PaddingLeftName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, PaddingLeft);
	static const inline FName PaddingRightName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, PaddingRight);
	static const inline FName PaddingTopName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, PaddingTop);
	static const inline FName PaddingBottomName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, PaddingBottom);
	static const inline FName OverrideRenderTargetSizeName = GET_MEMBER_NAME_CHECKED(UDMRenderTargetTextRenderer, bOverrideRenderTargetSize);

	static const inline TSet<FName> PropertyNames = {
		FontInfoName, TextName, TextColorName,
		JustifyName,
		FlowDirectionName,
		LineHeightName,
		AutoWrapTextName, WrapTextAtName, WrappingPolicyName,
		PaddingLeftName, PaddingTopName, PaddingRightName, PaddingBottomName,
		HasHighlightName, HighlightColorName,
		HasShadowName, ShadowColorName, ShadowOffsetName,
		TransformPolicyName,
		ShapingMethodName,
		StrikeBrushName,
		OverrideRenderTargetSizeName
	};
};
#endif

UDMRenderTargetTextRenderer::UDMRenderTargetTextRenderer()
{
	FontInfo.FontObject = UE::DynamicMaterial::Private::GetDefaultFont();
	Text = LOCTEXT("Text", "Text");

#if WITH_EDITOR
	EditableProperties.Append(FDMRenderTargetTextRenderer::PropertyNames.Array());
#endif
}

#if WITH_EDITOR
void UDMRenderTargetTextRenderer::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMRenderTargetTextRenderer* OtherTextRenderer = CastChecked<UDMRenderTargetTextRenderer>(InOther);
	OtherTextRenderer->SetFontInfo(FontInfo);
	OtherTextRenderer->SetText(Text);
	OtherTextRenderer->SetTextColor(TextColor);
	OtherTextRenderer->SetBackgroundColor(GetBackgroundColor());
	OtherTextRenderer->SetHasHighlight(bHasHighlight);
	OtherTextRenderer->SetHighlightColor(HighlightColor);
	OtherTextRenderer->SetHasShadow(bHasShadow);
	OtherTextRenderer->SetShadowColor(ShadowColor);
	OtherTextRenderer->SetShadowOffset(ShadowOffset);
	OtherTextRenderer->SetAutoWrapText(bAutoWrapText);
	OtherTextRenderer->SetWrapTextAt(WrapTextAt);
	OtherTextRenderer->SetWrappingPolicy(WrappingPolicy);
	OtherTextRenderer->SetJustify(Justify);
	OtherTextRenderer->SetTransformPolicy(TransformPolicy);
	OtherTextRenderer->SetFlowDirection(FlowDirection);
	OtherTextRenderer->SetShapingMethod(ShapingMethod);
	OtherTextRenderer->SetStrikeBrush(StrikeBrush);
	OtherTextRenderer->SetLineHeight(LineHeight);
	OtherTextRenderer->SetPaddingLeft(PaddingLeft);
	OtherTextRenderer->SetPaddingRight(PaddingRight);
	OtherTextRenderer->SetPaddingTop(PaddingTop);
	OtherTextRenderer->SetPaddingBottom(PaddingBottom);
	OtherTextRenderer->SetOverrideRenderTargetSize(bOverrideRenderTargetSize);
	OtherTextRenderer->SetStrikeBrush(StrikeBrush);
}

TSharedPtr<FJsonValue> UDMRenderTargetTextRenderer::JsonSerialize() const
{
	TSharedRef<FJsonObject> Object = MakeShared<FJsonObject>();

	Object->SetField(FDMRenderTargetTextRenderer::FontInfoName.GetPlainNameString(), FDMJsonUtils::Serialize<FSlateFontInfo>(FontInfo));
	Object->SetField(FDMRenderTargetTextRenderer::TextName.GetPlainNameString(), FDMJsonUtils::Serialize(Text));
	Object->SetField(FDMRenderTargetTextRenderer::TextColorName.GetPlainNameString(), FDMJsonUtils::Serialize(TextColor));
	Object->SetField(FDMRenderTargetTextRenderer::HasHighlightName.GetPlainNameString(), FDMJsonUtils::Serialize(bHasHighlight));
	Object->SetField(FDMRenderTargetTextRenderer::HighlightColorName.GetPlainNameString(), FDMJsonUtils::Serialize(HighlightColor));
	Object->SetField(FDMRenderTargetTextRenderer::HasShadowName.GetPlainNameString(), FDMJsonUtils::Serialize(bHasShadow));
	Object->SetField(FDMRenderTargetTextRenderer::ShadowColorName.GetPlainNameString(), FDMJsonUtils::Serialize(ShadowColor));
	Object->SetField(FDMRenderTargetTextRenderer::ShadowOffsetName.GetPlainNameString(), FDMJsonUtils::Serialize(ShadowOffset));
	Object->SetField(FDMRenderTargetTextRenderer::AutoWrapTextName.GetPlainNameString(), FDMJsonUtils::Serialize(bAutoWrapText));
	Object->SetField(FDMRenderTargetTextRenderer::WrapTextAtName.GetPlainNameString(), FDMJsonUtils::Serialize(WrapTextAt));
	Object->SetField(FDMRenderTargetTextRenderer::WrappingPolicyName.GetPlainNameString(), FDMJsonUtils::Serialize(WrappingPolicy));
	Object->SetField(FDMRenderTargetTextRenderer::JustifyName.GetPlainNameString(), FDMJsonUtils::Serialize(Justify.GetValue()));
	Object->SetField(FDMRenderTargetTextRenderer::TransformPolicyName.GetPlainNameString(), FDMJsonUtils::Serialize(TransformPolicy));
	Object->SetField(FDMRenderTargetTextRenderer::FlowDirectionName.GetPlainNameString(), FDMJsonUtils::Serialize(FlowDirection));
	Object->SetField(FDMRenderTargetTextRenderer::ShapingMethodName.GetPlainNameString(), FDMJsonUtils::Serialize(ShapingMethod));
	Object->SetField(FDMRenderTargetTextRenderer::LineHeightName.GetPlainNameString(), FDMJsonUtils::Serialize(LineHeight));
	Object->SetField(FDMRenderTargetTextRenderer::PaddingLeftName.GetPlainNameString(), FDMJsonUtils::Serialize(PaddingLeft));
	Object->SetField(FDMRenderTargetTextRenderer::PaddingRightName.GetPlainNameString(), FDMJsonUtils::Serialize(PaddingRight));
	Object->SetField(FDMRenderTargetTextRenderer::PaddingTopName.GetPlainNameString(), FDMJsonUtils::Serialize(PaddingTop));
	Object->SetField(FDMRenderTargetTextRenderer::PaddingBottomName.GetPlainNameString(), FDMJsonUtils::Serialize(PaddingBottom));
	Object->SetField(FDMRenderTargetTextRenderer::OverrideRenderTargetSizeName.GetPlainNameString(), FDMJsonUtils::Serialize(bOverrideRenderTargetSize));

	if (StrikeBrush.IsValid())
	{
		Object->SetField(FDMRenderTargetTextRenderer::StrikeBrushName.GetPlainNameString(), FDMJsonUtils::Serialize<FSlateBrush>(StrikeBrush.Get<FSlateBrush>()));
	}
	else
	{
		Object->SetField(FDMRenderTargetTextRenderer::StrikeBrushName.GetPlainNameString(), MakeShared<FJsonValueNull>());
	}

	return MakeShared<FJsonValueObject>(Object);
}

bool UDMRenderTargetTextRenderer::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	TSharedPtr<FJsonObject> Object = InJsonValue->AsObject();

	if (!Object.IsValid())
	{
		return false;
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::FontInfoName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize<FSlateFontInfo>(JsonValue, FontInfo);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::TextName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, Text);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::TextColorName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, TextColor);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::HasHighlightName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, bHasHighlight);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::HighlightColorName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, HighlightColor);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::HasShadowName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, bHasShadow);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::ShadowColorName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, ShadowColor);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::ShadowOffsetName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, ShadowOffset);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::AutoWrapTextName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, bAutoWrapText);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::WrapTextAtName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, WrapTextAt);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::WrappingPolicyName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, WrappingPolicy);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::JustifyName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		ETextJustify::Type EnumValue = ETextJustify::Left;
		FDMJsonUtils::Deserialize(JsonValue, EnumValue);
		Justify = EnumValue;
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::TransformPolicyName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, TransformPolicy);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::FlowDirectionName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, FlowDirection);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::ShapingMethodName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, ShapingMethod);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::LineHeightName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, LineHeight);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::PaddingLeftName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, PaddingLeft);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::PaddingRightName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, PaddingRight);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::PaddingTopName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, PaddingTop);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::PaddingBottomName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, PaddingBottom);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::OverrideRenderTargetSizeName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		FDMJsonUtils::Deserialize(JsonValue, bOverrideRenderTargetSize);
	}

	if (const TSharedPtr<FJsonValue> JsonValue = Object->GetFieldUntyped(FDMRenderTargetTextRenderer::StrikeBrushName.GetPlainNameSharedString()); JsonValue.IsValid())
	{
		if (!(JsonValue)->IsNull())
		{
			FDMJsonUtils::Deserialize<FSlateBrush>(JsonValue, *StrikeBrush.GetMutablePtr<FSlateBrush>());
		}
	}

	return true;
}

FText UDMRenderTargetTextRenderer::GetComponentDescription() const
{
	return LOCTEXT("Text", "Text");
}

void UDMRenderTargetTextRenderer::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == FDMRenderTargetTextRenderer::HasHighlightName
		|| PropertyName == FDMRenderTargetTextRenderer::HasShadowName
		|| PropertyName == FDMRenderTargetTextRenderer::AutoWrapTextName)
	{
		// Cause details panel refresh.
		Update(this, EDMUpdateType::Value | EDMUpdateType::RefreshDetailView);
	}
	else
	{
		Update(this, EDMUpdateType::Value);
	}

	if (FDMRenderTargetTextRenderer::PropertyNames.Contains(PropertyName))
	{
		Widget.Reset();
		bRecalculateTextSize = true;
		AsyncUpdateRenderTarget();
	}
}
#endif

const FSlateFontInfo& UDMRenderTargetTextRenderer::GetFontInfo() const
{
	return FontInfo;
}

void UDMRenderTargetTextRenderer::SetFontInfo(const FSlateFontInfo& InFontInfo)
{
	if (FontInfo == InFontInfo)
	{
		return;
	}

	FontInfo = InFontInfo;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

const FText& UDMRenderTargetTextRenderer::GetText() const
{
	return Text;
}

void UDMRenderTargetTextRenderer::SetText(const FText& InText)
{
	if (Text.EqualTo(InText, ETextComparisonLevel::Default))
	{
		return;
	}

	Text = InText;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();

}

const FLinearColor& UDMRenderTargetTextRenderer::GetTextColor() const
{
	return TextColor;
}

void UDMRenderTargetTextRenderer::SetTextColor(const FLinearColor& InColor)
{
	if (TextColor == InColor)
	{
		return;
	}

	TextColor = InColor;

	AsyncUpdateRenderTarget();
}

const FLinearColor& UDMRenderTargetTextRenderer::GetBackgroundColor() const
{
	if (UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue())
	{
		return RenderTargetValue->GetClearColor();
	}

	return FLinearColor::Black;
}

void UDMRenderTargetTextRenderer::SetBackgroundColor(const FLinearColor& InBackgroundColor)
{
	if (UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue())
	{
		RenderTargetValue->SetClearColor(InBackgroundColor);
	}
}

bool UDMRenderTargetTextRenderer::GetHasHighlight() const
{
	return bHasHighlight;
}

void UDMRenderTargetTextRenderer::SetHasHighlight(bool bInHasHighlight)
{
	if (bHasHighlight == bInHasHighlight)
	{
		return;
	}

	bHasHighlight = bInHasHighlight;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();

}

const FLinearColor& UDMRenderTargetTextRenderer::GetHighlightColor() const
{
	if (UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue())
	{
		return RenderTargetValue->GetClearColor();
	}

	return FLinearColor::Black;
}

void UDMRenderTargetTextRenderer::SetHighlightColor(const FLinearColor& InHighlightColor)
{
	if (UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue())
	{
		RenderTargetValue->SetClearColor(InHighlightColor);
	}
}

ETextJustify::Type UDMRenderTargetTextRenderer::GetJustify() const
{
	return Justify;
}

void UDMRenderTargetTextRenderer::SetJustify(ETextJustify::Type InJustify)
{
	if (Justify == InJustify)
	{
		return;
	}

	Justify = InJustify;

	AsyncUpdateRenderTarget();
}

float UDMRenderTargetTextRenderer::GetLineHeight() const
{
	return LineHeight;
}

void UDMRenderTargetTextRenderer::SetLineHeight(float InLineHeight)
{
	if (LineHeight == InLineHeight)
	{
		return;
	}

	LineHeight = InLineHeight;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();

}

float UDMRenderTargetTextRenderer::GetPaddingLeft() const
{
	return PaddingLeft;
}

void UDMRenderTargetTextRenderer::SetPaddingLeft(float InPaddingLeft)
{
	if (PaddingLeft == InPaddingLeft)
	{
		return;
	}

	PaddingLeft = InPaddingLeft;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

float UDMRenderTargetTextRenderer::GetPaddingRight() const
{
	return PaddingRight;
}

void UDMRenderTargetTextRenderer::SetPaddingRight(float InPaddingRight)
{
	if (PaddingRight == InPaddingRight)
	{
		return;
	}

	PaddingRight = InPaddingRight;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

float UDMRenderTargetTextRenderer::GetPaddingTop() const
{
	return PaddingTop;
}

void UDMRenderTargetTextRenderer::SetPaddingTop(float InPaddingTop)
{
	if (PaddingTop == InPaddingTop)
	{
		return;
	}

	PaddingTop = InPaddingTop;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

float UDMRenderTargetTextRenderer::GetPaddingBottom() const
{
	return PaddingBottom;
}

void UDMRenderTargetTextRenderer::SetPaddingBottom(float InPaddingBottom)
{
	if (PaddingBottom == InPaddingBottom)
	{
		return;
	}

	PaddingBottom = InPaddingBottom;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

bool UDMRenderTargetTextRenderer::IsOverridingRenderTargetSize() const
{
	return bOverrideRenderTargetSize;
}

void UDMRenderTargetTextRenderer::SetOverrideRenderTargetSize(bool bInOverride)
{
	if (bOverrideRenderTargetSize == bInOverride)
	{
		return;
	}

	bOverrideRenderTargetSize = bInOverride;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

bool UDMRenderTargetTextRenderer::GetHasShadow() const
{
	return bHasShadow;
}

void UDMRenderTargetTextRenderer::SetHasShadow(bool bInHasShadow)
{
	if (bHasShadow == bInHasShadow)
	{
		return;
	}

	bHasShadow = bInHasShadow;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

const FLinearColor& UDMRenderTargetTextRenderer::GetShadowColor() const
{
	return ShadowColor;
}

void UDMRenderTargetTextRenderer::SetShadowColor(const FLinearColor& InShadowColor)
{
	if (ShadowColor == InShadowColor)
	{
		return;
	}

	ShadowColor = InShadowColor;

	AsyncUpdateRenderTarget();
}

const FVector2D& UDMRenderTargetTextRenderer::GetShadowOffset() const
{
	return ShadowOffset;
}

void UDMRenderTargetTextRenderer::SetShadowOffset(const FVector2D& InShadowOffset)
{
	if (ShadowOffset == InShadowOffset)
	{
		return;
	}

	ShadowOffset = InShadowOffset;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

bool UDMRenderTargetTextRenderer::GetAutoWrapText() const
{
	return bAutoWrapText;
}

void UDMRenderTargetTextRenderer::SetAutoWrapText(bool bInAutoWrap)
{
	if (bAutoWrapText == bInAutoWrap)
	{
		return;
	}

	bAutoWrapText = bInAutoWrap;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

float UDMRenderTargetTextRenderer::GetWrapTextAt() const
{
	return WrapTextAt;
}

void UDMRenderTargetTextRenderer::SetWrapTextAt(float InWrapAt)
{
	if (WrapTextAt == InWrapAt)
	{
		return;
	}

	WrapTextAt = InWrapAt;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

ETextWrappingPolicy UDMRenderTargetTextRenderer::GetWrappingPolicy() const
{
	return WrappingPolicy;
}

void UDMRenderTargetTextRenderer::SetWrappingPolicy(ETextWrappingPolicy InWrappingPolicy)
{
	if (WrappingPolicy == InWrappingPolicy)
	{
		return;
	}

	WrappingPolicy = InWrappingPolicy;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

ETextTransformPolicy UDMRenderTargetTextRenderer::GetTransformPolicy() const
{
	return TransformPolicy;
}

void UDMRenderTargetTextRenderer::SetTransformPolicy(ETextTransformPolicy InTransformPolicy)
{
	if (TransformPolicy == InTransformPolicy)
	{
		return;
	}

	TransformPolicy = InTransformPolicy;

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

ETextFlowDirection UDMRenderTargetTextRenderer::GetFlowDirection() const
{
	return FlowDirection;
}

void UDMRenderTargetTextRenderer::SetFlowDirection(ETextFlowDirection InFlowDirection)
{
	if (FlowDirection == InFlowDirection)
	{
		return;
	}

	FlowDirection = InFlowDirection;

	AsyncUpdateRenderTarget();
}

ETextShapingMethod UDMRenderTargetTextRenderer::GetShapingMethod() const
{
	return ShapingMethod;
}

void UDMRenderTargetTextRenderer::SetShapingMethod(ETextShapingMethod InShapingMethod)
{
	if (ShapingMethod == InShapingMethod)
	{
		return;
	}

	ShapingMethod = InShapingMethod;

	AsyncUpdateRenderTarget();
}

const TInstancedStruct<FSlateBrush>& UDMRenderTargetTextRenderer::GetStrikeBrush() const
{
	return StrikeBrush;
}

void UDMRenderTargetTextRenderer::SetStrikeBrush(const TInstancedStruct<FSlateBrush>& InStrikeBrush)
{
	if (StrikeBrush == InStrikeBrush)
	{
		return;
	}

	StrikeBrush = InStrikeBrush;

	AsyncUpdateRenderTarget();
}

void UDMRenderTargetTextRenderer::UpdateTextLines()
{
	const FString TextStr = Text.ToString();
	TArray<FString> NewLines;
	TextStr.ParseIntoArray(NewLines, TEXT("\n"), /* Cull empty */ false);

	for (FString& NewLine : NewLines)
	{
		if (NewLine.EndsWith(TEXT("\r")))
		{
			NewLine.LeftChopInline(1, EAllowShrinking::No);
		}
	}

	Lines.Empty();
	Lines.Reserve(NewLines.Num());

	for (const FString& NewLine : NewLines)
	{
		FDMTextLine& Line = Lines.AddDefaulted_GetRef();
		Line.Line = NewLine;
		Line.Widget = CreateTextWidget(FText::FromString(NewLine));
	}	

	bRecalculateTextSize = true;
	AsyncUpdateRenderTarget();
}

TSharedRef<STextBlock> UDMRenderTargetTextRenderer::CreateTextWidget(const FText& InText) const
{
	return SNew(STextBlock)
		.Font(FontInfo)
		.Text(InText)
		.LineHeightPercentage(LineHeight)
		.ColorAndOpacity(TextColor)
		.HighlightColor(bHasHighlight ? HighlightColor : TAttribute<FLinearColor>())
		.ShadowColorAndOpacity(bHasShadow ? ShadowColor : TAttribute<FLinearColor>())
		.ShadowOffset(bHasShadow ? ShadowOffset : TAttribute<FVector2D>())
		.Justification(Justify)
		.TransformPolicy(TransformPolicy)
		.TextFlowDirection(FlowDirection)
		.TextShapingMethod(ShapingMethod)
		.StrikeBrush(StrikeBrush.GetPtr<FSlateBrush>())
		.Margin(FMargin(PaddingLeft, PaddingTop, PaddingRight, PaddingBottom))
		.AutoWrapText(bAutoWrapText)
		.WrappingPolicy(bAutoWrapText ? WrappingPolicy : TAttribute<ETextWrappingPolicy>())
		.WrapTextAt(bAutoWrapText ? WrapTextAt : TAttribute<float>());
}

void UDMRenderTargetTextRenderer::CreateWidgetInstance()
{
	TSharedRef<SVerticalBox> NewWidget = SNew(SVerticalBox);

	for (FDMTextLine& Line : Lines)
	{
		if (!Line.Widget.IsValid())
		{
			Line.Widget = CreateTextWidget(FText::FromString(Line.Line));
		}

		NewWidget->AddSlot()
			.AutoHeight()
			[
				Line.Widget.ToSharedRef()
			];
	}

	Widget = NewWidget;
}

void UDMRenderTargetTextRenderer::SetCustomTextureSize()
{
	UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue();

	if (!RenderTargetValue)
	{
		return;
	}

	if (!Widget.IsValid())
	{
		return;
	}

	Widget->SlatePrepass(1.f);
	const FVector2D Size = Widget->GetDesiredSize();

	RenderTargetValue->SetTextureSize(FIntPoint(FMath::CeilToInt32(Size.X), FMath::CeilToInt32(Size.Y)));
	RenderTargetValue->FlushCreateRenderTarget();
}

void UDMRenderTargetTextRenderer::UpdateRenderTarget_Internal()
{
	if (Text.IsEmpty())
	{
		return;
	}

	UpdateTextLines();
	CreateWidgetInstance();

	if (bOverrideRenderTargetSize && bRecalculateTextSize)
	{
		SetCustomTextureSize();
	}

	Super::UpdateRenderTarget_Internal();
}

#undef LOCTEXT_NAMESPACE
