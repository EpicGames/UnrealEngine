// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RenderTargetRenderers/DMRenderTargetUMGWidgetRenderer.h"
#include "Components/MaterialValues/DMMaterialValueRenderTarget.h"
#include "Components/Widget.h"
#include "Containers/SharedString.h"
#include "Dom/JsonValue.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMRenderTargetUMGWidgetRenderer)

#define LOCTEXT_NAMESPACE "DMRenderTargetUWidgetRenderer"

UDMRenderTargetUMGWidgetRenderer::UDMRenderTargetUMGWidgetRenderer()
{
#if WITH_EDITOR
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMRenderTargetUMGWidgetRenderer, WidgetClass));
#endif
}

void UDMRenderTargetUMGWidgetRenderer::SetWidgetClass(TSubclassOf<UWidget> InWidgetClass)
{
	if (InWidgetClass == WidgetClass)
	{
		return;
	}

	WidgetClass = InWidgetClass;
	CreateWidgetInstance();
	AsyncUpdateRenderTarget();
}

#if WITH_EDITOR
void UDMRenderTargetUMGWidgetRenderer::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMRenderTargetUMGWidgetRenderer* OtherUMGRenderer = CastChecked<UDMRenderTargetUMGWidgetRenderer>(InOther);
	SetWidgetClass(OtherUMGRenderer->GetWidgetClass());
}

TSharedPtr<FJsonValue> UDMRenderTargetUMGWidgetRenderer::JsonSerialize() const
{
	return FDMJsonUtils::Serialize(WidgetClass.Get());
}

bool UDMRenderTargetUMGWidgetRenderer::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	TSubclassOf<UWidget> WidgetClassJson;

	if (FDMJsonUtils::Deserialize(InJsonValue, WidgetClassJson))
	{
		SetWidgetClass(WidgetClassJson);
		return true;
	}

	return false;
}

FText UDMRenderTargetUMGWidgetRenderer::GetComponentDescription() const
{
	return LOCTEXT("UMGWidget", "UMG Widget");
}

void UDMRenderTargetUMGWidgetRenderer::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMRenderTargetUMGWidgetRenderer, WidgetClass))
	{
		CreateWidgetInstance();
		AsyncUpdateRenderTarget();
	}
}
#endif

void UDMRenderTargetUMGWidgetRenderer::CreateWidgetInstance()
{
	UClass* WidgetClassLocal = WidgetClass.Get();

	if (!WidgetClassLocal)
	{
		return;
	}

	WidgetInstance = NewObject<UWidget>(this, WidgetClassLocal, NAME_None, RF_Transient);
	Widget = WidgetInstance->TakeWidget();
}

#undef LOCTEXT_NAMESPACE
