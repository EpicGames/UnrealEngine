// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RenderTargetRenderers/DMRenderTargetWidgetRendererBase.h"
#include "Components/MaterialValues/DMMaterialValueRenderTarget.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Slate/WidgetRenderer.h"
#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMRenderTargetWidgetRendererBase)

#define LOCTEXT_NAMESPACE "DMRenderTargetUMGWidgetRenderer"

UDMRenderTargetWidgetRendererBase::UDMRenderTargetWidgetRendererBase()
{
	WidgetRenderer = MakeShared<FWidgetRenderer>(/* Gamma correction */ false);
	WidgetRenderer->SetIsPrepassNeeded(true);
	WidgetRenderer->SetShouldClearTarget(true);
}

#if WITH_EDITOR
FText UDMRenderTargetWidgetRendererBase::GetComponentDescription() const
{
	return LOCTEXT("Widget", "Widget");
}
#endif

void UDMRenderTargetWidgetRendererBase::UpdateRenderTarget_Internal()
{
	UDMMaterialValueRenderTarget* RenderTargetValue = GetRenderTargetValue();

	if (!RenderTargetValue)
	{
		return;
	}

	RenderTargetValue->EnsureRenderTarget(/* Async */ false);

	UTextureRenderTarget2D* RenderTarget = RenderTargetValue->GetRenderTarget();

	if (!RenderTarget)
	{
		return;
	}

	if (!Widget.IsValid())
	{
		CreateWidgetInstance();

		if (!Widget.IsValid())
		{
			return;
		}
	}

	WidgetRenderer->DrawWidget(
		RenderTarget,
		Widget.ToSharedRef(),
		{(double)RenderTarget->SizeX, (double)RenderTarget->SizeY},
		/* Delta Time */ 0.f
	);
}

#undef LOCTEXT_NAMESPACE
