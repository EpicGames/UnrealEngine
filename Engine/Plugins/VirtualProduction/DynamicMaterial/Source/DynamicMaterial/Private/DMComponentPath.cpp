// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMComponentPath.h"
#include "DynamicMaterialModule.h"

FDMComponentPath::FDMComponentPath(FStringView InPath)
{
	Path = InPath;
}

FDMComponentPath::FDMComponentPath(const FString& InPathString)
	: FDMComponentPath(FStringView(InPathString))
{
}

FDMComponentPathSegment FDMComponentPath::GetFirstSegment()
{
	FStringView PathSegment = Path;
	FDMComponentPathSegment Segment(/* Token */ PathSegment, /* Parameter */ PathSegment.Left(0));

	int32 TokenLength = INDEX_NONE;

	if (PathSegment.FindChar(Separator, TokenLength))
	{
		PathSegment.LeftInline(TokenLength);
		Path.RightChopInline(TokenLength + 1);
	}
	else
	{
		Path.LeftInline(0);
	}

	Segment.Token = PathSegment;

	if (Segment.Token[Segment.Token.Len() - 1] != ParameterClose)
	{
		return Segment;
	}

	Segment.Token.LeftChopInline(1);

	int32 OpenPosition = INDEX_NONE;

	if (!Segment.Token.FindChar(ParameterOpen, OpenPosition))
	{
		UE_LOGF(LogDynamicMaterial, Error, "Malformed component path: %ls", PathSegment.GetData());
		return Segment;
	}

	Segment.Parameter = Segment.Token.Mid(OpenPosition + 1);
	Segment.Token.LeftInline(OpenPosition);

	if (Segment.Token.IsEmpty())
	{
		UE_LOGF(LogDynamicMaterial, Error, "Malformed component path token: %ls", PathSegment.GetData());
	}

	if (Segment.Parameter.IsEmpty())
	{
		UE_LOGF(LogDynamicMaterial, Error, "Malformed component path parameter: %ls", PathSegment.GetData());
	}

	return Segment;
}

FDMComponentPathSegment::FDMComponentPathSegment(FStringView InToken, FStringView InParameter)
	: Token(InToken)
	, Parameter(InParameter)
{
}

bool FDMComponentPath::IsLeaf() const
{
	return Path.IsEmpty();
}

bool FDMComponentPathSegment::HasParameter() const
{
	return Parameter.Len() > 0;
}

bool FDMComponentPathSegment::GetParameter(int32& OutParameter) const
{
	if (Parameter.IsEmpty())
	{
		return false;
	}

	/**
	 * Originally there was an IsNumeric check here, but that accepts '.', '-' and '+' so was unusable.
	 * This function should _only_ accept digits.
	 *
	 * The same goes for the Atoi function, to quote the docs:
	 * - The str argument to atoi and _wtoi has the following form: [whitespace] [sign] [digits]
	 *
	 * The input should be consistent with the output of the path segmnent generated by the code: [digits]
	 *
	 * Considering the need to check each character individually, it is just added up as that happens.
	 */

	constexpr TCHAR Zero = '0';
	constexpr TCHAR Nine = '9';
	int32 Multiplier = 1;
	int32 Value = 0;

	for (int32 Index = Parameter.Len() - 1; Index >= 0; --Index)
	{
		if (Parameter[Index] < Zero || Parameter[Index] > Nine)
		{
			return false;
		}

		Value += static_cast<int32>(Parameter[Index] - Zero) * Multiplier;
		Multiplier *= 10;
	}

	OutParameter = Value;

	return true;
}

bool FDMComponentPathSegment::GetParameter(FString& OutParameter) const
{
	if (Parameter.IsEmpty())
	{
		return false;
	}

	OutParameter = FString::ConstructFromPtrSize(Parameter.GetData(), Parameter.Len());
	return true;
}
