// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMDefs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMDefs)

int32 FDMUpdateGuard::GuardCount = 0;
uint32 FDMInitializationGuard::GuardCount = 0;

void UE::DynamicMaterial::ForEachMaterialPropertyType(TFunctionRef<EDMIterationResult(EDMMaterialPropertyType InType)> InCallable,
	EDMMaterialPropertyType InStart, EDMMaterialPropertyType InEnd)
{
	for (uint8 PropertyIndex = static_cast<uint8>(InStart); PropertyIndex <= static_cast<uint8>(InEnd); ++PropertyIndex)
	{
		const EDMMaterialPropertyType Property = static_cast<EDMMaterialPropertyType>(PropertyIndex);

		if (InCallable(Property) == EDMIterationResult::Break)
		{
			break;
		}
	}
}
