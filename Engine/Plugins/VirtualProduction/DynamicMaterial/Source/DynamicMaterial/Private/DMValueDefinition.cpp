// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMValueDefinition.h"

#include "Components/MaterialValues/DMMaterialValueBool.h"
#include "Components/MaterialValues/DMMaterialValueColorAtlas.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RPY.h"
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Containers/Map.h"

#if WITH_EDITOR
#include "Textures/SlateIcon.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMValueDefinition)

#define LOCTEXT_NAMESPACE "DMValueDefinition"

namespace UE::MaterialDesigner::Private
{
	const TArray<EDMValueType> TypeList = {
			EDMValueType::VT_Bool,
			EDMValueType::VT_Float1,
			EDMValueType::VT_Float2,
			EDMValueType::VT_Float3_RPY,
			EDMValueType::VT_Float3_RGB,
			EDMValueType::VT_Float3_XYZ,
			EDMValueType::VT_Float4_RGBA,
			EDMValueType::VT_Texture,
			EDMValueType::VT_ColorAtlas
	};

	const TMap<EDMValueType, FDMValueDefinition> TypeDefinitions = {
		{EDMValueType::VT_None,
			{EDMValueType::VT_None,
			0,
			LOCTEXT("None", "None"),
			{},
			TSubclassOf<UDMMaterialValue>()}
		},
		{EDMValueType::VT_Bool,
			{EDMValueType::VT_Bool,
			0,
			LOCTEXT("Bool", "Bool"),
			{LOCTEXT("Value", "Value")},
			UDMMaterialValueBool::StaticClass()}
		},
		{EDMValueType::VT_Float1,
			{EDMValueType::VT_Float1,
			1,
			LOCTEXT("Float", "Float"),
			{LOCTEXT("Value", "Value")},
			UDMMaterialValueFloat1::StaticClass()}
		},
		{EDMValueType::VT_Float2,
			{EDMValueType::VT_Float2,
			2,
			LOCTEXT("Vector2D", "Vector 2D"),
			{LOCTEXT("U", "U"),
				LOCTEXT("V", "V")},
			UDMMaterialValueFloat2::StaticClass()}
		},
		{EDMValueType::VT_Float3_RPY,
			{EDMValueType::VT_Float3_RPY,
			3,
			LOCTEXT("Rotator", "Rotator"),
			{LOCTEXT("Roll", "Roll"),
				LOCTEXT("Pitch", "Pitch"),
				LOCTEXT("Yaw", "Yaw")},
			UDMMaterialValueFloat3RPY::StaticClass()}
		},
		{EDMValueType::VT_Float3_RGB,
			{EDMValueType::VT_Float3_RGB,
			3,
			LOCTEXT("ColorRGB", "Color (RGB)"),
			{LOCTEXT("Red", "Red"),
				LOCTEXT("Green", "Green"),
				LOCTEXT("Blue", "Blue")},
			UDMMaterialValueFloat3RGB::StaticClass()}
		},
		{EDMValueType::VT_Float3_XYZ,
			{EDMValueType::VT_Float3_XYZ,
			3,
			LOCTEXT("Vector3D", "Vector 3D"),
			{LOCTEXT("X", "X"),
				LOCTEXT("Y", "Y"),
				LOCTEXT("Z", "Z")},
			UDMMaterialValueFloat3XYZ::StaticClass()}
		},
		{EDMValueType::VT_Float4_RGBA,
			{EDMValueType::VT_Float4_RGBA,
			4,
			LOCTEXT("ColorRGBA", "Color (RGBA)"),
			{LOCTEXT("Red", "Red"),
				LOCTEXT("Green", "Green"),
				LOCTEXT("Blue", "Blue"),
				LOCTEXT("Alpha", "Alpha")},
			UDMMaterialValueFloat4::StaticClass()}
		},
		{EDMValueType::VT_Float_Any,
			{EDMValueType::VT_Float_Any,
			0,
			LOCTEXT("FloatAny", "Float (Any)"),
			{},
			UDMMaterialValueFloat4::StaticClass()}
		},
		{EDMValueType::VT_Texture,
			{EDMValueType::VT_Texture,
			0,
			LOCTEXT("Texture", "Texture"),
			{},
			UDMMaterialValueTexture::StaticClass()}
		},
		{EDMValueType::VT_ColorAtlas,
			{EDMValueType::VT_ColorAtlas,
			4,
			LOCTEXT("ColorAtlas", "Color Atlas"),
			{LOCTEXT("Red", "Red"),
				LOCTEXT("Green", "Green"),
				LOCTEXT("Blue", "Blue"),
				LOCTEXT("Alpha", "Alpha")},
			UDMMaterialValueColorAtlas::StaticClass()}
		}
	};
}

const FDMValueDefinition& UDMValueDefinitionLibrary::GetTypeForFloatCount(uint8 Enum)
{
	using namespace UE::MaterialDesigner::Private;

	switch (Enum)
	{
		case 1:
			return TypeDefinitions[EDMValueType::VT_Float1];

		case 2:
			return TypeDefinitions[EDMValueType::VT_Float2];

		case 3:
			return TypeDefinitions[EDMValueType::VT_Float3_RGB]; // Default to RGB because we can't tell.

		case 4:
			return TypeDefinitions[EDMValueType::VT_Float4_RGBA];

		default:
			checkNoEntry();
			return TypeDefinitions[EDMValueType::VT_None];
	}
}

const FDMValueDefinition& UDMValueDefinitionLibrary::GetTypeForFloatCount(int32 Enum)
{
	return GetTypeForFloatCount(static_cast<uint8>(Enum));
}

bool UDMValueDefinitionLibrary::AreTypesCompatible(EDMValueType InA, EDMValueType InB, int32 InAChannel, int32 BChannel)
{
	using namespace UE::MaterialDesigner::Private;

	// While all floats are compatible with all over floats, this may change in the future.
	const FDMValueDefinition* TypeA = &TypeDefinitions[InA];
	const FDMValueDefinition* TypeB = &TypeDefinitions[InB];

	if (InAChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		const int32 Count =
			!!(InAChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL)
			+ !!(InAChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL)
			+ !!(InAChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL)
			+ !!(InAChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);

		switch (Count)
		{
			default:
				break;

			case 1:
				TypeA = &TypeDefinitions[EDMValueType::VT_Float1];
				break;

			case 2:
				TypeA = &TypeDefinitions[EDMValueType::VT_Float2];
				break;

			case 3:
				TypeA = &TypeDefinitions[EDMValueType::VT_Float3_RGB];
				break;

			case 4:
				TypeA = &TypeDefinitions[EDMValueType::VT_Float4_RGBA];
				break;
		}
	}

	if (BChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		const int32 Count =
			!!(BChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL)
			+ !!(BChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL)
			+ !!(BChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL)
			+ !!(BChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);

		switch (Count)
		{
			default:
				break;

			case 1:
				TypeB = &TypeDefinitions[EDMValueType::VT_Float1];
				break;

			case 2:
				TypeB = &TypeDefinitions[EDMValueType::VT_Float2];
				break;

			case 3:
				TypeB = &TypeDefinitions[EDMValueType::VT_Float3_RGB];
				break;

			case 4:
				TypeB = &TypeDefinitions[EDMValueType::VT_Float4_RGBA];
				break;
		}
	}

	if (TypeA->IsFloatType() && TypeB->IsFloatType())
	{
		return true;
	}

	return TypeA->GetType() == TypeB->GetType();
}

bool FDMValueDefinition::IsFloatType() const
{
	return FloatCount > 0 || Type == EDMValueType::VT_Float_Any;
}

bool FDMValueDefinition::IsFloat3Type() const
{
	return FloatCount == 3;
}

const FText& FDMValueDefinition::GetChannelName(int32 InChannel) const
{
	if (InChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		static const FText WholeChannel = LOCTEXT("WholeChannel", "Whole");
		return WholeChannel;
	}

	InChannel -= FDMMaterialStageConnectorChannel::FIRST_CHANNEL;

	if (ChannelNames.IsValidIndex(InChannel))
	{
		return ChannelNames[InChannel];
	}

	static const FText Error = LOCTEXT("Error", "Error");
	return Error;
}

const TArray<EDMValueType>& UDMValueDefinitionLibrary::GetValueTypes()
{
	using namespace UE::MaterialDesigner::Private;

	return TypeList;
}

const FDMValueDefinition& UDMValueDefinitionLibrary::GetValueDefinition(EDMValueType InValueType)
{
	using namespace UE::MaterialDesigner::Private;

	return TypeDefinitions[InValueType];
}

#if WITH_EDITOR
FSlateIcon UDMValueDefinitionLibrary::GetValueIcon(EDMValueType InType)
{
	using namespace UE::MaterialDesigner::Private;

	if (UClass* ValueClass = TypeDefinitions[InType].GetValueClass())
	{
		if (UDMMaterialValue* ValueCDO = ValueClass->GetDefaultObject<UDMMaterialValue>())
		{
			return ValueCDO->GetComponentIcon();
		}
	}

	return GetDefault<UDMMaterialComponent>()->GetComponentIcon();
}
#endif

#undef LOCTEXT_NAMESPACE
