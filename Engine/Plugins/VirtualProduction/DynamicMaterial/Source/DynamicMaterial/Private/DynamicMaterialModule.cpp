// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialModule.h"

#include "CoreGlobals.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectBase.h"

DEFINE_LOG_CATEGORY(LogDynamicMaterial);

static TAutoConsoleVariable<bool> CVarExportMaterials(
	TEXT("DM.ExportMaterials"),
	false,
	TEXT("If enabled, all materials, including previews, are exported to /Game/DynamicMaterials."),
	ECVF_SetByConsole);

bool FDynamicMaterialModule::IsMaterialExportEnabled()
{
	return CVarExportMaterials.GetValueOnAnyThread();
}

bool FDynamicMaterialModule::AreUObjectsSafe()
{
	return UObjectInitialized() && !IsEngineExitRequested();
}

FDynamicMaterialModule& FDynamicMaterialModule::Get()
{
	return FModuleManager::LoadModuleChecked<FDynamicMaterialModule>("DynamicMaterial");
}

IMPLEMENT_MODULE(FDynamicMaterialModule, DynamicMaterial)
