// Copyright Epic Games, Inc. All Rights Reserved.

#include "Material/DynamicMaterialInstance.h"

#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "Model/IDynamicMaterialModelEditorOnlyDataInterface.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialInstance)

#define LOCTEXT_NAMESPACE "DynamicMaterialInstance"

namespace UE::DynamicMaterial::Private
{
	const FLazyName ModelType = TEXT("ModelType");
}

const FString UDynamicMaterialInstance::ModelTypeTag_Material = TEXT("Material");
const FString UDynamicMaterialInstance::ModelTypeTag_Instance = TEXT("Instance");

#if WITH_EDITOR
FString UDynamicMaterialInstance::GetMaterialTypeTag(const FAssetData& InAssetData)
{
	using namespace UE::DynamicMaterial::Private;

	if (InAssetData.GetClass(EResolveClass::Yes) != UDynamicMaterialInstance::StaticClass())
	{
		return TEXT("");
	}

	if (!InAssetData.TagsAndValues.Contains(ModelType))
	{
		return TEXT("");		
	}

	return InAssetData.TagsAndValues.FindTag(ModelType).AsString();
}
#endif

UDynamicMaterialInstance::UDynamicMaterialInstance()
{
	MaterialModelBase = nullptr;

	bOutputTranslucentVelocity = true;
}

UDynamicMaterialModelBase* UDynamicMaterialInstance::GetMaterialModelBase() const
{
	return MaterialModelBase;
}

UDynamicMaterialModel* UDynamicMaterialInstance::GetMaterialModel() const
{
	if (IsValid(MaterialModelBase))
	{
		return MaterialModelBase->ResolveMaterialModel();
	}

	return nullptr;
}

void UDynamicMaterialInstance::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	using namespace UE::DynamicMaterial::Private;

	if (MaterialModelBase)
	{
		if (MaterialModelBase->IsA<UDynamicMaterialModel>())
		{
			Context.AddTag(FAssetRegistryTag(
				ModelType, 
				ModelTypeTag_Material,
				FAssetRegistryTag::TT_Alphabetical
			));
		}
		else if (MaterialModelBase->IsA<UDynamicMaterialModelDynamic>())
		{
			Context.AddTag(FAssetRegistryTag(
				ModelType,
				ModelTypeTag_Instance,
				FAssetRegistryTag::TT_Alphabetical
			));
		}
	}
}

#if WITH_EDITOR
void UDynamicMaterialInstance::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	using namespace UE::DynamicMaterial::Private;

	OutMetadata.Add(
		ModelType,
		FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("ModelType", "Model Type"))
		.SetTooltip(LOCTEXT("ModelTypeToolTip", "The type of Model used in this Material"))
		.SetImportantValue(TEXT("0"))
	);
}

void UDynamicMaterialInstance::SetMaterialModel(UDynamicMaterialModelBase* InMaterialModel)
{
	MaterialModelBase = InMaterialModel;

	if (InMaterialModel)
	{
		InMaterialModel->Rename(*InMaterialModel->GetName(), this, UE::DynamicMaterial::RenameFlags);
	}
}

void UDynamicMaterialInstance::InitializeMIDPublic()
{
	check(MaterialModelBase);

	UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel();
	check(MaterialModel);

	SetParentInternal(MaterialModel->GetGeneratedMaterial(), false);
	ClearParameterValues();
	UpdateCachedData();
}

void UDynamicMaterialInstance::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (MaterialModelBase)
	{
		MaterialModelBase->SetDynamicMaterialInstance(this);

		if (UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel())
		{
			if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = MaterialModel->GetEditorOnlyData())
			{
				ModelEditorOnlyData->RequestMaterialBuild(
					bInDuplicateForPIE
						? EDMBuildRequestType::Immediate
						: EDMBuildRequestType::Async
				);
			}
		}
	}
}

void UDynamicMaterialInstance::PostEditImport()
{
	Super::PostEditImport();

	if (MaterialModelBase)
	{
		MaterialModelBase->SetDynamicMaterialInstance(this);

		if (UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel())
		{
			if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = MaterialModel->GetEditorOnlyData())
			{
				ModelEditorOnlyData->RequestMaterialBuild();
			}
		}
	}
}

void UDynamicMaterialInstance::OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModel)
{
	if (MaterialModelBase != InMaterialModel)
	{
		return;
	}

	InitializeMIDPublic();
}
#endif

#undef LOCTEXT_NAMESPACE
