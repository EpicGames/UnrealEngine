// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DynamicMaterialModel.h"
#include "Components/DMMaterialParameter.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Components/DMTextureUV.h"
#include "DMComponentPath.h"
#include "DMDefs.h"
#include "DMValueDefinition.h"
#include "DynamicMaterialModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/IDynamicMaterialModelEditorOnlyDataInterface.h"

#if WITH_EDITOR
#include "Materials/Material.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialModel)

const FString UDynamicMaterialModel::ValuesPathToken = FString(TEXT("Values"));
const FString UDynamicMaterialModel::ParametersPathToken = FString(TEXT("Parameters"));
const FLazyName UDynamicMaterialModel::GlobalBaseColorValueName = FLazyName(TEXT("GlobalBaseColorValue"));
const FLazyName UDynamicMaterialModel::GlobalBaseColorParameterName = FLazyName(TEXT("GlobalBaseColor"));
const FLazyName UDynamicMaterialModel::GlobalEmissiveColorValueName = FLazyName(TEXT("GlobalEmissiveColorValue"));
const FLazyName UDynamicMaterialModel::GlobalEmissiveColorParameterName = FLazyName(TEXT("GlobalEmissiveColor"));
const FLazyName UDynamicMaterialModel::GlobalOpacityValueName = FLazyName(TEXT("GlobalOpacityValue"));
const FLazyName UDynamicMaterialModel::GlobalOpacityParameterName = FLazyName(TEXT("GlobalOpacity"));
const FLazyName UDynamicMaterialModel::GlobalMetallicValueName = FLazyName(TEXT("GlobalMetallicValue"));
const FLazyName UDynamicMaterialModel::GlobalMetallicParameterName = FLazyName(TEXT("GlobalMetallic"));
const FLazyName UDynamicMaterialModel::GlobalSpecularValueName = FLazyName(TEXT("GlobalSpecularValue"));
const FLazyName UDynamicMaterialModel::GlobalSpecularParameterName = FLazyName(TEXT("GlobalSpecular"));
const FLazyName UDynamicMaterialModel::GlobalRoughnessValueName = FLazyName(TEXT("GlobalRoughnessValue"));
const FLazyName UDynamicMaterialModel::GlobalRoughnessParameterName = FLazyName(TEXT("GlobalRoughness"));
const FLazyName UDynamicMaterialModel::GlobalNormalValueName = FLazyName(TEXT("GlobalNormalValue"));
const FLazyName UDynamicMaterialModel::GlobalNormalParameterName = FLazyName(TEXT("GlobalNormal"));
const FLazyName UDynamicMaterialModel::GlobalAnisotropyValueName = FLazyName(TEXT("GlobalAnisotropyValue"));
const FLazyName UDynamicMaterialModel::GlobalAnisotropyParameterName = FLazyName(TEXT("GlobalAnisotropy"));
const FLazyName UDynamicMaterialModel::GlobalWorldPositionOffsetValueName = FLazyName(TEXT("GlobalWorldPositionOffsetValue"));
const FLazyName UDynamicMaterialModel::GlobalWorldPositionOffsetParameterName = FLazyName(TEXT("GlobalWorldPositionOffset"));
const FLazyName UDynamicMaterialModel::GlobalAmbientOcclusionValueName = FLazyName(TEXT("GlobalAmbientOcclusionValue"));
const FLazyName UDynamicMaterialModel::GlobalAmbientOcclusionParameterName = FLazyName(TEXT("GlobalAmbientOcclusion"));
const FLazyName UDynamicMaterialModel::GlobalRefractionValueName = FLazyName(TEXT("GlobalRefractionValue"));
const FLazyName UDynamicMaterialModel::GlobalRefractionParameterName = FLazyName(TEXT("GlobalRefraction"));
const FLazyName UDynamicMaterialModel::GlobalTangentValueName = FLazyName(TEXT("GlobalTangentValue"));
const FLazyName UDynamicMaterialModel::GlobalTangentParameterName = FLazyName(TEXT("GlobalTangent"));
const FLazyName UDynamicMaterialModel::GlobalPixelDepthOffsetValueName = FLazyName(TEXT("GlobalPixelDepthOffsetValue"));
const FLazyName UDynamicMaterialModel::GlobalPixelDepthOffsetParameterName = FLazyName(TEXT("GlobalPixelDepthOffset"));
const FLazyName UDynamicMaterialModel::GlobalDisplacementValueName = FLazyName(TEXT("GlobalDisplacementValue"));
const FLazyName UDynamicMaterialModel::GlobalDisplacementParameterName = FLazyName(TEXT("GlobalDisplacement"));
const FLazyName UDynamicMaterialModel::GlobalSubsurfaceColorValueName = FLazyName(TEXT("GlobalSubsurfaceColorValue"));
const FLazyName UDynamicMaterialModel::GlobalSubsurfaceColorParameterName = FLazyName(TEXT("GlobalSubsurfaceColor"));
const FLazyName UDynamicMaterialModel::GlobalSurfaceThicknessValueName = FLazyName(TEXT("GlobalSurfaceThicknessValue"));
const FLazyName UDynamicMaterialModel::GlobalSurfaceThicknessParameterName = FLazyName(TEXT("GlobalSurfaceThickness"));

const FLazyName UDynamicMaterialModel::GlobalOffsetValueName = FLazyName(TEXT("GlobalOffsetValue"));
const FLazyName UDynamicMaterialModel::GlobalOffsetParameterName = FLazyName(TEXT("GlobalOffset"));
const FLazyName UDynamicMaterialModel::GlobalTilingValueName = FLazyName(TEXT("GlobalTilingValue"));
const FLazyName UDynamicMaterialModel::GlobalTilingParameterName = FLazyName(TEXT("GlobalTiling"));
const FLazyName UDynamicMaterialModel::GlobalRotationValueName = FLazyName(TEXT("GlobalRotationValue"));
const FLazyName UDynamicMaterialModel::GlobalRotationParameterName = FLazyName(TEXT("GlobalRotation"));

namespace UE::DynamicMaterial::Private
{
	const TMap<FName, EDMMaterialPropertyType> ValueNameToMaterialProperty = {
		{UDynamicMaterialModel::GlobalBaseColorValueName,           EDMMaterialPropertyType::BaseColor},
		{UDynamicMaterialModel::GlobalEmissiveColorValueName,       EDMMaterialPropertyType::EmissiveColor},
		{UDynamicMaterialModel::GlobalOpacityValueName,             EDMMaterialPropertyType::Opacity},
		{UDynamicMaterialModel::GlobalMetallicValueName,            EDMMaterialPropertyType::Metallic},
		{UDynamicMaterialModel::GlobalSpecularValueName,            EDMMaterialPropertyType::Specular},
		{UDynamicMaterialModel::GlobalRoughnessValueName,           EDMMaterialPropertyType::Roughness},
		{UDynamicMaterialModel::GlobalAnisotropyValueName,          EDMMaterialPropertyType::Anisotropy},
		{UDynamicMaterialModel::GlobalNormalValueName,              EDMMaterialPropertyType::Normal},
		{UDynamicMaterialModel::GlobalWorldPositionOffsetValueName, EDMMaterialPropertyType::WorldPositionOffset},
		{UDynamicMaterialModel::GlobalAmbientOcclusionValueName,    EDMMaterialPropertyType::AmbientOcclusion},
		{UDynamicMaterialModel::GlobalRefractionValueName,          EDMMaterialPropertyType::Refraction},
		{UDynamicMaterialModel::GlobalTangentValueName,             EDMMaterialPropertyType::Tangent},
		{UDynamicMaterialModel::GlobalPixelDepthOffsetValueName,    EDMMaterialPropertyType::PixelDepthOffset},
		{UDynamicMaterialModel::GlobalDisplacementValueName,        EDMMaterialPropertyType::Displacement},
		{UDynamicMaterialModel::GlobalSubsurfaceColorValueName,     EDMMaterialPropertyType::SubsurfaceColor},
		{UDynamicMaterialModel::GlobalSurfaceThicknessValueName,    EDMMaterialPropertyType::SurfaceThickness}
	};
}

UDynamicMaterialModel::UDynamicMaterialModel()
{
	DynamicMaterialInstance = nullptr;

#if WITH_EDITOR
	EditorOnlyDataSI = nullptr;
#endif

	auto AddFloatParameter = [this](TObjectPtr<UDMMaterialValue>& InValuePtr, FName InPropertyName, FName InParameterName, float InDefaultValue = 1.f, bool bInSetRange = false)
		{
			UDMMaterialValueFloat1* Property = CreateDefaultSubobject<UDMMaterialValueFloat1>(InPropertyName);
			Property->CachedParameterName = InParameterName;
			InValuePtr = Property;

#if WITH_EDITOR
			Property->SetDefaultValue(InDefaultValue);
			Property->ApplyDefaultValue();

			if (bInSetRange)
			{
				Property->SetValueRange({0.f, 1.f});
			}
#endif

			UDMMaterialParameter* Parameter = CreateDefaultSubobject<UDMMaterialParameter>(FName(*(InParameterName.GetPlainNameString() + TEXT("Parameter"))));
			Parameter->ParameterName = InParameterName;
			Property->Parameter = Parameter;

			ParameterMap.Add(InParameterName, Parameter);
		};

	auto AddVector2Parameter = [this](TObjectPtr<UDMMaterialValue>& InValuePtr, FName InPropertyName, FName InParameterName, const FVector2D& InDefaultValue)
		{
			UDMMaterialValueFloat2* Property = CreateDefaultSubobject<UDMMaterialValueFloat2>(InPropertyName);
			Property->CachedParameterName = InParameterName;
			InValuePtr = Property;

#if WITH_EDITOR
			Property->SetDefaultValue(InDefaultValue);
			Property->ApplyDefaultValue();
#endif

			UDMMaterialParameter* Parameter = CreateDefaultSubobject<UDMMaterialParameter>(FName(*(InParameterName.GetPlainNameString() + TEXT("Parameter"))));
			Parameter->ParameterName = InParameterName;
			Property->Parameter = Parameter;

			ParameterMap.Add(InParameterName, Parameter);
		};

	FDMUpdateGuard Guard;

	AddFloatParameter(GlobalBaseColorParameterValue, GlobalBaseColorValueName, GlobalBaseColorParameterName);
	AddFloatParameter(GlobalEmissiveColorParameterValue, GlobalEmissiveColorValueName, GlobalEmissiveColorParameterName);
	AddFloatParameter(GlobalOpacityParameterValue, GlobalOpacityValueName, GlobalOpacityParameterName, 1.f, /* Set Range */ true);
	AddFloatParameter(GlobalMetallicParameterValue, GlobalMetallicValueName, GlobalMetallicParameterName);
	AddFloatParameter(GlobalSpecularParameterValue, GlobalSpecularValueName, GlobalSpecularParameterName);
	AddFloatParameter(GlobalRoughnessParameterValue, GlobalRoughnessValueName, GlobalRoughnessParameterName);
	AddFloatParameter(GlobalNormalParameterValue, GlobalNormalValueName, GlobalNormalParameterName);
	AddFloatParameter(GlobalAnisotropyParameterValue, GlobalAnisotropyValueName, GlobalAnisotropyParameterName);
	AddFloatParameter(GlobalWorldPositionOffsetParameterValue, GlobalWorldPositionOffsetValueName, GlobalWorldPositionOffsetParameterName);
	AddFloatParameter(GlobalAmbientOcclusionParameterValue, GlobalAmbientOcclusionValueName, GlobalAmbientOcclusionParameterName);
	AddFloatParameter(GlobalTangentParameterValue, GlobalTangentValueName, GlobalTangentParameterName);
	AddFloatParameter(GlobalRefractionParameterValue, GlobalRefractionValueName, GlobalRefractionParameterName);
	AddFloatParameter(GlobalPixelDepthOffsetParameterValue, GlobalPixelDepthOffsetValueName, GlobalPixelDepthOffsetParameterName);
	AddFloatParameter(GlobalDisplacementParameterValue, GlobalDisplacementValueName, GlobalDisplacementParameterName);
	AddFloatParameter(GlobalSubsurfaceColorParameterValue, GlobalSubsurfaceColorValueName, GlobalSubsurfaceColorParameterName);
	AddFloatParameter(GlobalSurfaceThicknessParameterValue, GlobalSurfaceThicknessValueName, GlobalSurfaceThicknessParameterName);

	AddVector2Parameter(GlobalOffsetParameterValue, GlobalOffsetValueName, GlobalOffsetParameterName, FVector2D::ZeroVector);
	AddVector2Parameter(GlobalTilingParameterValue, GlobalTilingValueName, GlobalTilingParameterName, FVector2D::UnitVector);
	AddFloatParameter(GlobalRotationParameterValue, GlobalRotationValueName, GlobalRotationParameterName, /* Default Value */ 0.f);
}

void UDynamicMaterialModel::SetDynamicMaterialInstance(UDynamicMaterialInstance* InDynamicMaterialInstance)
{
	if (DynamicMaterialInstance == InDynamicMaterialInstance)
	{
		return;
	}

	DynamicMaterialInstance = InDynamicMaterialInstance;

	if (InDynamicMaterialInstance)
	{
		ApplyComponents(InDynamicMaterialInstance);
	}

#if WITH_EDITOR
	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->PostEditorDuplicate();
	}
#endif
}

UDMMaterialValueFloat1* UDynamicMaterialModel::GetGlobalOpacityValue() const
{
	return GetTypedGlobalParameterValue<UDMMaterialValueFloat1>(GlobalOpacityValueName);
}

UDMMaterialValue* UDynamicMaterialModel::GetGlobalParameterValue(FName InName) const
{
	using namespace UE::DynamicMaterial::Private;

	if (const EDMMaterialPropertyType* MaterialPropertyPtr = ValueNameToMaterialProperty.Find(InName))
	{
		return GetGlobalParameterValueForMaterialProperty(*MaterialPropertyPtr);
	}

	if (InName == GlobalOffsetValueName)
	{
		return GlobalOffsetParameterValue;
	}

	if (InName == GlobalTilingValueName)
	{
		return GlobalTilingParameterValue;
	}

	if (InName == GlobalRotationValueName)
	{
		return GlobalRotationParameterValue;
	}

	return nullptr;
}

bool UDynamicMaterialModel::IsModelValid() const
{
	return (!HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed) && IsValidChecked(this));
}

UDMMaterialValue* UDynamicMaterialModel::GetGlobalParameterValueForMaterialProperty(EDMMaterialPropertyType InProperty) const
{
	switch (InProperty)
	{
		case EDMMaterialPropertyType::BaseColor:
			return GlobalBaseColorParameterValue;

		case EDMMaterialPropertyType::EmissiveColor:
			return GlobalEmissiveColorParameterValue;

		case EDMMaterialPropertyType::Opacity:
			return GlobalOpacityParameterValue;

		case EDMMaterialPropertyType::Metallic:
			return GlobalMetallicParameterValue;

		case EDMMaterialPropertyType::Specular:
			return GlobalSpecularParameterValue;

		case EDMMaterialPropertyType::Roughness:
			return GlobalRoughnessParameterValue;

		case EDMMaterialPropertyType::Anisotropy:
			return GlobalAnisotropyParameterValue;

		case EDMMaterialPropertyType::Normal:
			return GlobalNormalParameterValue;

		case EDMMaterialPropertyType::WorldPositionOffset:
			return GlobalWorldPositionOffsetParameterValue;

		case EDMMaterialPropertyType::AmbientOcclusion:
			return GlobalAmbientOcclusionParameterValue;

		case EDMMaterialPropertyType::Refraction:
			return GlobalRefractionParameterValue;

		case EDMMaterialPropertyType::Tangent:
			return GlobalTangentParameterValue;

		case EDMMaterialPropertyType::PixelDepthOffset:
			return GlobalPixelDepthOffsetParameterValue;

		case EDMMaterialPropertyType::Displacement:
			return GlobalDisplacementParameterValue;

		case EDMMaterialPropertyType::SubsurfaceColor:
			return GlobalSubsurfaceColorParameterValue;

		case EDMMaterialPropertyType::SurfaceThickness:
			return GlobalSurfaceThicknessParameterValue;

		default:
			return nullptr;
	}
}

void UDynamicMaterialModel::ForEachGlobalParameter(TFunctionRef<void(UDMMaterialValue* InGlobalParameterValue)> InCallable)
{
	InCallable(GlobalBaseColorParameterValue);
	InCallable(GlobalEmissiveColorParameterValue);
	InCallable(GlobalOpacityParameterValue);
	InCallable(GlobalMetallicParameterValue);
	InCallable(GlobalSpecularParameterValue);
	InCallable(GlobalRoughnessParameterValue);
	InCallable(GlobalAnisotropyParameterValue);
	InCallable(GlobalNormalParameterValue);
	InCallable(GlobalWorldPositionOffsetParameterValue);
	InCallable(GlobalAmbientOcclusionParameterValue);
	InCallable(GlobalRefractionParameterValue);
	InCallable(GlobalTangentParameterValue);
	InCallable(GlobalPixelDepthOffsetParameterValue);
	InCallable(GlobalDisplacementParameterValue);
	InCallable(GlobalSubsurfaceColorParameterValue);
	InCallable(GlobalSurfaceThicknessParameterValue);
	InCallable(GlobalOffsetParameterValue);
	InCallable(GlobalTilingParameterValue);
	InCallable(GlobalRotationParameterValue);
}

UDMMaterialComponent* UDynamicMaterialModel::GetComponentByPath(const FString& InPath) const
{
	FDMComponentPath Path(InPath);
	return GetComponentByPath(Path);
}

UDMMaterialComponent* UDynamicMaterialModel::GetComponentByPath(FDMComponentPath& InPath) const
{
	if (InPath.IsLeaf())
	{
		return nullptr;
	}

	const FDMComponentPathSegment FirstComponent = InPath.GetFirstSegment();

	if (FirstComponent.GetToken() == ValuesPathToken)
	{
		int32 ValueIndex;

		if (FirstComponent.GetParameter(ValueIndex))
		{
			if (Values.IsValidIndex(ValueIndex))
			{
				return Values[ValueIndex]->GetComponentByPath(InPath);
			}
		}

		return nullptr;
	}

	if (FirstComponent.GetToken() == ParametersPathToken)
	{
		FString ParameterStr;

		if (FirstComponent.GetParameter(ParameterStr))
		{
			const FName ParameterName = FName(*ParameterStr);

			if (const TWeakObjectPtr<UDMMaterialParameter>* ParameterPtr = ParameterMap.Find(ParameterName))
			{
				return (*ParameterPtr)->GetComponentByPath(InPath);
			}
		}

		return nullptr;
	}

#if WITH_EDITOR
	if (IDynamicMaterialModelEditorOnlyDataInterface* EditorOnlyData = GetEditorOnlyData())
	{
		return EditorOnlyData->GetSubComponentByPath(InPath, FirstComponent);
	}
#endif

	return nullptr;
}

UDMMaterialValue* UDynamicMaterialModel::GetValueByName(FName InName) const
{
	for (UDMMaterialValue* Value : Values)
	{
		if (Value->GetParameter() && Value->GetParameter()->GetParameterName() == InName)
		{
			return Value;
		}
	}

	return nullptr;
}

#if WITH_EDITOR
TScriptInterface<IDynamicMaterialModelEditorOnlyDataInterface> UDynamicMaterialModel::BP_GetEditorOnlyData() const
{
	TScriptInterface<IDynamicMaterialModelEditorOnlyDataInterface> Interface;
	Interface.SetObject(EditorOnlyDataSI);

	return Interface;
}

IDynamicMaterialModelEditorOnlyDataInterface* UDynamicMaterialModel::GetEditorOnlyData() const
{
	return Cast<IDynamicMaterialModelEditorOnlyDataInterface>(EditorOnlyDataSI);
}

UDMMaterialValue* UDynamicMaterialModel::AddValue(TSubclassOf<UDMMaterialValue> InValueClass)
{
	UDMMaterialValue* NewValue = UDMMaterialValue::CreateMaterialValue(this, TEXT(""), InValueClass, false);
	Values.Add(NewValue);

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->OnValueListUpdate();
	}

	return NewValue;
}

void UDynamicMaterialModel::AddRuntimeComponentReference(UDMMaterialComponent* InValue)
{
	RuntimeComponents.Add(InValue);
}

void UDynamicMaterialModel::RemoveRuntimeComponentReference(UDMMaterialComponent* InValue)
{
	RuntimeComponents.Remove(InValue);
}

void UDynamicMaterialModel::RemoveValueByParameterName(FName InName)
{
	int32 FoundIndex = Values.IndexOfByPredicate([InName](UDMMaterialValue* Value)
		{
			return Value->GetParameter() && Value->GetParameter()->GetParameterName() == InName;
		});

	if (FoundIndex == INDEX_NONE)
	{
		return;
	}

	Values.RemoveAt(FoundIndex);

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->RequestMaterialBuild();
		ModelEditorOnlyData->OnValueListUpdate();
	}
}

bool UDynamicMaterialModel::HasParameterName(FName InParameterName) const
{
	if (Values.ContainsByPredicate([InParameterName](const UDMMaterialValue* InValue)
		{
			return IsValid(InValue) && InValue->GetParameter() && InValue->GetParameter()->GetParameterName() == InParameterName;
		}))
	{
		return true;
	}

	const TWeakObjectPtr<UDMMaterialParameter>* ParameterWeak = ParameterMap.Find(InParameterName);

	return (ParameterWeak && ParameterWeak->IsValid());
}

UDMMaterialParameter* UDynamicMaterialModel::CreateUniqueParameter(FName InBaseName)
{
	check(!InBaseName.IsNone());

	UDMMaterialParameter* NewParameter = nullptr;

	{
		const FDMInitializationGuard InitGuard;

		NewParameter = NewObject<UDMMaterialParameter>(this, NAME_None, RF_Transactional);
		check(NewParameter);

		RenameParameter(NewParameter, InBaseName);
	}

	ParameterMap.Emplace(NewParameter->GetParameterName(), NewParameter);
	NewParameter->SetComponentState(EDMComponentLifetimeState::Added);

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->RequestMaterialBuild();
	}

	return NewParameter;
}

void UDynamicMaterialModel::RenameParameter(UDMMaterialParameter* InParameter, FName InBaseName)
{
	check(InParameter);
	check(!InBaseName.IsNone());

	if (!InParameter->ParameterName.IsNone())
	{
		FreeParameter(InParameter);
	}

	if (GUndo)
	{
		InParameter->Modify();
	}

	InParameter->ParameterName = CreateUniqueParameterName(InBaseName);

	ParameterMap.Emplace(InParameter->ParameterName, InParameter);

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->RequestMaterialBuild();
	}
}

void UDynamicMaterialModel::FreeParameter(UDMMaterialParameter* InParameter)
{
	check(InParameter);

	if (InParameter->ParameterName.IsNone())
	{
		return;
	}

	const FName ParameterName = InParameter->GetParameterName();

	for (TMap<FName, TWeakObjectPtr<UDMMaterialParameter>>::TIterator It(ParameterMap); It; ++It)
	{
		UDMMaterialParameter* Parameter = It->Value.Get();

		if (!Parameter || Parameter->GetParameterName() == ParameterName)
		{
			It.RemoveCurrent();
		}
	}

	if (GUndo)
	{
		InParameter->Modify();
	}

	InParameter->ParameterName = NAME_None;
	InParameter->SetComponentState(EDMComponentLifetimeState::Removed);
}

bool UDynamicMaterialModel::ConditionalFreeParameter(UDMMaterialParameter* InParameter)
{
	check(InParameter);

	// Parameteres without names are not in the map
	if (InParameter->ParameterName.IsNone())
	{
		return true;
	}

	TWeakObjectPtr<UDMMaterialParameter>* ParameterPtr = ParameterMap.Find(InParameter->ParameterName);

	// Parameter name isn't in the map or it's mapped to a different object.
	if (!ParameterPtr || InParameter != ParameterPtr->Get())
	{
		return true;
	}

	// We're in the map at the given name.
	return false;
}
#endif

void UDynamicMaterialModel::OnValueUpdated(UDMMaterialValue* InValue, EDMUpdateType InUpdateType)
{
	check(InValue);

	if (InValue->GetMaterialModel() != this)
	{
		return;
	}

	if (!EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure) && IsValid(DynamicMaterialInstance))
	{
		InValue->SetMIDParameter(DynamicMaterialInstance);
	}

	OnValueUpdateDelegate.Broadcast(this, InValue);

#if WITH_EDITOR
	if (IDynamicMaterialModelEditorOnlyDataInterface* EditorOnlyData = GetEditorOnlyData())
	{
		EditorOnlyData->OnValueUpdated(InValue, InUpdateType);
	}
#endif
}

void UDynamicMaterialModel::OnTextureUVUpdated(UDMTextureUV* InTextureUV)
{
	check(InTextureUV);

	if (InTextureUV->GetMaterialModel() != this)
	{
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsThisNotNull(this, "UDynamicMaterialModel::OnTextureUVUpdated") && IsValidChecked(this) && IsValid(DynamicMaterialInstance))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		InTextureUV->SetMIDParameters(DynamicMaterialInstance);
	}

	OnTextureUVUpdateDelegate.Broadcast(this, InTextureUV);

#if WITH_EDITOR
	if (IDynamicMaterialModelEditorOnlyDataInterface* EditorOnlyData = GetEditorOnlyData())
	{
		EditorOnlyData->OnTextureUVUpdated(InTextureUV);
	}
#endif
}

void UDynamicMaterialModel::ApplyComponents(UMaterialInstanceDynamic* InMID)
{
	ForEachGlobalParameter(
		[this, InMID](UDMMaterialValue* InValue)
		{
			if (!InValue)
			{
				return;
			}

#if WITH_EDITOR
			if (InValue->IsComponentCreated())
			{
				InValue->SetComponentState(EDMComponentLifetimeState::Added);
			}
#endif

			InValue->SetMIDParameter(InMID);
		}
	);

	for (UDMMaterialValue* Value : Values)
	{
#if WITH_EDITOR
		if (Value->IsComponentCreated())
		{
			Value->SetComponentState(EDMComponentLifetimeState::Added);
		}
#endif

		Value->SetMIDParameter(InMID);
	}

	for (const TObjectPtr<UDMMaterialComponent>& RuntimeComponent : RuntimeComponents)
	{
		if (UDMMaterialValue* Value = Cast<UDMMaterialValue>(RuntimeComponent))
		{
			Value->SetMIDParameter(InMID);
		}
		else if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(RuntimeComponent))
		{
			TextureUV->SetMIDParameters(InMID);
		}
	}
}

void UDynamicMaterialModel::PostLoad()
{
	Super::PostLoad();

	FixGlobalParameterValues();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// This requires a parameter name change which is editor-only code. It cannot correct itself at runtime.
	// Open assets in the editor first to fix version upgrades!
	if (GlobalOpacityValue)
	{
		GlobalOpacityParameterValue = GlobalOpacityValue;

		if (IDynamicMaterialModelEditorOnlyDataInterface* EditorOnlyDataInterface = GetEditorOnlyData())
		{
			EditorOnlyDataInterface->SetPropertyComponent(EDMMaterialPropertyType::Opacity, TEXT("AlphaValue"), GlobalOpacityValue);
			EditorOnlyDataInterface->SetPropertyComponent(EDMMaterialPropertyType::OpacityMask, TEXT("AlphaValue"), GlobalOpacityValue);
		}

		GlobalOpacityValue->SetParameterName("VALUE_GlobalOpacity");

		if (FMath::IsNearlyZero(GlobalOpacityValue->GetDefaultValue()))
		{
			GlobalOpacityValue->SetDefaultValue(1.f);

			if (FMath::IsNearlyZero(GlobalOpacityValue->GetValue()))
			{
				GlobalOpacityValue->SetValue(1.f);
			}
		}

		GlobalOpacityValue = nullptr;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	SetFlags(RF_Transactional);

	ReinitComponents();
#endif
}

#if WITH_EDITOR
void UDynamicMaterialModel::PostEditUndo()
{
	Super::PostEditUndo();

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->RequestMaterialBuild();
	}
}

void UDynamicMaterialModel::PostEditImport()
{
	Super::PostEditImport();

	FixGlobalVars();
	PostEditorDuplicate();
	ReinitComponents();

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->RequestMaterialBuild();
	}
}

void UDynamicMaterialModel::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (!bInDuplicateForPIE)
	{
		FixGlobalVars();
		PostEditorDuplicate();
		ReinitComponents();

		if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
		{
			ModelEditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Async);
		}
	}
}

void UDynamicMaterialModel::PostEditorDuplicate()
{
	ForEachGlobalParameter(
		[this](UDMMaterialValue* InValue)
		{
			if (!InValue)
			{
				return;
			}

			if (GUndo)
			{
				InValue->Modify();
			}

			InValue->PostEditorDuplicate(this, nullptr);
		}
	);

	for (const TPair<FName, TWeakObjectPtr<UDMMaterialParameter>>& ParameterPair : ParameterMap)
	{
		if (UDMMaterialParameter* Parameter = ParameterPair.Value.Get())
		{
			if (GUndo)
			{
				Parameter->Modify();
			}

			Parameter->PostEditorDuplicate(this, nullptr);
		}
	}

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->PostEditorDuplicate();
	}
}
#endif

void UDynamicMaterialModel::FixGlobalParameterValues()
{
	if (UDMMaterialValueFloat1* FloatValue = Cast<UDMMaterialValueFloat1>(GlobalOpacityParameterValue))
	{
		FloatValue->SetValueRange({0.f, 1.f});
	}
}

#if WITH_EDITOR
void UDynamicMaterialModel::ReinitComponents()
{
	if (IsValid(DynamicMaterial) && !DynamicMaterial->HasAnyFlags(RF_DuplicateTransient))
	{
		if (GUndo)
		{
			DynamicMaterial->Modify();
		}

		DynamicMaterial->SetFlags(RF_DuplicateTransient);
	}

	FixGlobalVars();

	// Clean up old parameters
	ParameterMap.Empty();

	TArray<UObject*> Subobjects;
	GetObjectsWithOuter(this, Subobjects, EGetObjectsFlags::None);

	for (UObject* Subobject : Subobjects)
	{
		if (UDMMaterialValue* Value = Cast<UDMMaterialValue>(Subobject))
		{
			if (UDMMaterialParameter* Parameter = Value->GetParameter())
			{
				ParameterMap.Emplace(Parameter->GetParameterName(), Parameter);
			}
		}
		else if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(Subobject))
		{
			TArray<UDMMaterialParameter*> Parameters = TextureUV->GetParameters();

			for (UDMMaterialParameter* Parameter : Parameters)
			{
				ParameterMap.Emplace(Parameter->GetParameterName(), Parameter);
			}
		}
	}

	if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = GetEditorOnlyData())
	{
		ModelEditorOnlyData->ReinitComponents();
	}
}

void UDynamicMaterialModel::FixGlobalVars()
{
	auto FixGlobalVar = [this](FName InValueName, FName InParameterName)
		{
			UDMMaterialValue* GlobalVar = GetGlobalParameterValue(InValueName);

			if (!GlobalVar)
			{
				return;
			}

			GlobalVar->CachedParameterName = InParameterName;

			UDMMaterialParameter* Parameter = GlobalVar->GetParameter();

			if (!Parameter)
			{
				GlobalVar->Parameter = NewObject<UDMMaterialParameter>(this, NAME_None, RF_Transactional);
				Parameter = GlobalVar->Parameter;
			}

			if (Parameter->GetParameterName() != InParameterName)
			{
				Parameter->ParameterName = InParameterName;
			}
		};

	FixGlobalVar(GlobalOpacityValueName, GlobalOpacityParameterName);
	FixGlobalVar(GlobalRoughnessValueName, GlobalRoughnessParameterName);
	FixGlobalVar(GlobalNormalValueName, GlobalNormalParameterName);
	FixGlobalVar(GlobalSpecularValueName, GlobalSpecularParameterName);
	FixGlobalVar(GlobalMetallicValueName, GlobalMetallicParameterName);
	FixGlobalVar(GlobalAnisotropyValueName, GlobalAnisotropyParameterName);
	FixGlobalVar(GlobalWorldPositionOffsetValueName, GlobalWorldPositionOffsetParameterName);
	FixGlobalVar(GlobalAmbientOcclusionValueName, GlobalAmbientOcclusionParameterName);
	FixGlobalVar(GlobalRefractionValueName, GlobalRefractionParameterName);
	FixGlobalVar(GlobalPixelDepthOffsetValueName, GlobalPixelDepthOffsetParameterName);
	FixGlobalVar(GlobalDisplacementValueName, GlobalDisplacementParameterName);
	FixGlobalVar(GlobalOffsetValueName, GlobalOffsetParameterName);
	FixGlobalVar(GlobalTilingValueName, GlobalTilingParameterName);
	FixGlobalVar(GlobalRotationValueName, GlobalRotationParameterName);
}

FName UDynamicMaterialModel::CreateUniqueParameterName(FName InBaseName)
{
	int32 CurrentTest = 0;
	FName UniqueName = InBaseName;

	auto UpdateName = [&InBaseName, &CurrentTest, &UniqueName]()
		{
			++CurrentTest;
			UniqueName = FName(InBaseName.ToString() + TEXT("_") + FString::FromInt(CurrentTest));
		};

	bool bIsNameUnique = false;

	while (!bIsNameUnique)
	{
		bIsNameUnique = true;

		if (Values.ContainsByPredicate([UniqueName](const UDMMaterialValue* Value)
			{
				return Value->GetMaterialParameterName() == UniqueName;
			}))
		{
			bIsNameUnique = false;
			UpdateName();
		}
	}

	bIsNameUnique = false;

	while (!bIsNameUnique)
	{
		bIsNameUnique = true;

		if (const TWeakObjectPtr<UDMMaterialParameter>* CurrentParameter = ParameterMap.Find(UniqueName))
		{
			if (CurrentParameter->IsValid())
			{
				bIsNameUnique = false;
				UpdateName();
			}
			else
			{
				ParameterMap.Remove(UniqueName);
				break; // Not needed, but informative.
			}
		}
	}

	return UniqueName;
}
#endif
