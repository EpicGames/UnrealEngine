// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DynamicMaterialModelDynamic.h"

#include "Components/DMTextureUVDynamic.h"
#include "DMComponentPath.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/DynamicMaterialModel.h"

#if WITH_EDITOR
#include "Components/DMMaterialValue.h"
#include "Components/DMTextureUV.h"
#include "DynamicMaterialModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialModelDynamic)

const FString UDynamicMaterialModelDynamic::ParentModelPathToken = TEXT("ParentModel");
const FString UDynamicMaterialModelDynamic::DynamicComponentsPathToken = TEXT("DynamicComponents");

#if WITH_EDITOR
UDynamicMaterialModelDynamic* UDynamicMaterialModelDynamic::Create(UObject* InOuter, UDynamicMaterialModel* InParentModel)
{
	check(InParentModel);

	const FName ModelName = MakeUniqueObjectName(InOuter, UDynamicMaterialModelDynamic::StaticClass(), TEXT("MaterialDesignerModelInstance"));

	UDynamicMaterialModelDynamic* NewModelDynamic = NewObject<UDynamicMaterialModelDynamic>(InOuter, ModelName, RF_Transactional);
	NewModelDynamic->ParentModelSoft = InParentModel;
	NewModelDynamic->ParentModel = InParentModel;
	NewModelDynamic->InitComponents();

	return NewModelDynamic;
}
#endif

UDynamicMaterialModelDynamic::UDynamicMaterialModelDynamic()
	: ParentModel(nullptr)
{
}

UDynamicMaterialModel* UDynamicMaterialModelDynamic::GetParentModel() const
{
	return const_cast<UDynamicMaterialModelDynamic*>(this)->EnsureParentModel();
}

#if WITH_EDITOR
UDMMaterialComponentDynamic* UDynamicMaterialModelDynamic::GetComponentDynamic(FName InName)
{
	return DynamicComponents.FindRef(InName);
}

bool UDynamicMaterialModelDynamic::AddComponentDynamic(UDMMaterialComponentDynamic* InValueDynamic)
{
	if (!InValueDynamic)
	{
		return false;
	}

	const FName ParentValueName = InValueDynamic->GetParentComponentName();

	if (DynamicComponents.Contains(ParentValueName))
	{
		return false;
	}

	DynamicComponents.Add(ParentValueName, InValueDynamic);

	InValueDynamic->SetComponentState(EDMComponentLifetimeState::Added);

	return true;
}

bool UDynamicMaterialModelDynamic::RemoveComponentDynamic(UDMMaterialComponentDynamic* InValueDynamic)
{
	if (!InValueDynamic)
	{
		return false;
	}

	const FName ParentValueName = InValueDynamic->GetParentComponentName();

	TObjectPtr<UDMMaterialComponentDynamic>* FoundComponent = DynamicComponents.Find(ParentValueName);

	if (!FoundComponent || FoundComponent->Get() != InValueDynamic)
	{
		return false;
	}

	InValueDynamic->SetComponentState(EDMComponentLifetimeState::Removed);

	DynamicComponents.Remove(ParentValueName);

	return true;
}
#endif

void UDynamicMaterialModelDynamic::OnValueUpdated(UDMMaterialValueDynamic* InValueDynamic)
{
	check(InValueDynamic);

	if (InValueDynamic->GetMaterialModelDynamic() != this)
	{
		return;
	}

	if (IsValid(DynamicMaterialInstance))
	{
		InValueDynamic->SetMIDParameter(DynamicMaterialInstance);
	}

	OnValueDynamicUpdateDelegate.Broadcast(this, InValueDynamic);
}

void UDynamicMaterialModelDynamic::OnTextureUVUpdated(UDMTextureUVDynamic* InTextureUVDynamic)
{
	check(InTextureUVDynamic);

	if (InTextureUVDynamic->GetMaterialModelDynamic() != this)
	{
		return;
	}

	if (IsValid(DynamicMaterialInstance))
	{
		InTextureUVDynamic->SetMIDParameters(DynamicMaterialInstance);
	}

	OnTextureUVDynamicUpdateDelegate.Broadcast(this, InTextureUVDynamic);
}

void UDynamicMaterialModelDynamic::ApplyComponents(UMaterialInstanceDynamic* InMID)
{
	for (const TPair<FName, TObjectPtr<UDMMaterialComponentDynamic>>& DynamicComponent : DynamicComponents)
	{
		if (UDMMaterialValueDynamic* ValueDynamic = Cast<UDMMaterialValueDynamic>(DynamicComponent.Value))
		{
			ValueDynamic->SetMIDParameter(InMID);
		}
		else if (UDMTextureUVDynamic* TextureUVDynamic = Cast<UDMTextureUVDynamic>(DynamicComponent.Value))
		{
			TextureUVDynamic->SetMIDParameters(InMID);
		}
	}
}

UDynamicMaterialModel* UDynamicMaterialModelDynamic::ResolveMaterialModel()
{
	if (UDynamicMaterialModel* ParentModelLocal = GetParentModel())
	{
		return ParentModelLocal;
	}

	return nullptr;
}

UDynamicMaterialInstance* UDynamicMaterialModelDynamic::GetDynamicMaterialInstance() const
{
	return DynamicMaterialInstance;
}

void UDynamicMaterialModelDynamic::SetDynamicMaterialInstance(UDynamicMaterialInstance* InDynamicMaterialInstance)
{
	if (DynamicMaterialInstance == InDynamicMaterialInstance)
	{
		return;
	}

	DynamicMaterialInstance = InDynamicMaterialInstance;

	if (InDynamicMaterialInstance)
	{
		ApplyComponents(InDynamicMaterialInstance);
	}
}

UMaterial* UDynamicMaterialModelDynamic::GetGeneratedMaterial() const
{
	if (UDynamicMaterialModel* ParentModelLocal = GetParentModel())
	{
		return ParentModelLocal->GetGeneratedMaterial();
	}

	return nullptr;
}

void UDynamicMaterialModelDynamic::PostLoad()
{
	UObject::PostLoad();

	EnsureParentModel();
}

UDynamicMaterialModel* UDynamicMaterialModelDynamic::EnsureParentModel()
{
	if (!ParentModel)
	{
		ParentModel = ParentModelSoft.LoadSynchronous();
	}

	return ParentModel;
}

#if WITH_EDITOR
void UDynamicMaterialModelDynamic::InitComponents()
{
	UDynamicMaterialModel* ParentModelLocal = GetParentModel();

	if (!ParentModelLocal)
	{
		return;
	}

	int32 GlobalParamCount = 0;
	ParentModelLocal->ForEachGlobalParameter([&GlobalParamCount](UDMMaterialValue*) { ++GlobalParamCount; });

	const TArray<UDMMaterialValue*>& ParentValues = ParentModelLocal->GetValues();
	const TSet<TObjectPtr<UDMMaterialComponent>>& RuntimeComponents = ParentModelLocal->GetRuntimeComponents();

	DynamicComponents.Empty(GlobalParamCount + ParentValues.Num() + RuntimeComponents.Num()); // Rough estimate on the number of global params

	ParentModelLocal->ForEachGlobalParameter(
		[this](UDMMaterialValue* InValue)
		{
			DynamicComponents.Add(InValue->GetFName(), InValue->ToDynamic(this));
		}
	);

	for (UDMMaterialValue* ParentValue : ParentValues)
	{
		DynamicComponents.Add(ParentValue->GetFName(), ParentValue->ToDynamic(this));
	}

	for (const TObjectPtr<UDMMaterialComponent>& RuntimeComponent : RuntimeComponents)
	{
		if (UDMMaterialValue* ParentValue = Cast<UDMMaterialValue>(RuntimeComponent))
		{
			DynamicComponents.Add(ParentValue->GetFName(), ParentValue->ToDynamic(this));
		}
		else if (UDMTextureUV* ParentTextureUV = Cast<UDMTextureUV>(RuntimeComponent))
		{
			DynamicComponents.Add(ParentTextureUV->GetFName(), ParentTextureUV->ToDynamic(this));
		}		
	}
}

void UDynamicMaterialModelDynamic::EnsureComponents()
{
	UDynamicMaterialModel* ParentModelLocal = GetParentModel();

	if (!ParentModelLocal)
	{
		return;
	}

	int32 GlobalParamCount = 0;
	ParentModelLocal->ForEachGlobalParameter([&GlobalParamCount](UDMMaterialValue*) { ++GlobalParamCount; });

	const TArray<UDMMaterialValue*>& ParentValues = ParentModelLocal->GetValues();
	const TSet<TObjectPtr<UDMMaterialComponent>>& RuntimeComponents = ParentModelLocal->GetRuntimeComponents();

	int32 RequiredComponentCount = GlobalParamCount + ParentValues.Num() + RuntimeComponents.Num();

	DynamicComponents.Reserve(RequiredComponentCount);

	ParentModelLocal->ForEachGlobalParameter(
		[this, &RequiredComponentCount](UDMMaterialValue* InValue)
		{
			if (!IsValid(InValue))
			{
				--RequiredComponentCount;
				return;
			}

			const FName ValueName = InValue->GetFName();

			if (!DynamicComponents.Contains(ValueName))
			{
				DynamicComponents.Add(ValueName, InValue->ToDynamic(this));
			}
		}
	);

	for (UDMMaterialValue* ParentValue : ParentValues)
	{
		if (!IsValid(ParentValue))
		{
			--RequiredComponentCount;
			continue;
		}

		const FName ParentValueName = ParentValue->GetFName();

		if (!DynamicComponents.Contains(ParentValueName))
		{
			DynamicComponents.Add(ParentValueName, ParentValue->ToDynamic(this));
		}
	}

	for (const TObjectPtr<UDMMaterialComponent>& RuntimeComponent : RuntimeComponents)
	{
		if (!IsValid(RuntimeComponent))
		{
			--RequiredComponentCount;
			continue;
		}

		if (UDMMaterialValue* ParentValue = Cast<UDMMaterialValue>(RuntimeComponent))
		{
			const FName ParentValueName = ParentValue->GetFName();

			if (!DynamicComponents.Contains(ParentValueName))
			{
				DynamicComponents.Add(ParentValueName, ParentValue->ToDynamic(this));
			}
		}
		else if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(RuntimeComponent))
		{
			const FName ParentValueName = TextureUV->GetFName();

			if (!DynamicComponents.Contains(ParentValueName))
			{
				DynamicComponents.Add(ParentValueName, TextureUV->ToDynamic(this));
			}
		}
		else
		{
			--RequiredComponentCount;
			UE_LOGF(LogDynamicMaterial, Error, "Invalid Component type while creating Material Designer Instance [%ls]", *RuntimeComponent->GetName());
		}
	}

	const int32 RemovedComponentCount = DynamicComponents.Num() - RequiredComponentCount;

	if (RemovedComponentCount <= 0)
	{
		return;
	}

	TArray<FName> ValuesToRemove;
	ValuesToRemove.Reserve(RemovedComponentCount);

	for (const TPair<FName, TObjectPtr<UDMMaterialComponentDynamic>>& DynamicValuePair : DynamicComponents)
	{
		if (!DynamicValuePair.Value->GetParentComponent())
		{
			ValuesToRemove.Add(DynamicValuePair.Key);
		}
	}

	for (const FName ParentValueName : ValuesToRemove)
	{
		DynamicComponents.Remove(ParentValueName);
	}
}
#endif

UDMMaterialComponent* UDynamicMaterialModelDynamic::GetComponentByPath(const FString& InPath) const
{
	FDMComponentPath Path(InPath);
	return GetComponentByPath(Path);
}

UDMMaterialComponent* UDynamicMaterialModelDynamic::GetComponentByPath(FDMComponentPath& InPath) const
{
	if (InPath.IsLeaf())
	{
		return nullptr;
	}

	const FDMComponentPathSegment FirstComponent = InPath.GetFirstSegment();

	if (FirstComponent.GetToken() == ParentModelPathToken)
	{
		if (UDynamicMaterialModel* ParentModelLocal = GetParentModel())
		{
			return ParentModelLocal->GetComponentByPath(InPath);
		}

		return nullptr;
	}

	if (FirstComponent.GetToken() == DynamicComponentsPathToken)
	{
		FString ParameterStr;

		if (FirstComponent.GetParameter(ParameterStr))
		{
			const FName ParameterName = FName(*ParameterStr);

			if (const TObjectPtr<UDMMaterialComponentDynamic>* ComponentPtr = DynamicComponents.Find(ParameterName))
			{
				return (*ComponentPtr)->GetComponentByPath(InPath);
			}
		}

		return nullptr;
	}

	return nullptr;
}

const TMap<FName, TObjectPtr<UDMMaterialComponentDynamic>>& UDynamicMaterialModelDynamic::GetComponentMap() const
{
	return DynamicComponents;
}

#if WITH_EDITOR
UDynamicMaterialModel* UDynamicMaterialModelDynamic::ToEditable(UObject* InOuter)
{
	UDynamicMaterialModel* CurrentModel = ResolveMaterialModel();

	if (!CurrentModel)
	{
		return nullptr;
	}

	UDynamicMaterialModel* NewModel = Cast<UDynamicMaterialModel>(StaticDuplicateObject(CurrentModel, InOuter));

	if (!NewModel)
	{
		return nullptr;
	}

	for (const TPair<FName, TObjectPtr<UDMMaterialComponentDynamic>>& ComponentPair : DynamicComponents)
	{
		UObject* NewParentObject = FindObjectFast<UDMMaterialComponent>(NewModel, ComponentPair.Key);

		if (UDMMaterialComponent* NewComponent = Cast<UDMMaterialComponent>(NewParentObject))
		{
			if (UDMMaterialComponentDynamic* ComponentDynamic = Cast<UDMMaterialComponentDynamic>(ComponentPair.Value))
			{
				ComponentDynamic->CopyDynamicPropertiesTo(NewComponent);
			}
		}
	}

	return NewModel;
}
#endif
