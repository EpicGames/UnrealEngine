// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMJsonUtils.h"

#if WITH_EDITOR

#include "DMDefs.h"
#include "Dom/JsonValue.h"
#include "DynamicMaterialModule.h"
#include "JsonObjectConverter.h"

#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMJsonUtils)

#if WITH_EDITOR

namespace UE::DynamicMaterial::Private
{
	const FJsonObject::FStringType JsonKey_Class(TEXT("Class"));
	const FJsonObject::FStringType JsonKey_Data(TEXT("Data"));
}

TSharedPtr<FJsonValue> FDMJsonUtils::SerializeNumber(double InNumber)
{
	return MakeShared<FJsonValueNumber>(InNumber);
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(bool bInValue)
{
	return MakeShared<FJsonValueBoolean>(bInValue);
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const FString& InString)
{
	return MakeShared<FJsonValueString>(InString);
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const FText& InText)
{
	return MakeShared<FJsonValueString>(InText.ToString());
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const FName& InName)
{
	return MakeShared<FJsonValueString>(InName.GetPlainNameString());
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const UClass* InClass)
{
	if (IsValid(InClass))
	{
		return Serialize(InClass->GetClassPathName().ToString());
	}

	return MakeShared<FJsonValueNull>();
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const UScriptStruct* InScriptStruct, const void* InData)
{
	if (!InData)
	{
		return MakeShared<FJsonValueNull>();
	}

	if (!IsValid(InScriptStruct))
	{
		UE_LOGF(LogDynamicMaterial, Error, "Invalid script struct.");
		return MakeShared<FJsonValueNull>();
	}

	TSharedRef<FJsonObject> JsonStruct = MakeShared<FJsonObject>();

	if (FJsonObjectConverter::UStructToJsonObject(InScriptStruct, InData, JsonStruct))
	{
		return MakeShared<FJsonValueObject>(JsonStruct);
	}

	UE_LOGF(LogDynamicMaterial, Error,"Failed to convert struct to json. [%ls]", *InScriptStruct->GetName());
	return MakeShared<FJsonValueNull>();
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const UObject* InObject)
{
	if (!IsValid(InObject))
	{
		return MakeShared<FJsonValueNull>();
	}

	if (InObject->IsAsset())
	{
		return Serialize(InObject->GetPathName());
	}

	using namespace UE::DynamicMaterial::Private;

	if (InObject->Implements<UDMJsonSerializable>())
	{
		TSharedPtr<FJsonValue> JsonObject = Cast<IDMJsonSerializable>(InObject)->JsonSerialize();

		if (JsonObject.IsValid())
		{
			return Serialize({
				{JsonKey_Class, Serialize(InObject->GetClass())},
				{JsonKey_Data, JsonObject}
			});
		}
	}
	else
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();

		if (FJsonObjectConverter::UStructToJsonObject(InObject->GetClass(), InObject, JsonObject))
		{
			return Serialize({
				{JsonKey_Class, Serialize(InObject->GetClass())},
				{JsonKey_Data, MakeShared<FJsonValueObject>(JsonObject)}
			});
		}
	}

	UE_LOGF(LogDynamicMaterial, Error, "Failed to convert object to json. [%ls] [%ls]", 
		*InObject->GetClass()->GetName(), *InObject->GetPathName());

	return MakeShared<FJsonValueNull>();
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const FObjectPtr& InObject)
{
	return Serialize(InObject.Get());
}

TSharedPtr<FJsonValue> FDMJsonUtils::Serialize(const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& InMap)
{
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->Values = InMap;

	return MakeShared<FJsonValueObject>(JsonObject);
}

bool FDMJsonUtils::DeserializeNumber(const TSharedPtr<FJsonValue>& InJsonValue, double& OutNumber)
{
	if (InJsonValue->TryGetNumber(OutNumber))
	{
		return true;
	}

	UE_LOGF(LogDynamicMaterial, Error, "Failed to read number from json.");
	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, bool& bOutValue)
{
	if (InJsonValue->TryGetBool(bOutValue))
	{
		return true;
	}

	UE_LOGF(LogDynamicMaterial, Error,"Failed to read bool from json.");
	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FString& OutString)
{
	if (InJsonValue->TryGetString(OutString))
	{
		return true;
	}

	UE_LOGF(LogDynamicMaterial, Error, "Failed to read string from json.");
	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FText& OutText)
{
	FString String;

	if (InJsonValue->TryGetString(String))
	{
		OutText = FText::FromString(String);
		return true;
	}

	UE_LOGF(LogDynamicMaterial, Error, "Failed to read text from json.");
	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FName& OutName)
{
	FString String;

	if (Deserialize(InJsonValue, String))
	{
		OutName = FName(*String);
		return true;
	}

	UE_LOGF(LogDynamicMaterial, Error, "Failed to read name from json.");
	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, UClass*& OutClass)
{
	FString ClassString;

	if (Deserialize(InJsonValue, ClassString))
	{
		if (UClass* Class = LoadClass<UObject>(nullptr, *ClassString))
		{
			OutClass = Class;
			return true;
		}

		UE_LOGF(LogDynamicMaterial, Error, "Failed to find class. [%ls]", *ClassString);
	}
	else
	{
		UE_LOGF(LogDynamicMaterial, Error, "Failed to read class path.");
	}

	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, const UScriptStruct* InScriptStruct, void* OutData)
{
	if (!IsValid(InScriptStruct))
	{
		return false;
	}

	const TSharedPtr<FJsonObject>* JsonObject;

	if (InJsonValue->TryGetObject(JsonObject))
	{
		if (FJsonObjectConverter::JsonObjectToUStruct((*JsonObject).ToSharedRef(), InScriptStruct, OutData))
		{
			return true;
		}

		UE_LOGF(LogDynamicMaterial, Error, "Failed to find deserialize struct. [%ls]", *InScriptStruct->GetName());
	}
	else
	{
		UE_LOGF(LogDynamicMaterial, Error, "Failed to read struct data. [%ls]", *InScriptStruct->GetName());
	}

	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FObjectPtr& OutObject, UObject* InOuter)
{
	UObject* Object;

	if (Deserialize(InJsonValue, Object, InOuter))
	{
		OutObject = Object;
		return true;
	}

	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, const UEnum* InEnum, int64& OutValue)
{
	if (!InEnum)
	{
		UE_LOGF(LogDynamicMaterial, Error, "Invalid enum.");
		return false;
	}

	int64 Value;

	if (Deserialize(InJsonValue, Value))
	{
		if (InEnum->IsValidEnumValueOrBitfield(Value))
		{
			OutValue = Value;
			return true;
		}

		UE_LOGF(LogDynamicMaterial, Error, "Invalid enum value. [%ls] [%lli]", *InEnum->GetName(), Value);
	}
	else
	{
		UE_LOGF(LogDynamicMaterial, Error, "Failed to read data from json.");
	}

	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, UObject*& OutObject, UObject* InOuter)
{
	if (InJsonValue->IsNull())
	{
		OutObject = nullptr;
		return true;
	}

	FString ObjectPath;

	if (InJsonValue->TryGetString(ObjectPath))
	{
		// Objects in packages have no outer, even if one is provided
		if (UObject* Object = LoadObject<UObject>(nullptr, *ObjectPath))
		{
			OutObject = Object;
			return true;
		}

		UE_LOGF(LogDynamicMaterial, Error, "Missing object. [%ls]", *ObjectPath);
		return false;
	}

	do
	{
		using namespace UE::DynamicMaterial::Private;

		TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

		if (!Deserialize(InJsonValue, Data))
		{
			UE_LOGF(LogDynamicMaterial, Error, "Failed to read data from json.");
			break;
		}

		if (!Data.Contains(JsonKey_Class) || !Data.Contains(JsonKey_Data))
		{
			UE_LOGF(LogDynamicMaterial, Error, "Missing data in json.");
			break;
		}

		UClass* Class = nullptr;

		if (!Deserialize(Data[JsonKey_Class], Class))
		{
			UE_LOGF(LogDynamicMaterial, Error, "Failed to read class from json.");
			break;
		}

		const TSharedPtr<FJsonObject>* ObjectData = nullptr;

		if (!Data[JsonKey_Data]->TryGetObject(ObjectData))
		{
			UE_LOGF(LogDynamicMaterial, Error, "Failed to read object data from json.");
			break;
		}

		UObject* Object = NewObject<UObject>(InOuter, Class, NAME_None, RF_Transactional);

		if (!Object)
		{
			UE_LOGF(LogDynamicMaterial, Error, "Failed to find instantiate class. [%ls]", *Class->GetName());
			break;
		}

		if (Object->Implements<UDMJsonSerializable>())
		{
			if (!Cast<IDMJsonSerializable>(Object)->JsonDeserialize(Data[JsonKey_Data]))
			{
				UE_LOGF(LogDynamicMaterial, Error, "Failed to find deserialize object.");
				break;
			}
		}
		else
		{
			if (!FJsonObjectConverter::JsonObjectToUStruct(ObjectData->ToSharedRef(), Class, Object))
			{
				UE_LOGF(LogDynamicMaterial, Error, "Failed to find deserialize object.");
				break;
			}
		}

		if (InOuter)
		{
			Object->Rename(/* New name */ nullptr, InOuter, UE::DynamicMaterial::RenameFlags);
		}

		return true;
	}
	while (false);

	return false;
}

bool FDMJsonUtils::Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& OutMap)
{
	const TSharedPtr<FJsonObject>* JsonObject;

	if (InJsonValue->TryGetObject(JsonObject))
	{
		OutMap = (*JsonObject)->Values;
		return true;
	}

	UE_LOGF(LogDynamicMaterial, Error, "Failed to read object from json.");		
	return false;
}

#endif
