// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMUtils.h"

#include "DMDefs.h"
#include "DMTextureSetMaterialProperty.h"
#include "SceneTypes.h"

#if WITH_EDITOR
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#endif

#if WITH_EDITOR
FString FDMUtils::CreateNodeComment(const ANSICHAR* InFile, int InLine, const ANSICHAR* InFunction, const FString* InComment)
{
	FString File = FPaths::GetCleanFilename(ANSI_TO_TCHAR(InFile)).RightChop(2);

	if (InComment)
	{
		return FString::Printf(TEXT("%s[%d]: %hs: %s"), *File, InLine, InFunction, **InComment);
	}
	else
	{
		return FString::Printf(TEXT("%s[%d]: %hs"), *File, InLine, InFunction);
	}
}
#endif

EDMTextureSetMaterialProperty FDMUtils::MaterialPropertyTypeToTextureSetMaterialProperty(EDMMaterialPropertyType InPropertyType)
{
	switch (InPropertyType)
	{
		case EDMMaterialPropertyType::BaseColor:
			return EDMTextureSetMaterialProperty::BaseColor;

		case EDMMaterialPropertyType::EmissiveColor:
			return EDMTextureSetMaterialProperty::EmissiveColor;

		case EDMMaterialPropertyType::Opacity:
			return EDMTextureSetMaterialProperty::Opacity;

		case EDMMaterialPropertyType::OpacityMask:
			return EDMTextureSetMaterialProperty::OpacityMask;

		case EDMMaterialPropertyType::Roughness:
			return EDMTextureSetMaterialProperty::Roughness;

		case EDMMaterialPropertyType::Specular:
			return EDMTextureSetMaterialProperty::Specular;

		case EDMMaterialPropertyType::Metallic:
			return EDMTextureSetMaterialProperty::Metallic;

		case EDMMaterialPropertyType::Normal:
			return EDMTextureSetMaterialProperty::Normal;

		case EDMMaterialPropertyType::PixelDepthOffset:
			return EDMTextureSetMaterialProperty::PixelDepthOffset;

		case EDMMaterialPropertyType::WorldPositionOffset:
			return EDMTextureSetMaterialProperty::WorldPositionOffset;

		case EDMMaterialPropertyType::AmbientOcclusion:
			return EDMTextureSetMaterialProperty::AmbientOcclusion;

		case EDMMaterialPropertyType::Anisotropy:
			return EDMTextureSetMaterialProperty::Anisotropy;

		case EDMMaterialPropertyType::Refraction:
			return EDMTextureSetMaterialProperty::Refraction;

		case EDMMaterialPropertyType::Tangent:
			return EDMTextureSetMaterialProperty::Tangent;

		case EDMMaterialPropertyType::Displacement:
			return EDMTextureSetMaterialProperty::Displacement;

		case EDMMaterialPropertyType::SubsurfaceColor:
			return EDMTextureSetMaterialProperty::SubsurfaceColor;

		case EDMMaterialPropertyType::SurfaceThickness:
			return EDMTextureSetMaterialProperty::SurfaceThickness;

		default:
			return EDMTextureSetMaterialProperty::None;
	}
}

EDMMaterialPropertyType FDMUtils::TextureSetMaterialPropertyToMaterialPropertyType(EDMTextureSetMaterialProperty InPropertyType)
{
	switch (InPropertyType)
	{
		case EDMTextureSetMaterialProperty::BaseColor:
			return EDMMaterialPropertyType::BaseColor;

		case EDMTextureSetMaterialProperty::EmissiveColor:
			return EDMMaterialPropertyType::EmissiveColor;

		case EDMTextureSetMaterialProperty::Opacity:
			return EDMMaterialPropertyType::Opacity;

		case EDMTextureSetMaterialProperty::OpacityMask:
			return EDMMaterialPropertyType::OpacityMask;

		case EDMTextureSetMaterialProperty::Roughness:
			return EDMMaterialPropertyType::Roughness;

		case EDMTextureSetMaterialProperty::Specular:
			return EDMMaterialPropertyType::Specular;

		case EDMTextureSetMaterialProperty::Metallic:
			return EDMMaterialPropertyType::Metallic;

		case EDMTextureSetMaterialProperty::Normal:
			return EDMMaterialPropertyType::Normal;

		case EDMTextureSetMaterialProperty::PixelDepthOffset:
			return EDMMaterialPropertyType::PixelDepthOffset;

		case EDMTextureSetMaterialProperty::WorldPositionOffset:
			return EDMMaterialPropertyType::WorldPositionOffset;

		case EDMTextureSetMaterialProperty::AmbientOcclusion:
			return EDMMaterialPropertyType::AmbientOcclusion;

		case EDMTextureSetMaterialProperty::Anisotropy:
			return EDMMaterialPropertyType::Anisotropy;

		case EDMTextureSetMaterialProperty::Refraction:
			return EDMMaterialPropertyType::Refraction;

		case EDMTextureSetMaterialProperty::Tangent:
			return EDMMaterialPropertyType::Tangent;

		case EDMTextureSetMaterialProperty::Displacement:
			return EDMMaterialPropertyType::Displacement;

		case EDMTextureSetMaterialProperty::SubsurfaceColor:
			return EDMMaterialPropertyType::SubsurfaceColor;

		case EDMTextureSetMaterialProperty::SurfaceThickness:
			return EDMMaterialPropertyType::SurfaceThickness;

		default:
			return EDMMaterialPropertyType::None;
	}
}

EMaterialProperty FDMUtils::MaterialPropertyTypeToMaterialProperty(EDMMaterialPropertyType InPropertyType)
{
	switch (InPropertyType)
	{
		case EDMMaterialPropertyType::BaseColor:
			return EMaterialProperty::MP_BaseColor;

		case EDMMaterialPropertyType::EmissiveColor:
			return EMaterialProperty::MP_EmissiveColor;

		case EDMMaterialPropertyType::Opacity:
			return EMaterialProperty::MP_Opacity;

		case EDMMaterialPropertyType::OpacityMask:
			return EMaterialProperty::MP_OpacityMask;

		case EDMMaterialPropertyType::Roughness:
			return EMaterialProperty::MP_Roughness;

		case EDMMaterialPropertyType::Specular:
			return EMaterialProperty::MP_Specular;

		case EDMMaterialPropertyType::Metallic:
			return EMaterialProperty::MP_Metallic;

		case EDMMaterialPropertyType::Normal:
			return EMaterialProperty::MP_Normal;

		case EDMMaterialPropertyType::PixelDepthOffset:
			return EMaterialProperty::MP_PixelDepthOffset;

		case EDMMaterialPropertyType::WorldPositionOffset:
			return EMaterialProperty::MP_WorldPositionOffset;

		case EDMMaterialPropertyType::AmbientOcclusion:
			return EMaterialProperty::MP_AmbientOcclusion;

		case EDMMaterialPropertyType::Anisotropy:
			return EMaterialProperty::MP_Anisotropy;

		case EDMMaterialPropertyType::Refraction:
			return EMaterialProperty::MP_Refraction;

		case EDMMaterialPropertyType::Tangent:
			return EMaterialProperty::MP_Tangent;

		case EDMMaterialPropertyType::Displacement:
			return EMaterialProperty::MP_Displacement;

		case EDMMaterialPropertyType::SubsurfaceColor:
			return EMaterialProperty::MP_SubsurfaceColor;

		case EDMMaterialPropertyType::SurfaceThickness:
			return EMaterialProperty::MP_SurfaceThickness;

		default:
			return EMaterialProperty::MP_MAX;
	}
}

EDMMaterialPropertyType FDMUtils::MaterialPropertyToMaterialPropertyType(EMaterialProperty InPropertyType)
{
	switch (InPropertyType)
	{
		case EMaterialProperty::MP_BaseColor:
			return EDMMaterialPropertyType::BaseColor;

		case EMaterialProperty::MP_EmissiveColor:
			return EDMMaterialPropertyType::EmissiveColor;

		case EMaterialProperty::MP_Opacity:
			return EDMMaterialPropertyType::Opacity;

		case EMaterialProperty::MP_OpacityMask:
			return EDMMaterialPropertyType::OpacityMask;

		case EMaterialProperty::MP_Roughness:
			return EDMMaterialPropertyType::Roughness;

		case EMaterialProperty::MP_Specular:
			return EDMMaterialPropertyType::Specular;

		case EMaterialProperty::MP_Metallic:
			return EDMMaterialPropertyType::Metallic;

		case EMaterialProperty::MP_Normal:
			return EDMMaterialPropertyType::Normal;

		case EMaterialProperty::MP_PixelDepthOffset:
			return EDMMaterialPropertyType::PixelDepthOffset;

		case EMaterialProperty::MP_WorldPositionOffset:
			return EDMMaterialPropertyType::WorldPositionOffset;

		case EMaterialProperty::MP_AmbientOcclusion:
			return EDMMaterialPropertyType::AmbientOcclusion;

		case EMaterialProperty::MP_Anisotropy:
			return EDMMaterialPropertyType::Anisotropy;

		case EMaterialProperty::MP_Refraction:
			return EDMMaterialPropertyType::Refraction;

		case EMaterialProperty::MP_Tangent:
			return EDMMaterialPropertyType::Tangent;

		case EMaterialProperty::MP_Displacement:
			return EDMMaterialPropertyType::Displacement;

		case EMaterialProperty::MP_SubsurfaceColor:
			return EDMMaterialPropertyType::SubsurfaceColor;

		case EMaterialProperty::MP_SurfaceThickness:
			return EDMMaterialPropertyType::SurfaceThickness;

		default:
			return EDMMaterialPropertyType::None;
	}
}
