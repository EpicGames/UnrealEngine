// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "DMDefs.h"
#include "Misc/NotifyHook.h"
#include "DMMaterialComponent.generated.h"

class UDynamicMaterialModel;
enum class EDMUpdateType : uint8;
struct FDMComponentPath;
struct FDMComponentPathSegment;
struct FSlateIcon;

UENUM(BlueprintType)
enum class EDMComponentLifetimeState : uint8
{
	Created,
	Added,
	Removed
};

/**
 * The base class for all material components. Has a few useful things.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, meta = (DisplayName = "Material Designer Component"))
class UDMMaterialComponent : public UObject, public FNotifyHook
{
	GENERATED_BODY()

	friend class UDynamicMaterialEditorSettings;

public:
#if WITH_EDITOR
	/** Returns true if we can attempt to un-dirty a component. */
	DYNAMICMATERIAL_API static bool CanClean();

	/** Called to prevent cleaning for InDelayFor. **/
	DYNAMICMATERIAL_API static void PreventClean(double InDelayFor = MinTimeBeforeClean);
#endif

	DYNAMICMATERIAL_API UDMMaterialComponent();

	/** Checks object flags and IsValid() */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool IsComponentValid() const;

	/** Searches the component for a specific component based on a path. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialComponent* GetComponentByPath(const FString& InPath) const;

	/** Searches the component for a specific component based on a path. */
	DYNAMICMATERIAL_API UDMMaterialComponent* GetComponentByPath(FDMComponentPath& InPath) const;

	template<typename InComponentClass>
	UDMMaterialComponent* GetComponentByPath(FDMComponentPath& InPath) const
	{
		return Cast<InComponentClass>(GetComponentByPath(InPath));
	}

	/** Event that is triggered when this component, or a sub-component, changes to trigger other updates in the model. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

#if WITH_EDITOR
	/** Returns the complete path from the model to this component. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API FString GetComponentPath() const;

	/** Returns the component that owns this component in the model hierarchy. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetParentComponent() const;

	/** Returns the first in the model hierarchy above this component of the given type. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialComponent* GetTypedParent(UClass* InParentClass, bool bInAllowSubclasses) const;

	template<class InParentClass>
	InParentClass* GetTypedParent(bool bInAllowSubclasses) const
	{
		return Cast<InParentClass>(GetTypedParent(InParentClass::StaticClass(), bInAllowSubclasses));
	}

	/* Returns a description of this class/object. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const;

	/** Returns a brush which indicates this component (type) */
	DYNAMICMATERIAL_API virtual FSlateIcon GetComponentIcon() const;

	/** Returns true if this component has been marked dirty. */
	DYNAMICMATERIAL_API bool NeedsClean();

	/** Performs whatever operation is involved in cleaning this component. */
	DYNAMICMATERIAL_API virtual void DoClean();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMComponentLifetimeState GetComponentState() const { return ComponentState; }

	/** Returns true if this component is in the original "Created" state and has not been moved onto Added or Removed. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsComponentCreated() const { return ComponentState == EDMComponentLifetimeState::Created; }

	/** This is a kind of "useless" check, a component has _always_ been created. It's here for completeness. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool HasComponentBeenCreated() const
	{
		return true;
	}

	/** Returns true if this component is in the Added state. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsComponentAdded() const { return ComponentState == EDMComponentLifetimeState::Added; }

	/** Returns true if this component is in the Added or greater state. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool HasComponentBeenAdded() const { return ComponentState >= EDMComponentLifetimeState::Added; }

	/** Returns true if this component is in the Removed state. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsComponentRemoved() const { return ComponentState == EDMComponentLifetimeState::Removed; }

	/** Returns true if this component is in the Removed or greater state. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool HasComponentBeenRemoved() const { return ComponentState >= EDMComponentLifetimeState::Removed; }

	/** Changes the component state to a new one. Should not be used to set it back to Created. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetComponentState(EDMComponentLifetimeState InNewState);

	/** Returns a list of FNames for this component representing editable UPROPERTYs. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<FName>& GetEditableProperties() const { return EditableProperties; }

	/** Returns true the given UPROPERTY name is editable. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual bool IsPropertyVisible(FName InProperty) const { return true; }

	/** Returns the part of the component representing just this object */
	DYNAMICMATERIAL_API virtual FString GetComponentPathComponent() const;

	/** Called to ensure that the object hierarchy is correct. */
	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent);

	DECLARE_MULTICAST_DELEGATE_ThreeParams(
		FOnUpdate,
		UDMMaterialComponent* /* Triggering Component */,
		UDMMaterialComponent* /* Source Component */,
		EDMUpdateType
	)

	/** Delegate called when this component's Update method is called. */
	FOnUpdate::RegistrationType& GetOnUpdate() { return OnUpdate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLifetimeStateChanged, UDMMaterialComponent*, EDMComponentLifetimeState)

	/** Event called when this component's state changes to Added. */
	FOnLifetimeStateChanged::RegistrationType& GetOnAdded() { return OnAdded; }

	/** Event called when this component's state changes to Removed. */
	FOnLifetimeStateChanged::RegistrationType& GetOnRemoved() { return OnRemoved; }

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	//~ End UObject
#endif

protected:
	static const double MinTimeBeforeClean;
	static double MinCleanTime;

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMComponentLifetimeState ComponentState;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, Category = "Material Designer")
	bool bComponentDirty;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, Category = "Material Designer")
	TArray<FName> EditableProperties;

	FOnUpdate OnUpdate;
	FOnLifetimeStateChanged OnAdded;
	FOnLifetimeStateChanged OnRemoved;
#endif

	/** Does some checks to see whether the out is safe to retrieve and retrieves it. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UObject* GetOuterSafe() const;

	/** Searches the component for a specific component based on a path. */
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const;

	/** Marks this component as needed to be cleaned. Preview materials updates, etc. */
	DYNAMICMATERIAL_API void MarkComponentDirty();

#if WITH_EDITOR
	/** Allows this object to modify the child path when generating a path. */
	DYNAMICMATERIAL_API virtual void GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const;

	/** Called when a component state changes to call the enter state functions. */
	DYNAMICMATERIAL_API virtual void OnComponentStateChange(EDMComponentLifetimeState InNewState);

	/** Called when a component enters the Added state. */
	DYNAMICMATERIAL_API virtual void OnComponentAdded();

	/** Called when a component enters the Removed state. */
	DYNAMICMATERIAL_API virtual void OnComponentRemoved();
#endif
};
