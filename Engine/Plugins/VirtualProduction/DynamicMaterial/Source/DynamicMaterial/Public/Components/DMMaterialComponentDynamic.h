// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"
#include "DMMaterialComponentDynamic.generated.h"

class UDynamicMaterialModelDynamic;

/**
 * Base version of a dynamic material component. Links to the original in the parent material model.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Value Instance"))
class UDMMaterialComponentDynamic : public UDMMaterialComponent
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API static const FString ParentValuePathToken;

#if WITH_EDITOR
	using Super::PostEditorDuplicate;
#endif

	DYNAMICMATERIAL_API UDMMaterialComponentDynamic();

	virtual ~UDMMaterialComponentDynamic() override = default;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDynamicMaterialModelDynamic* GetMaterialModelDynamic() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API FName GetParentComponentName() const;

	/** Resolves and returns the component this dynamic one is based on. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialComponent* GeResolvedParentComponent() const;

#if WITH_EDITOR
	/** Copies the properties of this dynamic component back to a non-dynamic counterpart. */
	virtual void CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const PURE_VIRTUAL(UDMMaterialComponentDynamic::CopyPropertiesTo)

	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModelDynamic* InMaterialModelDynamic);

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void BeginDestroy() override;
	//~ End UObject
#endif

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FName ParentComponentName;

	UPROPERTY(Transient)
	TObjectPtr<UDMMaterialComponent> ParentComponent;

	void ResolveParentComponent();

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent
};
