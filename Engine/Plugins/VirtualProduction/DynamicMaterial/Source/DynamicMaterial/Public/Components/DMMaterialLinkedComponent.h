// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialComponent.h"
#include "DMMaterialLinkedComponent.generated.h"

/** A component which links to a specific parent component in the hierarchy instead of its Outer. */
UCLASS(MinimalAPI, Abstract, BlueprintType, meta = (DisplayName = "Material Designer Linked Component"))
class UDMMaterialLinkedComponent : public UDMMaterialComponent
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/** Returns the linked parent component. */
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetParentComponent() const override;

	/** Sets the linked parent component. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetParentComponent(UDMMaterialComponent* InParentComponent);

	/** Sets the parent component to the InParent parameter. */
	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
#endif

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UDMMaterialComponent> ParentComponent = nullptr;
#endif
};
