// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialLinkedComponent.h"
#include "DMMaterialParameter.generated.h"

class UDynamicMaterialModel;

/**
 * A parameter on a Material Designer Material.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Material Designer Parameter"))
class UDMMaterialParameter : public UDMMaterialLinkedComponent
{
	GENERATED_BODY()

	friend class UDynamicMaterialModel;

public:
	DYNAMICMATERIAL_API UDMMaterialParameter();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDynamicMaterialModel* GetMaterialModel() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	FName GetParameterName() const { return ParameterName; }

#if WITH_EDITOR
	/** Changes the parameter name registered with the Model. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void RenameParameter(FName InBaseParameterName);

	//~ Begin UObject
	/** Unregisters the parameter name with the model. */
	DYNAMICMATERIAL_API virtual void BeginDestroy() override;
	//~ End UObject

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent
#endif

protected:
	UPROPERTY(BlueprintReadOnly, VisibleInstanceOnly, Category = "Material Designer")
	FName ParameterName;

private:
	#if WITH_EDITOR
	//~ Begin UDMMaterialComponent
	/** Unregisters the parameter name with the model. */
	DYNAMICMATERIAL_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
#endif
};
