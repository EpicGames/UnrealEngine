// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "Components/DMMaterialLinkedComponent.h"
#include "IDMParameterContainer.h"
#include "Utils/DMJsonUtils.h"

#include "DMDefs.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"

#include "DMMaterialValue.generated.h"

class IPropertyHandle;
class UDMMaterialParameter;
class UDynamicMaterialModel;
class UMaterial;
class UMaterialExpressionParameter;
class UMaterialInstanceDynamic;

#if WITH_EDITOR
class UDMMaterialValueDynamic;
class UDynamicMaterialModelDynamic;
enum class EDMMaterialParameterGroup : uint8;
struct IDMMaterialBuildStateInterface;
#endif

/**
 * A value used in a material. Manages its own parameter.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Value"))
class UDMMaterialValue : public UDMMaterialLinkedComponent, public IDMJsonSerializable, public IDMParameterContainer
{
	GENERATED_BODY()

	friend class UDynamicMaterialModel;
 
public:
	DYNAMICMATERIAL_API static const FString ParameterPathToken;

#if WITH_EDITOR
	/* Name that should be used by all child classes for its value property ("Value"). */
	DYNAMICMATERIAL_API static const FName ValueName;

	/** Creates a new material value and initializes it with the given material model. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIAL_API UDMMaterialValue* CreateMaterialValue(UDynamicMaterialModel* InMaterialModel, const FString& InName, TSubclassOf<UDMMaterialValue> InValueClass, bool bInLocal);
#endif
 
	DYNAMICMATERIAL_API UDMMaterialValue();

	/** Returns the owning material model (this object's Outer). */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDynamicMaterialModel* GetMaterialModel() const;

	/** Returns the type of value as respresented by the possible base type enums. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMValueType GetType() const { return Type; }
 
#if WITH_EDITOR
	/** Uses the Value Definition Library to get the type name of the base enum type. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API FText GetTypeName() const;

	/** Combination of parameter name and type name. May be an automatically generated parameter name. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API FText GetDescription() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual bool IsDefaultValue() const PURE_VIRTUAL(UDMMaterialValue::IsDefaultValue, return false;)

	bool CanResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle) const { return !IsDefaultValue(); }
 
	/**
	 * Subclasses should implement a SetDefaultValue.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void ApplyDefaultValue() PURE_VIRTUAL(UDMMaterialValue::ApplyDefaultValue)

	DYNAMICMATERIAL_API virtual void ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	/**
	 * Resets to the default default value. 0, nullptr, etc.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void ResetDefaultValue() PURE_VIRTUAL(UDMMaterialValue::ResetDefaultValue)
#endif

	/** Returns true is this value is not a "global" value belonging directly to the Model, but belongs to a specific component. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsLocal() const { return bLocal; }

	/** Returns the parameter component managed by this value. Will only exist if a parameter name has been set. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMMaterialParameter* GetParameter() const { return Parameter; }

	/** Returns the specifically set parameter name or an automatically generated parameter name. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API FName GetMaterialParameterName() const;
 
#if WITH_EDITOR
	/**
	 * Sets a specific parameter name, overriding the automatic one if a base name is provided, or resetting back to the original
	 * if a base name is not provider (NAME_None). When setting a specific name a parameter component is created and registered
	 * with the Model.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API bool SetParameterName(FName InBaseName);

	/** Determines the group this value's parameter will be in in the Material. */
	DYNAMICMATERIAL_API EDMMaterialParameterGroup GetParameterGroup() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool GetShouldExposeParameter() const { return bExposeParameter; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetShouldExposeParameter(bool bInExpose);

	virtual void GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const
		PURE_VIRTUAL(UDMMaterialStageSource::GenerateExpressions)
#endif
 
#if WITH_EDITOR 
	/**
	 * Returns the output index (channel WHOLE_CHANNEL) if this expression has pre-masked outputs.
	 * Returns INDEX_NONE if it is not supported.
	 */
	DYNAMICMATERIAL_API virtual int32 GetInnateMaskOutput(int32 OutputChannels) const;

	/** Return true if, when setting the base stage, the same value should be applied to the mask stage. */
	virtual bool IsWholeLayerValue() const { return false; }

	/** Returns true if the property row generator should expose the value property. */
	virtual bool AllowEditValue() const { return true; }
#endif
 
	virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const PURE_VIRTUAL(UDMMaterialStageSource::SetMIDParameter)

#if WITH_EDITOR
	virtual UDMMaterialValueDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic) PURE_VIRTUAL(UDMMaterialStageSource::ToDynamic, return nullptr;)
#endif
 
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
#endif
	//~ End UDMMaterialComponent

	/** Non-editor implementation. */
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
 
#if WITH_EDITOR
	//~ Begin UObject
	DYNAMICMATERIAL_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIAL_API virtual void PostEditUndo() override;
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIAL_API virtual void PostCDOContruct() override;
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	DYNAMICMATERIAL_API virtual void PostEditImport() override;
	//~ End UObject
#endif
 
protected:
	static TMap<EDMValueType, TStrongObjectPtr<UClass>> TypeClasses;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMValueType Type;
 
	/**
	 * True: The value is local to the stage it is used in. 
	 * False: The value is a global value owned directly by the Model.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bLocal;

	/**
	 * The parameter name used to expose this value in a material.
	 * If it isn't provided, an automatic name will be generated.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialParameter> Parameter;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, DuplicateTransient, TextExportTransient, Category = "Material Designer")
	FName CachedParameterName;

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bExposeParameter;
#endif
 
	DYNAMICMATERIAL_API UDMMaterialValue(EDMValueType InType);

	/** Called when the value changes to call this component's and the parent component's Update if possible. */
	DYNAMICMATERIAL_API virtual void OnValueChanged(EDMUpdateType InUpdateType);

#if WITH_EDITOR
	/** Generates an automatic path name based on the component hierarchy */
	DYNAMICMATERIAL_API FName GenerateAutomaticParameterName() const;

	/** Updates the cached parameter name based on the Parameter object or the above method. */
	void UpdateCachedParameterName(bool bInResetName);
#endif
 
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent

	#if WITH_EDITOR
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void OnComponentAdded() override;
	DYNAMICMATERIAL_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
#endif
};
