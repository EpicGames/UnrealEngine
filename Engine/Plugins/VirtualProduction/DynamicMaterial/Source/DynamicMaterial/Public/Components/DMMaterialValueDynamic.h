// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponentDynamic.h"
#include "IDMParameterContainer.h"
#include "Utils/DMJsonUtils.h"

#include "Templates/SharedPointerFwd.h"

#include "DMMaterialValueDynamic.generated.h"

class IPropertyHandle;
class UDMMaterialValue;
class UDynamicMaterialModelDynamic;
class UMaterialInstanceDynamic;

/**
 * A value used inside an instanced material instance. Links to the original value in the parent material.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Value Instance"))
class UDMMaterialValueDynamic : public UDMMaterialComponentDynamic, public IDMJsonSerializable, public IDMParameterContainer
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/** Creates a value dynamic and initializes it with the given Material Designer Model Dyanmic. */
	DYNAMICMATERIAL_API static UDMMaterialValueDynamic* CreateValueDynamic(TSubclassOf<UDMMaterialValueDynamic> InInstanceValueClass, UDynamicMaterialModelDynamic* InMaterialModelDynamic,
		UDMMaterialValue* InParentValue);

	template<class InValueClass>
	static InValueClass* CreateValueDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic, UDMMaterialValue* InParentValue)
	{
		return Cast<InValueClass>(CreateValueDynamic(InValueClass::StaticClass(), InMaterialModelDynamic, InParentValue));
	}
#endif

	/** Resolves (if necessary) and returns the value this dynamic value is based on. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialValue* GetParentValue() const;

#if WITH_EDITOR
	/** Returns true if this value dynamic's value is the same as the parent value's value. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual bool IsDefaultValue() const PURE_VIRTUAL(UDMMaterialValue::IsDefaultValue, return false;)

	bool CanResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle) const { return !IsDefaultValue(); }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void ApplyDefaultValue() PURE_VIRTUAL(UDMMaterialValue::ApplyDefaultValue)

	DYNAMICMATERIAL_API virtual void ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
#endif

	virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const PURE_VIRTUAL(UDMMaterialValueDynamic::SetMIDParameter)

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

	/** Non-editor implementation. */
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

#if WITH_EDITOR
	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostEditUndo() override;
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
#endif

protected:
	/** Call when this value dynamic changes and calls Update. */
	DYNAMICMATERIAL_API virtual void OnValueChanged();

#if WITH_EDITOR
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void OnComponentAdded() override;
	//~ End UDMMaterialComponent
#endif
};
