// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialComponent.h"
#include "Utils/DMJsonUtils.h"
#include "DMRenderTargetRenderer.generated.h"

class UDMMaterialValueRenderTarget;
template <typename T> class TSubclassOf;

/**
 * A component responsible for rendering something to a texture render target (value).
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Render Target Renderer"))
class UDMRenderTargetRenderer : public UDMMaterialComponent, public IDMJsonSerializable
{
	GENERATED_BODY()

public:
	FDelegateHandle EndOfFrameDelegateHandle;

	/** Creates a render of the given class and initializes it. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIAL_API UDMRenderTargetRenderer* CreateRenderTargetRenderer(TSubclassOf<UDMRenderTargetRenderer>
		InRendererClass, UDMMaterialValueRenderTarget* InRenderTargetValue);

	template<typename InRendererClass
		UE_REQUIRES(std::is_base_of_v<UDMRenderTargetRenderer, InRendererClass>)>
	static InRendererClass* CreateRenderTargetRenderer(UDMMaterialValueRenderTarget* InRenderTargetValue)
	{
		return Cast<InRendererClass>(CreateRenderTargetRenderer(InRendererClass::StaticClass(), InRenderTargetValue));
	}

	/** Returns the render target value (the object's Outer). */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialValueRenderTarget* GetRenderTargetValue() const;

	/** Updates the contents of the render target, redrawing and possibly resizing it. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void UpdateRenderTarget();

	/** Updates the contents of the render target, redrawing and possibly resizing it, at the end of the frame. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void AsyncUpdateRenderTarget();

	/** Will trigger the end of frame update if it is currently queued. */
	DYNAMICMATERIAL_API void FlushUpdateRenderTarget();

	/** Returns true is this target is currently being re-rendering. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsUpdating() const { return bUpdating; }

	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

	//~ Begin UDMMaterialComponent
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
#endif
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	//~ End UObject

protected:
	bool bUpdating = false;

	/** Extend this to perform the actual render target rendering in a subclass. */
	virtual void UpdateRenderTarget_Internal() PURE_VIRTUAL(UDMRenderTargetRenderer::UpdateRenderTarget_Internal)
};
