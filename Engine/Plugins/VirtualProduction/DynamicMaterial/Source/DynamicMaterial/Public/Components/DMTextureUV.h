// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialLinkedComponent.h"
#include "IDMParameterContainer.h"

#include "DMDefs.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

#if WITH_EDITOR
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#endif

#include "DMTextureUV.generated.h"

class UDMMaterialParameter;
class UDMTextureUV;
class UDMTextureUVDynamic;
class UDynamicMaterialModel;
class UMaterialInstanceDynamic;

#if WITH_EDITOR
class UDynamicMaterialModelDynamic;
class UMaterialInstanceDynamic;
enum class EDMMaterialParameterGroup : uint8;
#endif

namespace UE::DynamicMaterial::ParamID
{
	// Individual parameters
	constexpr int32 Invalid = -1;
	constexpr int32 PivotX = 0;
	constexpr int32 PivotY = 1;
	constexpr int32 TilingX = 2;
	constexpr int32 TilingY = 3;
	constexpr int32 Rotation = 4;
	constexpr int32 OffsetX = 5;
	constexpr int32 OffsetY = 6;

	// Parameter groups
	constexpr int32 Pivot = PivotX;
	constexpr int32 Tiling = TilingX;
	//constexpr int32 Rotation = 4; // No additional group value needed
	constexpr int32 Offset = OffsetX;
}

/*
 * Represents a Texture UV material function with the following parameters: offset, tiling, pivot and rotation.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Texture UV"))
class UDMTextureUV : public UDMMaterialLinkedComponent, public IDMParameterContainer
{
	GENERATED_BODY()

	friend class SDMStageEdit;
	friend class UDMMaterialStageInputTextureUV;

public:
#if WITH_EDITOR
	DYNAMICMATERIAL_API static const FName NAME_UVSource;
	DYNAMICMATERIAL_API static const FName NAME_bMirrorOnX;
	DYNAMICMATERIAL_API static const FName NAME_bMirrorOnY;
#endif

	DYNAMICMATERIAL_API static const FName NAME_Offset;
	DYNAMICMATERIAL_API static const FName NAME_Pivot;
	DYNAMICMATERIAL_API static const FName NAME_Rotation;
	DYNAMICMATERIAL_API static const FName NAME_Tiling;

	DYNAMICMATERIAL_API static const FString OffsetXPathToken;
	DYNAMICMATERIAL_API static const FString OffsetYPathToken;
	DYNAMICMATERIAL_API static const FString PivotXPathToken;
	DYNAMICMATERIAL_API static const FString PivotYPathToken;
	DYNAMICMATERIAL_API static const FString RotationPathToken;
	DYNAMICMATERIAL_API static const FString TilingXPathToken;
	DYNAMICMATERIAL_API static const FString TilingYPathToken;
	
	static const FGuid GUID;

#if WITH_EDITOR
	// The bool is whether or not the property should be exposed to Sequencer as keyable.
	DYNAMICMATERIAL_API static const TMap<FName, bool> TextureProperties;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIAL_API UDMTextureUV* CreateTextureUV(UObject* InOuter);
#endif

	DYNAMICMATERIAL_API UDMTextureUV();

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Material Designer")
	bool bLinkTiling = true;
#endif

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDynamicMaterialModel* GetMaterialModel() const;

#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMUVSource GetUVSource() const { return UVSource; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetUVSource(EDMUVSource InUVSource);
#endif

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector2D& GetOffset() const { return Offset; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetOffset(const FVector2D& InOffset);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector2D& GetPivot() const { return Pivot; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetPivot(const FVector2D& InPivot);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	float GetRotation() const { return Rotation; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetRotation(float InRotation);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector2D& GetTiling() const { return Tiling; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetTiling(const FVector2D& InTiling);

#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool GetMirrorOnX() const { return bMirrorOnX; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetMirrorOnX(bool bInMirrorOnX);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool GetMirrorOnY() const { return bMirrorOnY; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetMirrorOnY(bool bInMirrorOnY);
#endif

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API TArray<UDMMaterialParameter*> GetParameters() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialParameter* GetMaterialParameter(FName InPropertyName, int32 InComponent) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API FName GetMaterialParameterName(FName InPropertyName, int32 InComponent) const;

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API bool SetMaterialParameterName(FName InPropertyName, int32 InComponent, FName InNewName);

	DYNAMICMATERIAL_API EDMMaterialParameterGroup GetParameterGroup(FName InPropertyName, int32 InComponent) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool GetShouldExposeParameter(FName InPropertyName, int32 InComponent) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetShouldExposeParameter(FName InPropertyName, int32 InComponent, bool bInExpose);
#endif

	DYNAMICMATERIAL_API void SetMIDParameters(UMaterialInstanceDynamic* InMID);

#if WITH_EDITOR
	DYNAMICMATERIAL_API UDMTextureUVDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic);
#endif

	//~ Begin UObject
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	DYNAMICMATERIAL_API virtual void PostEditImport() override;
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIAL_API virtual void PreEditUndo() override;
	DYNAMICMATERIAL_API virtual void PostEditUndo() override;
#endif
	DYNAMICMATERIAL_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject

	//~ Begin UDMMaterialComponent
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
#endif
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

protected:
	static int32 PropertyComponentToParamId(FName InPropertyName, int32 InComponent);

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", ToolTip = "Forces a material rebuild.", NotKeyframeable))
	EDMUVSource UVSource = EDMUVSource::Texture;

	EDMUVSource UVSource_PreUndo = UVSource;
#endif

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetOffset, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", Delta = 0.001))
	FVector2D Offset = FVector2D(0.f, 0.f);

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetPivot, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", ToolTip="Pivot for rotation and tiling.", Delta = 0.001))
	FVector2D Pivot = FVector2D(0.5, 0.5);

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetRotation, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", Delta = 1.0))
	float Rotation = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetTiling, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", AllowPreserveRatio, VectorRatioMode = 3, Delta = 0.001))
	FVector2D Tiling = FVector2D(1.f, 1.f);

	UE_DEPRECATED(5.5, "Changed to tiling.")
	UPROPERTY()
	FVector2D Scale = FVector2D(1.f, 1.f);

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", ToolTip = "Forces a material rebuild.", NotKeyframeable))
	bool bMirrorOnX = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", ToolTip = "Forces a material rebuild.", NotKeyframeable))
	bool bMirrorOnY = false;

	bool bMirrorOnX_PreUndo = bMirrorOnX;
	bool bMirrorOnY_PreUndo = bMirrorOnY;
#endif

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, DuplicateTransient, TextExportTransient, Category = "Material Designer")
	TMap<int32, TObjectPtr<UDMMaterialParameter>> MaterialParameters;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, DuplicateTransient, TextExportTransient, Category = "Material Designer")
	TMap<int32, FName> CachedParameterNames;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TSet<int32> ExposedParameters;

#if WITH_EDITOR
	bool bNeedsPostLoadValueUpdate = false;
	bool bNeedsPostLoadStructureUpdate = false;

	void RemoveParameterNames();
#endif

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

	void OnTextureUVChanged(EDMUpdateType InUpdateType);

#if WITH_EDITOR
	/** Generates an automatic path component based on the property name and component. */
	FName GenerateAutomaticPathComponent(FName InPropertyName, int32 InComponent) const;

	/** Generates an automatic path name based on the component hierarchy */
	FName GenerateAutomaticParameterName(FName InPropertyName, int32 InComponent) const;

	/** Update an individual cached parameter name. */
	void UpdateCachedParameterName(FName InPropertyName, int32 InComponent);

	/** Updates the cached parameter name based on the Parameter object or the above method. */
	void UpdateCachedParameterNames(bool bInResetNames);
#endif

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const override;
	DYNAMICMATERIAL_API virtual void OnComponentAdded() override;
	DYNAMICMATERIAL_API virtual void OnComponentRemoved() override;
#endif
	//~ End UDMMaterialComponent
};
