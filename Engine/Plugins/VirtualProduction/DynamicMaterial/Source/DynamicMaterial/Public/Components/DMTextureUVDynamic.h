// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponentDynamic.h"
#include "IDMParameterContainer.h"

#include "Templates/SharedPointerFwd.h"

#include "DMTextureUVDynamic.generated.h"

class IPropertyHandle;
class UDMTextureUV;
class UDynamicMaterialModelDynamic;
class UMaterialInstanceDynamic;

/**
 * A texture uv used inside a instanced material instance. Links to the original texture uv in the parent material.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Texture UV Instance"))
class UDMTextureUVDynamic : public UDMMaterialComponentDynamic, public IDMParameterContainer
{
	GENERATED_BODY()

	friend class UDynamicMaterialModelDynamic;

public:
#if WITH_EDITOR
	/** Creates a new texture uv dynamic and initializes it with the model dynamic. */
	static DYNAMICMATERIAL_API UDMTextureUVDynamic* CreateTextureUVDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic,
		UDMTextureUV* InParentTextureUV);
#endif

	/** Resolves and returns the parent texture uv from the parent model. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMTextureUV* GetParentTextureUV() const;

	/** Returns the dynamic value for this property. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector2D& GetOffset() const { return Offset; }

	/** Sets the dynamic value for this property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetOffset(const FVector2D& InOffset);

	/** Returns the dynamic value for this property. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector2D& GetPivot() const { return Pivot; }

	/** Sets the dynamic value for this property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetPivot(const FVector2D& InPivot);

	/** Returns the dynamic value for this property. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	float GetRotation() const { return Rotation; }

	/** Sets the dynamic value for this property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetRotation(float InRotation);

	/** Returns the dynamic value for this property. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector2D& GetTiling() const { return Tiling; }

	/** Sets the dynamic value for this property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetTiling(const FVector2D& InTiling);

	/** Updates the given MID with the values of this texture uv. */
	DYNAMICMATERIAL_API void SetMIDParameters(UMaterialInstanceDynamic* InMID) const;

#if WITH_EDITOR
	//~ Begin UDMMaterialComponentDynamic
	virtual void CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const override;
	//~ End UDMMaterialComponentDynamic
#endif

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType = EDMUpdateType::Value) override;
	//~ End UDMMaterialComponent

#if WITH_EDITOR
	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostEditUndo() override;
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
#endif

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetOffset, Setter = SetOffset, BlueprintSetter = SetOffset, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", Delta = 0.001))
	FVector2D Offset = FVector2D(0.f, 0.f);

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetPivot, Setter = SetPivot, BlueprintSetter = SetPivot, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", ToolTip = "Pivot for rotation and tiling.", Delta = 0.001))
	FVector2D Pivot = FVector2D(0.5, 0.5);

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetRotation, Setter = SetRotation, BlueprintSetter = SetRotation, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", Delta = 1.0))
	float Rotation = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetTiling, Setter = SetTiling, BlueprintSetter = SetTiling, Category = "Material Designer|Texture UV",
		meta = (AllowPrivateAccess = "true", AllowPreserveRatio, VectorRatioMode = 3, Delta = 0.001))
	FVector2D Tiling = FVector2D(1.f, 1.f);

	/** Called when any value in this texture uv changes. */
	DYNAMICMATERIAL_API virtual void OnTextureUVChanged();

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

#if WITH_EDITOR
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void OnComponentAdded() override;
	//~ End UDMMaterialComponent
#endif
};
