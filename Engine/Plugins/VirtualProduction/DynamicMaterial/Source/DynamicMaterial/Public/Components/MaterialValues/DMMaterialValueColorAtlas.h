// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialValue.h"
#include "DMMaterialValueColorAtlas.generated.h"

#if WITH_EDITOR
class UCurveLinearColorAtlas;
class UCurveLinearColor;
#endif

/**
 * Component representing a color atlas value. Manages its own parameter.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueColorAtlas : public UDMMaterialValue
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API UDMMaterialValueColorAtlas();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	float GetValue() const { return Value; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetValue(float InValue);

#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	float GetDefaultValue() const { return DefaultValue; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetDefaultValue(float InDefaultValue);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UCurveLinearColorAtlas* GetAtlas() const { return Atlas; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetAtlas(UCurveLinearColorAtlas* InAtlas);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UCurveLinearColor* GetCurve() const { return Curve; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetCurve(UCurveLinearColor* InCurve);

	virtual bool IsWholeLayerValue() const override { return true; }
#endif

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif

	//~ Begin UDMMaterialValue
	DYNAMICMATERIAL_API virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const override;
	DYNAMICMATERIAL_API virtual bool IsDefaultValue() const override;
	DYNAMICMATERIAL_API virtual void ApplyDefaultValue() override;
	DYNAMICMATERIAL_API virtual void ResetDefaultValue() override;
	DYNAMICMATERIAL_API virtual UDMMaterialValueDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic) override;
#endif
	//~ End UDMMaterialValue

	//~ Begin UDMMaterialComponent
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
#endif
	//~ End UDMMaterialComponent

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetValue, Category = "Material Designer",
		meta = (DisplayName = "Alpha", AllowPrivateAccess = "true", ClampMin = 0, UIMin = 0, ClampMax = 1, UIMax = 1))
	float Value;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, BlueprintSetter = SetDefaultValue, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", ClampMin = 0, UIMin = 0, ClampMax = 1, UIMax = 1))
	float DefaultValue;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, BlueprintSetter = SetAtlas, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", NotKeyframeable, NoCreate))
	TObjectPtr<UCurveLinearColorAtlas> Atlas;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, BlueprintSetter = SetCurve, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", NotKeyframeable, NoCreate))
	TObjectPtr<UCurveLinearColor> Curve;
#endif

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer
};
