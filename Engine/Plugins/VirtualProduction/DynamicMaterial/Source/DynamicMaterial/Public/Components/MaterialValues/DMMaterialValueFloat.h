// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "Components/DMMaterialValue.h"
#include "DMMaterialValueFloat.generated.h"

/**
 * Base class for float/scalar values in Materials.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueFloat : public UDMMaterialValue
{
	GENERATED_BODY()
 
public:
	DYNAMICMATERIAL_API UDMMaterialValueFloat();

	/** The value range for any float component. If both values are the same, it is not set. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FFloatInterval& GetValueRange() const { return ValueRange; }

	/** Returns true if a value range has been set. This is true if the min and max aren't the same. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool HasValueRange() const { return ValueRange.Min != ValueRange.Max; }

	/** Sets the range of possible values. Set min and max to the same to disable. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	void SetValueRange(const FFloatInterval& InValueRange) { ValueRange = InValueRange; }
 
protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetValueRange, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", DisplayName = "Value Range"))
	FFloatInterval ValueRange;

	DYNAMICMATERIAL_API UDMMaterialValueFloat(EDMValueType InValueType);
};
