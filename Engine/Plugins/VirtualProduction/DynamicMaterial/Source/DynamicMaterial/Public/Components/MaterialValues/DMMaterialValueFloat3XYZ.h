// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "DMMaterialValueFloat.h"
#include "DMMaterialValueFloat3XYZ.generated.h"
 
/**
 * Component representing an FVector value. Manages its own parameter.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueFloat3XYZ : public UDMMaterialValueFloat
{
	GENERATED_BODY()
 
public: 
	DYNAMICMATERIAL_API UDMMaterialValueFloat3XYZ();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector& GetValue() const { return Value; }
 
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetValue(const FVector& InValue);
 
#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FVector& GetDefaultValue() const { return DefaultValue; }
 
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetDefaultValue(const FVector& InDefaultValue);
#endif

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif
 
	//~ Begin UDMMaterialValue
	DYNAMICMATERIAL_API virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const override;
	DYNAMICMATERIAL_API virtual int32 GetInnateMaskOutput(int32 OutputChannels) const override;
	DYNAMICMATERIAL_API virtual bool IsDefaultValue() const override;
	DYNAMICMATERIAL_API virtual void ApplyDefaultValue() override;
	DYNAMICMATERIAL_API virtual void ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	DYNAMICMATERIAL_API virtual void ResetDefaultValue() override;
	DYNAMICMATERIAL_API virtual UDMMaterialValueDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic) override;
#endif
	// ~End UDMMaterialValue

	//~ Begin UDMMaterialComponent
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
#endif
	//~ End UDMMaterialComponent
 
protected: 
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetValue, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", DisplayName = "Vector3D"))
	FVector Value;
 
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDefaultValue, 
		meta = (AllowPrivateAccess = "true"), Category = "Material Designer")
	FVector DefaultValue;
#endif

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer
};
