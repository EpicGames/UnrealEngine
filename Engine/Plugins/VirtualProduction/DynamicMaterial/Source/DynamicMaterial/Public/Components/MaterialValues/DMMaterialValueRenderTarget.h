// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Delegates/IDelegateInstance.h"
#include "DMMaterialValueRenderTarget.generated.h"

class UDMRenderTargetRenderer;
class UTextureRenderTarget2D;
enum ETextureRenderTargetFormat : int;
struct FLinearColor;

/**
 * Component representing a render target texture value. Manages its own parameter.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueRenderTarget : public UDMMaterialValueTexture
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API static const FString RendererPathToken;

	DYNAMICMATERIAL_API UDMMaterialValueRenderTarget();

	virtual ~UDMMaterialValueRenderTarget() override;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UTextureRenderTarget2D* GetRenderTarget() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FIntPoint& GetTextureSize() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetTextureSize(const FIntPoint& InTextureSize);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API ETextureRenderTargetFormat GetTextureFormat() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetTextureFormat(ETextureRenderTargetFormat InTextureFormat);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FLinearColor& GetClearColor() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetClearColor(const FLinearColor& InClearColor);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMRenderTargetRenderer* GetRenderer() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetRenderer(UDMRenderTargetRenderer* InRenderer);

	/**
	 * Allows outside objects to ensure our render target is valid.
	 * @param bInAsync If true, will create the render target on end of frame.
	 */
	void EnsureRenderTarget(bool bInAsync = false);

	/** Will trigger the end of frame update if it is currently queued or is invalid */
	void FlushCreateRenderTarget();

#if WITH_EDITOR	
	//~ Begin IDMParameterContainer
	DYNAMICMATERIAL_API virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

	//~ Begin UDMMaterialValue
	/** Render target is handled internally. */
	virtual bool AllowEditValue() const override { return false; }
	DYNAMICMATERIAL_API virtual UDMMaterialValueDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic) override;
	//~ End UDMMaterialValue
#endif

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIAL_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
#endif
	//~ End UDMMaterialComponent

	//~ Begin UObject
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	//~ End UObject

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = SetTextureSize, BlueprintSetter = SetTextureSize, Category = "Material Designer|Render Target",
		meta = (AllowPrivateAccess = "true", NotKeyframeable))
	FIntPoint TextureSize;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = SetTextureFormat, BlueprintSetter = SetTextureFormat, Category = "Material Designer|Render Target",
		meta = (AllowPrivateAccess = "true", NotKeyframeable))
	TEnumAsByte<ETextureRenderTargetFormat> TextureFormat;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = SetClearColor, BlueprintSetter = SetClearColor, Category = "Material Designer|Render Target",
		meta = (AllowPrivateAccess = "true", NotKeyframeable))
	FLinearColor ClearColor;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = SetRenderer, BlueprintSetter = SetRenderer, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", NotKeyframeable, NoCreate))
	TObjectPtr<UDMRenderTargetRenderer> Renderer;

	/** Used to asynchronously update the render target. */
	FDelegateHandle EndOfFrameDelegateHandle;

	void AsyncCreateRenderTarget();

	void CreateRenderTarget();

	void UpdateRenderTarget();

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void OnComponentAdded() override;
	DYNAMICMATERIAL_API virtual void OnComponentRemoved() override;
#endif
	//~ End UDMMaterialComponent
};
