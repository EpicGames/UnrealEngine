// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "Components/DMMaterialValue.h"
#include "DMMaterialValueTexture.generated.h"
 
class UTexture;

#if WITH_EDITOR
namespace UE::DynamicMaterial::Private
{
	/** Return true if the texture has an alpha channel. */
	bool DYNAMICMATERIAL_API HasAlpha(UTexture* InTexture);
}

DECLARE_DELEGATE_RetVal(UTexture*, FDMGetDefaultRGBTexture);
#endif

/**
 * Component representing a texture value. Manages its own parameter.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueTexture : public UDMMaterialValue
{
	GENERATED_BODY()
 
public:
	DYNAMICMATERIAL_API UDMMaterialValueTexture();

#if WITH_EDITOR
	DYNAMICMATERIAL_API static FDMGetDefaultRGBTexture GetDefaultRGBTexture;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	static DYNAMICMATERIAL_API UDMMaterialValueTexture* CreateMaterialValueTexture(UObject* InOuter, UTexture* InTexture);
#endif

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UTexture* GetValue() const { return Value; }
 
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetValue(UTexture* InValue);

#if WITH_EDITOR
	/** Return true if the texture value has an alpha channel. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool HasAlpha() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UTexture* GetDefaultValue() const { return DefaultValue; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetDefaultValue(UTexture* InDefaultValue);
#endif // WITH_EDITOR

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif

	//~ Begin UDMMaterialValue
	DYNAMICMATERIAL_API virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void GenerateExpression(const TSharedRef<IDMMaterialBuildStateInterface>& InBuildState) const override;
	DYNAMICMATERIAL_API virtual bool IsDefaultValue() const override;
	DYNAMICMATERIAL_API virtual void ApplyDefaultValue() override;
	DYNAMICMATERIAL_API virtual void ResetDefaultValue() override;
	DYNAMICMATERIAL_API virtual UDMMaterialValueDynamic* ToDynamic(UDynamicMaterialModelDynamic* InMaterialModelDynamic) override;
#endif
	//~ End UDMMaterialValue

	//~ Begin UDMMaterialComponent
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
#endif
	//~ End UDMMaterialComponent

	//~ Begin UObject
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	//~ End UObject

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, NoClear, Getter, Setter, BlueprintSetter = SetValue, Category = "Material Designer",
		meta = (DisplayThumbnail = true, DisplayName = "Texture", AllowPrivateAccess = "true", HighPriority, NoCreate))
	TObjectPtr<UTexture> Value;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDefaultValue, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UTexture> DefaultValue;

	/** Used for PostEditChangeProperty */
	UPROPERTY()
	TObjectPtr<UTexture> OldValue;
#endif

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer
};
