// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Components/DMMaterialValueDynamic.h"
#include "DMMaterialValueFloat1Dynamic.generated.h"
 
/**
 * Link to a UDMMaterialValueFloat1 for Material Designer Model Dynamics.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueFloat1Dynamic : public UDMMaterialValueDynamic
{
	GENERATED_BODY()
 
public:
	DYNAMICMATERIAL_API UDMMaterialValueFloat1Dynamic();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	float GetValue() const { return Value; }
 
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetValue(float InValue);
 
#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetDefaultValue() const;

	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif
 
	//~ Begin UDMMaterialValueDynamic
	DYNAMICMATERIAL_API virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual bool IsDefaultValue() const override;
	DYNAMICMATERIAL_API virtual void ApplyDefaultValue() override;
	DYNAMICMATERIAL_API virtual void CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const override;
#endif
	// ~End UDMMaterialValueDynamic
 
protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetValue, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true"), DisplayName = "Float")
	float Value;

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer
};
