// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialValueDynamic.h"
#include "Math/Color.h"
#include "DMMaterialValueFloat4Dynamic.generated.h"

/**
 * Link to a UDMMaterialValueFloat4 for Material Designer Model Dynamics.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueFloat4Dynamic : public UDMMaterialValueDynamic
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API UDMMaterialValueFloat4Dynamic();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FLinearColor& GetValue() const { return Value; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetValue(const FLinearColor& InValue);

#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FLinearColor& GetDefaultValue() const;

	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif

	//~ Begin UDMMaterialValueDynamic
	DYNAMICMATERIAL_API virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual bool IsDefaultValue() const override;
	DYNAMICMATERIAL_API virtual void ApplyDefaultValue() override;
	DYNAMICMATERIAL_API virtual void ResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
	DYNAMICMATERIAL_API virtual void CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const override;
#endif
	// ~End UDMMaterialValueDynamic

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetValue, Category = "Material Designer",
		meta = (AllowPrivateAccess = "true", DisplayName = "Color"))
	FLinearColor Value;

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer
};
