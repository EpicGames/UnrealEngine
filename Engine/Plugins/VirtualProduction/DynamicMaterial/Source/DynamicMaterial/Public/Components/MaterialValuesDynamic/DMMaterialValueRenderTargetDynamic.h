// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialValueDynamic.h"
#include "DMMaterialValueRenderTargetDynamic.generated.h"

/** Skeleton class because this offers no functionality in a dynamic model. Yet. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialValueRenderTargetDynamic : public UDMMaterialValueDynamic
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API UDMMaterialValueRenderTargetDynamic();

	//~ Begin IDMParameterContainer
	DYNAMICMATERIAL_API virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif

	//~ Begin UDMMaterialValueDynamic
	DYNAMICMATERIAL_API virtual void SetMIDParameter(UMaterialInstanceDynamic* InMID) const override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual bool IsDefaultValue() const override;
	DYNAMICMATERIAL_API virtual void ApplyDefaultValue() override;
	DYNAMICMATERIAL_API virtual void CopyDynamicPropertiesTo(UDMMaterialComponent* InDestinationComponent) const override;
#endif
	// ~End UDMMaterialValueDynamic
};
