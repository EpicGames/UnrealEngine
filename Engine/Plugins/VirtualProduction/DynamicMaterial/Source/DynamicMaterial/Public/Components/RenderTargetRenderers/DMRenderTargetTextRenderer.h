// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/RenderTargetRenderers/DMRenderTargetWidgetRendererBase.h"
#include "Delegates/IDelegateInstance.h"
#include "IDMParameterContainer.h"
#include "Fonts/FontCache.h"
#include "Framework/Text/TextLayout.h"
#include "StructUtils/InstancedStruct.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "DMRenderTargetTextRenderer.generated.h"

class FCanvasTextItem;
class STextBlock;
class SWidget;
class UCanvas;
class UFont;

USTRUCT()
struct FDMTextLine
{
	GENERATED_BODY()

	UPROPERTY()
	FString Line;

	TSharedPtr<STextBlock> Widget;
};

/**
 * Renderer that renders an STextBlock widget and exposes all its parameters/properties.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Render Target Text Renderer"))
class UDMRenderTargetTextRenderer : public UDMRenderTargetWidgetRendererBase, public IDMParameterContainer
{
	GENERATED_BODY()

	friend struct FDMRenderTargetTextRenderer;

public:
	DYNAMICMATERIAL_API UDMRenderTargetTextRenderer();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FSlateFontInfo& GetFontInfo() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetFontInfo(const FSlateFontInfo& InFontInfo);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FText& GetText() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetText(const FText& InText);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FLinearColor& GetTextColor() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetTextColor(const FLinearColor& InColor);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FLinearColor& GetBackgroundColor() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetBackgroundColor(const FLinearColor& InBackgroundColor);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool GetHasHighlight() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetHasHighlight(bool bInHasHighlight);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FLinearColor& GetHighlightColor() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetHighlightColor(const FLinearColor& InHighlightColor);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool GetHasShadow() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetHasShadow(bool bInHasShadow);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FLinearColor& GetShadowColor() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetShadowColor(const FLinearColor& InShadowColor);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const FVector2D& GetShadowOffset() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetShadowOffset(const FVector2D& InShadowOffset);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool GetAutoWrapText() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetAutoWrapText(bool bInAutoWrap);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetWrapTextAt() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetWrapTextAt(float InWrapAt);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API ETextWrappingPolicy GetWrappingPolicy() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetWrappingPolicy(ETextWrappingPolicy InWrappingPolicy);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API ETextJustify::Type GetJustify() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetJustify(ETextJustify::Type InJustify);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API ETextTransformPolicy GetTransformPolicy() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetTransformPolicy(ETextTransformPolicy InTransformPolicy);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API ETextFlowDirection GetFlowDirection() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetFlowDirection(ETextFlowDirection InFlowDirection);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API ETextShapingMethod GetShapingMethod() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetShapingMethod(ETextShapingMethod InShapingMethod);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API const TInstancedStruct<FSlateBrush>& GetStrikeBrush() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetStrikeBrush(const TInstancedStruct<FSlateBrush>& InStrikeBrush);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetLineHeight() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetLineHeight(float InLineHeight);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetPaddingLeft() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetPaddingLeft(float InPadding);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetPaddingRight() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetPaddingRight(float InPadding);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetPaddingTop() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetPaddingTop(float InPadding);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API float GetPaddingBottom() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetPaddingBottom(float InPadding);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool IsOverridingRenderTargetSize() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetOverrideRenderTargetSize(bool bInOverride);

#if WITH_EDITOR
	//~ Begin IDMParameterContainer
	DYNAMICMATERIAL_API virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
#endif

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetFontInfo, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	FSlateFontInfo FontInfo;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetText, Category = "Material Designer|Text",
		meta = (MultiLine, NotKeyframeable, AllowPrivateAccess = "true"))
	FText Text = FText::GetEmpty();

	/** Text broken down into individual lines and their corresponding widgets. */
	UPROPERTY()
	TArray<FDMTextLine> Lines = {};

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetTextColor, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true"))
	FLinearColor TextColor = FLinearColor::White;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetHasHighlight, Setter = SetHasHighlight, BlueprintSetter = SetHasHighlight, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	bool bHasHighlight = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetHighlightColor, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true", EditCondition = "bHasHighlight", EditConditionHides = true))
	FLinearColor HighlightColor = FLinearColor::Black;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetHasShadow, Setter = SetHasShadow, BlueprintSetter = SetHasShadow, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	bool bHasShadow = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetShadowColor, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true", EditCondition = "bHasShadow", EditConditionHides = true))
	FLinearColor ShadowColor = FLinearColor::Black;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetShadowOffset, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true", EditCondition = "bHasShadow", EditConditionHides = true))
	FVector2D ShadowOffset = FVector2D(1.0, 1.0);

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetAutoWrapText, Setter = SetAutoWrapText, BlueprintSetter = SetAutoWrapText, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	bool bAutoWrapText = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetWrapTextAt, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true", EditCondition="bAutoWrapText", EditConditionHides))
	float WrapTextAt = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetWrappingPolicy, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true", EditCondition = "bAutoWrapText", EditConditionHides))
	ETextWrappingPolicy WrappingPolicy = ETextWrappingPolicy::DefaultWrapping;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetJustify, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	TEnumAsByte<ETextJustify::Type> Justify = ETextJustify::Left;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetTransformPolicy, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	ETextTransformPolicy TransformPolicy = ETextTransformPolicy::None;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetFlowDirection, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	ETextFlowDirection FlowDirection = ETextFlowDirection::Auto;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetShapingMethod, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	ETextShapingMethod ShapingMethod = ETextShapingMethod::Auto;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetStrikeBrush, Category = "Material Designer|Text",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	TInstancedStruct<FSlateBrush> StrikeBrush;

	/** Multiplier on the base font height. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetLineHeight, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true", UIMin = 0.1, UIMax = 2.0))
	float LineHeight = 1.f;

	/** Extra space adding beyond the edge of the glyphs. Useful for shadows, glows, etc. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetPaddingLeft, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true"))
	float PaddingLeft = 0.f;

	/** Extra space adding beyond the edge of the glyphs. Useful for shadows, glows, etc. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetPaddingRight, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true"))
	float PaddingRight = 0.f;

	/** Extra space adding beyond the edge of the glyphs. Useful for shadows, glows, etc. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetPaddingTop, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true"))
	float PaddingTop = 0.f;

	/** Extra space adding beyond the edge of the glyphs. Useful for shadows, glows, etc. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetPaddingBottom, Category = "Material Designer|Text",
		meta = (AllowPrivateAccess = "true"))
	float PaddingBottom = 0.f;

	/** When true, will change the size of the render target to fit the text. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = IsOverridingRenderTargetSize, Setter = SetOverrideRenderTargetSize, BlueprintSetter = SetOverrideRenderTargetSize, Category = "Material Designer|Render Target",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	bool bOverrideRenderTargetSize = true;

	bool bRecalculateTextSize = false;

	/** Takes the Text and creates a new TextLines array. */
	void UpdateTextLines();

	/** Creates the widget for an individual line. */
	TSharedRef<STextBlock> CreateTextWidget(const FText& InText) const;

	/** Updates the texture size, if needed, for the given text. */
	void SetCustomTextureSize();

	//~ Begin UDMRenderTargetWidgetRendererBase
	DYNAMICMATERIAL_API virtual void CreateWidgetInstance() override;
	//~ End UDMRenderTargetWidgetRendererBase

	//~ Begin UDMRenderTargetRenderer
	DYNAMICMATERIAL_API virtual void UpdateRenderTarget_Internal() override;
	//~ End UDMRenderTargetRenderer
};
