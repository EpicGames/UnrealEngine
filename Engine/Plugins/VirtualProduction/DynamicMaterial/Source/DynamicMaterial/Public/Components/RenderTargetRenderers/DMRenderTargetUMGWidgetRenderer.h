// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/RenderTargetRenderers/DMRenderTargetWidgetRendererBase.h"
#include "IDMParameterContainer.h"
#include "DMRenderTargetUMGWidgetRenderer.generated.h"

class FWidgetRenderer;
class UWidget;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Render Target Widget Renderer"))
class UDMRenderTargetUMGWidgetRenderer : public UDMRenderTargetWidgetRendererBase, public IDMParameterContainer
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API UDMRenderTargetUMGWidgetRenderer();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	TSubclassOf<UWidget> GetWidgetClass() const { return WidgetClass; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API void SetWidgetClass(TSubclassOf<UWidget> InWidgetClass);

#if WITH_EDITOR
	//~ Begin IDMParameterContainer
	DYNAMICMATERIAL_API virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

	//~ Begin IDMJsonSerializable
	DYNAMICMATERIAL_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIAL_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
#endif

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetWidgetClass, BlueprintSetter = SetWidgetClass, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true", NoCreate))
	TSubclassOf<UWidget> WidgetClass;

	UPROPERTY()
	TObjectPtr<UWidget> WidgetInstance;

	//~ Begin UDMRenderTargetWidgetRendererBase
	DYNAMICMATERIAL_API virtual void CreateWidgetInstance() override;
	//~ End UDMRenderTargetWidgetRendererBase
};
