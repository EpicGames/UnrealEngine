// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMRenderTargetRenderer.h"
#include "DMRenderTargetWidgetRendererBase.generated.h"

class FWidgetRenderer;
class SWidget;
class UWidget;

/**
 * Renderer that renders UWidgets to render targets.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Render Target Widget Renderer"))
class UDMRenderTargetWidgetRendererBase : public UDMRenderTargetRenderer
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API UDMRenderTargetWidgetRendererBase();

#if WITH_EDITOR
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIAL_API virtual FText GetComponentDescription() const override;
	//~ End UDMMaterialComponent
#endif

protected:
	TSharedPtr<SWidget> Widget;
	TSharedPtr<FWidgetRenderer> WidgetRenderer;

	virtual void CreateWidgetInstance() PURE_VIRTUAL(UDMRenderTargetUMGWidgetRenderer::CreateWidgetInstance);

	//~ Begin UDMRenderTargetRenderer
	DYNAMICMATERIAL_API virtual void UpdateRenderTarget_Internal() override;
	//~ End UDMRenderTargetRenderer
};
