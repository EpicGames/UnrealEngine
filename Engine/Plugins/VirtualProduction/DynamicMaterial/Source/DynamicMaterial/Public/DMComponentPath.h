// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"

struct FDMComponentPathSegment;

/** A path such as Name.Component.Component[2].Value */
struct FDMComponentPath
{
	static constexpr TCHAR Separator = '.';
	static constexpr TCHAR ParameterOpen = '[';
	static constexpr TCHAR ParameterClose = ']';

	FDMComponentPath() = delete;
	DYNAMICMATERIAL_API FDMComponentPath(FStringView InPath);
	DYNAMICMATERIAL_API FDMComponentPath(const FString& InPathString);

	DYNAMICMATERIAL_API bool IsLeaf() const;

	/**
	 * Extracts the first component and removes it from the path
	 */
	DYNAMICMATERIAL_API FDMComponentPathSegment GetFirstSegment();

protected:
	FStringView Path;
};

/** Represents a single part of a component path */
struct FDMComponentPathSegment
{
	friend struct FDMComponentPath;

	DYNAMICMATERIAL_API FDMComponentPathSegment(FStringView InToken, FStringView InParameter);

	FStringView GetToken() const { return Token; }

	DYNAMICMATERIAL_API bool HasParameter() const;

	DYNAMICMATERIAL_API bool GetParameter(int32& OutParameter) const;

	DYNAMICMATERIAL_API bool GetParameter(FString& OutParameter) const;

protected:
	FStringView Token;
	FStringView Parameter;
};
