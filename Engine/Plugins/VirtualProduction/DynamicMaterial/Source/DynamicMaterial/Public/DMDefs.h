// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Engine/EngineTypes.h"

#include "DMDefs.generated.h"

class UDMMaterialStage;
class UTexture;

/** Some short names provided so they fit nicely on buttons! */
UENUM(BlueprintType)
enum class EDMMaterialPropertyType : uint8
{
	None = 0,
	BaseColor           UMETA(ShortName = "Base"),
	EmissiveColor       UMETA(ShortName = "Emis"),
	Opacity             UMETA(ShortName = "Opac"),
	OpacityMask         UMETA(ShortName = "Mask"),
	Roughness           UMETA(ShortName = "Rough"),
	Specular            UMETA(ShortName = "Spec"),
	Metallic            UMETA(ShortName = "Metal"),
	Normal              UMETA(ShortName = "Norm"),
	PixelDepthOffset    UMETA(ShortName = "PDO"),
	WorldPositionOffset UMETA(ShortName = "WPO"),
	AmbientOcclusion    UMETA(ShortName = "AO"),
	Anisotropy          UMETA(ShortName = "Aniso"),
	Refraction          UMETA(ShortName = "Refra"),
	Tangent             UMETA(ShortName = "Tang"),
	Custom1,
	Custom2,
	Custom3,
	Custom4,
	Displacement        UMETA(ShortName = "Disp"),
	SubsurfaceColor     UMETA(ShortName = "Surf"),
	SurfaceThickness    UMETA(ShortName = "Thick"),
	Any
};

enum class EDMIterationResult : uint8
{
	Continue,
	Break
};

namespace UE::DynamicMaterial
{
	constexpr int32 RenameFlags = REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch | REN_NonTransactional;
	constexpr EClassFlags InvalidClassFlags = CLASS_Abstract | CLASS_Hidden | CLASS_Deprecated | CLASS_NewerVersionExists;

	DYNAMICMATERIAL_API void ForEachMaterialPropertyType(TFunctionRef<EDMIterationResult(EDMMaterialPropertyType InType)> InCallable,
		EDMMaterialPropertyType InStart = static_cast<EDMMaterialPropertyType>(static_cast<uint8>(EDMMaterialPropertyType::None) + 1),
		EDMMaterialPropertyType InEnd = static_cast<EDMMaterialPropertyType>(static_cast<uint8>(EDMMaterialPropertyType::Any) - 1));
}

UENUM(BlueprintType)
enum class EDMValueType : uint8
{
	VT_None,
	VT_Bool,
	VT_Float1,
	VT_Float2,
	VT_Float3_RPY,
	VT_Float3_RGB,
	VT_Float3_XYZ,
	VT_Float4_RGBA,
	VT_Float_Any,
	VT_Texture,
	VT_ColorAtlas,
	VT_Unused,
	VT_MAX
};

UENUM(BlueprintType)
enum class EDMUpdateType : uint8
{
	Value = 0,
	Structure = 1 << 0,
	AllowParentUpdate = 1 << 1,
	RefreshDetailView = 1 << 2
};
ENUM_CLASS_FLAGS(EDMUpdateType)

UENUM(BlueprintType)
enum class EDMMaterialShadingModel : uint8
{
	Unlit      = EMaterialShadingModel::MSM_Unlit,
	DefaultLit = EMaterialShadingModel::MSM_DefaultLit
};

UENUM(BlueprintType, meta = (DisplayName = "Material Designer UV Source"))
enum class EDMUVSource : uint8
{
	Texture,
	ScreenPosition,
	WorldPosition
};

UENUM(BlueprintType)
enum class EDMLocationType : uint8
{
	World,
	Actor
};

/**
 * An individual component of a connector (e.g. G from RGB.)
 */
USTRUCT(BlueprintType, Category = "Material Designer", meta = (DisplayName = "Material Designer Stage Connector Channel"))
struct FDMMaterialStageConnectorChannel
{
	GENERATED_BODY()

	static constexpr int32 NO_SOURCE = -1;
	static constexpr int32 PREVIOUS_STAGE = 0;
	static constexpr int32 FIRST_STAGE_INPUT = 1;
	static constexpr int32 WHOLE_CHANNEL = 0;
	static constexpr int32 FIRST_CHANNEL = 1;
	static constexpr int32 SECOND_CHANNEL = 2;
	static constexpr int32 THIRD_CHANNEL = 4;
	static constexpr int32 FOURTH_CHANNEL = 8;
	static constexpr int32 TWO_CHANNELS = FIRST_CHANNEL | SECOND_CHANNEL;
	static constexpr int32 THREE_CHANNELS = FIRST_CHANNEL | SECOND_CHANNEL | THIRD_CHANNEL;
	/** Not really needed? Effectively the whole channel! */
	static constexpr int32 FOUR_CHANNELS = FIRST_CHANNEL | SECOND_CHANNEL | THIRD_CHANNEL | FOURTH_CHANNEL;

	/**
	 * The index of the source of this channel
	 * Index 0 is the previous stage, 1+ are the other inputs required by the current stage (e.g. textures, uvs, etc.)
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 SourceIndex = PREVIOUS_STAGE;

	/** When using previous stages, this is the material property the previous stage is using */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMMaterialPropertyType MaterialProperty = EDMMaterialPropertyType::None;

	/** The index of the output connector of the given stage. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 OutputIndex = 0;

	/**
	 * This can be used to break down float2/3/4 into single pieces of data
	 * A value of 0 will be the original output. A bitmask (1,2,4,8) will reference (and combine) the specific channels.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 OutputChannel = WHOLE_CHANNEL;

	bool operator==(const FDMMaterialStageConnectorChannel& InOther) const
	{
		return MaterialProperty == InOther.MaterialProperty
			&& SourceIndex == InOther.SourceIndex
			&& OutputIndex == InOther.OutputIndex
			&& OutputChannel == InOther.OutputChannel;
	}
};

/** Used to prevent Update from running during batch operations. */
struct FDMUpdateGuard
{
	FDMUpdateGuard()
	{
		++GuardCount;
	}

	~FDMUpdateGuard()
	{
		--GuardCount;
	}

	static bool CanUpdate()
	{
		return (GuardCount == 0);
	}

private:
	DYNAMICMATERIAL_API static int32 GuardCount;
};

/** Used to prevent initialization from running during batch operations. */
struct FDMInitializationGuard
{
	static bool IsInitializing()
	{
		return GuardCount > 0;
	}

	FDMInitializationGuard()
	{
		// Used the struct name to make it clear it's a static variable.
		++GuardCount;
	}

	~FDMInitializationGuard()
	{
		if (FDMInitializationGuard::GuardCount > 0)
		{
			--GuardCount;
		}
	}

private:
	DYNAMICMATERIAL_API static uint32 GuardCount;
};
