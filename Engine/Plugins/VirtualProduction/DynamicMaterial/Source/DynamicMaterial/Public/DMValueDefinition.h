// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMDefs.h"
#include "Engine/EngineTypes.h"
#include "Internationalization/Text.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "DMValueDefinition.generated.h"

class UDMMaterialValue;

#if WITH_EDITOR
struct FSlateIcon;
#endif

/**
 * Stores information about basic value types, such as EDMValueType::Float1.
 */
USTRUCT(BlueprintType)
struct FDMValueDefinition
{
	GENERATED_BODY()

	FDMValueDefinition()
		: FDMValueDefinition(EDMValueType::VT_None, 0, FText::GetEmpty(), {}, TSubclassOf<UDMMaterialValue>())
	{		
	}

	FDMValueDefinition(EDMValueType InType, uint8 InFloatCount, const FText& InDisplayName, const TArray<FText>& InChannelNames, 
		TSubclassOf<UDMMaterialValue> InValueClass)
		: Type(InType)
		, FloatCount(InFloatCount)
		, DisplayName(InDisplayName)
		, ChannelNames(InChannelNames)
		, ValueClass(InValueClass)
	{
	}

	FDMValueDefinition(EDMValueType InType, uint8 InFloatCount, FText&& InDisplayName, TArray<FText>&& InChannelNames, 
		TSubclassOf<UDMMaterialValue> InValueClass)
		: Type(InType)
		, FloatCount(InFloatCount)
		, DisplayName(MoveTemp(InDisplayName))
		, ChannelNames(MoveTemp(InChannelNames))
		, ValueClass(InValueClass)
	{
	}

	EDMValueType GetType() const { return Type; }

	/** Will return 0 for non-float/any-float types. */
	uint8 GetFloatCount() const { return FloatCount; }

	const FText& GetDisplayName() const { return DisplayName; }

	const TArray<FText>& GetChannelNames() const { return ChannelNames; }

	/** Returns the base class of this type. */
	TSubclassOf<UDMMaterialValue> GetValueClass() const { return ValueClass; }

	DYNAMICMATERIAL_API bool IsFloatType() const;

	DYNAMICMATERIAL_API bool IsFloat3Type() const;

	/** To be consistent without OutputChannel, 1 is the first channel, not 0. */
	DYNAMICMATERIAL_API const FText& GetChannelName(int32 InChannel) const;

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer")
	EDMValueType Type;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer")
	uint8 FloatCount;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer")
	FText DisplayName;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer")
	TArray<FText> ChannelNames;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer")
	TSubclassOf<UDMMaterialValue> ValueClass;
};

UCLASS(MinimalAPI, BlueprintType)
class UDMValueDefinitionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Returns the array of available Value Types, excluding generics like None or Max. */
	UFUNCTION(BlueprintPure, CallInEditor, Category = "Material Designer")
	static DYNAMICMATERIAL_API const TArray<EDMValueType>& GetValueTypes();

	/** Returns a value definition for the given value type. */
	UFUNCTION(BlueprintPure, CallInEditor, Category = "Material Designer")
	static DYNAMICMATERIAL_API const FDMValueDefinition& GetValueDefinition(EDMValueType InValueType);

	/** Returns whether the given types can be connected together as input/output. */
	UFUNCTION(BlueprintPure, CallInEditor, Category = "Material Designer", meta = (DisplayName = "Are Types Compatible"))
	static bool BP_AreTypesCompatible(EDMValueType A, EDMValueType B, int32 AChannel, int32 BChannel)
	{
		return AreTypesCompatible(A, B, AChannel, BChannel);
	}

	/** Returns whether the given types can be connected together as input/output. */
	DYNAMICMATERIAL_API static bool AreTypesCompatible(EDMValueType InA, EDMValueType InB,
		int32 InAChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		int32 InBChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);

	/** Converts a number of floats into the value type */
	DYNAMICMATERIAL_API static const FDMValueDefinition& GetTypeForFloatCount(uint8 InFloatCount);

	/** Converts a number of floats into the value type */
	UFUNCTION(BlueprintPure, CallInEditor, Category = "Material Designer")
	static DYNAMICMATERIAL_API const FDMValueDefinition& GetTypeForFloatCount(int32 InFloatCount);

#if WITH_EDITOR
	static DYNAMICMATERIAL_API FSlateIcon GetValueIcon(EDMValueType InType);
#endif
};
