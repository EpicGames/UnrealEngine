// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

#if WITH_EDITOR
#include "UObject/ScriptInterface.h"
#endif

#if WITH_EDITOR
class IDynamicMaterialModelEditorOnlyDataInterface;
class UDynamicMaterialModel;
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogDynamicMaterial, Log, All);

#if WITH_EDITOR
DECLARE_DELEGATE_RetVal_OneParam(TScriptInterface<IDynamicMaterialModelEditorOnlyDataInterface>, FDMCreateEditorOnlyDataDelegate, UDynamicMaterialModel*)
#endif

/**
 * Material Designer - Build your own materials in a slimline editor!
 */
class FDynamicMaterialModule : public IModuleInterface
{
public:
	DYNAMICMATERIAL_API static FDynamicMaterialModule& Get();

	/** Returns true if UObjects are currently safe to use. */
	DYNAMICMATERIAL_API static bool AreUObjectsSafe();

	/** Returns true if the material export flag has been enabled. @See DM.ExportMaterials */
	DYNAMICMATERIAL_API static bool IsMaterialExportEnabled();
};
