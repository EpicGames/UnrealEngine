// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/Object.h"

#include "IDMParameterContainer.generated.h"

UINTERFACE(MinimalAPI, BlueprintType)
class UDMParameterContainer : public UInterface
{
	GENERATED_BODY()
};

class IDMParameterContainer
{
	GENERATED_BODY()

public:
	/** Copies the parameter-based values of InFrom to InTo, if possible. */
	static void CopyParametersBetween(UObject* InFrom, UObject* InTo)
	{
		if (!InFrom || !InFrom->Implements<UDMParameterContainer>())
		{
			return;
		}

		if (!InTo)
		{
			return;
		}

		if (InFrom->GetClass() != InTo->GetClass())
		{
			return;
		}

		Execute_CopyParametersFrom(InFrom, InTo);
	}

	void CopyParametersTo(UObject* InOther)
	{
		CopyParametersBetween(Cast<UObject>(this), InOther);
	}

	void CopyParametersFromWrapper(UObject* InOther)
	{
		CopyParametersBetween(InOther, Cast<UObject>(this));
	}

	/** Copies the parameter-based value of the given value to this value, if possible. */
	UFUNCTION(BlueprintNativeEvent, Category = "Material Designer")
	DYNAMICMATERIAL_API void CopyParametersFrom(UObject* InOther);
	virtual void CopyParametersFrom_Implementation(UObject* InOther) {}
};
