// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialInstanceDynamic.h"
#include "DynamicMaterialInstance.generated.h"

class UDynamicMaterialModel;
class UDynamicMaterialModelBase;

#if WITH_EDITOR
class UDMMaterialStageInputTextureUV;
class UDMMaterialValue;
#endif

/** A Material Designer Material with its own integrated Material Designer Model that generates the base Material. */
UCLASS(MinimalAPI, ClassGroup = "Material Designer", DefaultToInstanced, BlueprintType, meta = (DisplayThumbnail = "true"))
class UDynamicMaterialInstance : public UMaterialInstanceDynamic
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API static const FString ModelTypeTag_Material;
	DYNAMICMATERIAL_API static const FString ModelTypeTag_Instance;

#if WITH_EDITOR
	DYNAMICMATERIAL_API static FString GetMaterialTypeTag(const FAssetData& InAssetData);
#endif

	UDynamicMaterialInstance();

	/** Returns the Material Model associated with this Material Designer Material. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDynamicMaterialModelBase* GetMaterialModelBase() const;

	/** Resolves the base Material Model used with this Instance and returns it. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDynamicMaterialModel* GetMaterialModel() const;

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	//~ End UObject

#if WITH_EDITOR
	/** Sets the Material Model used for this Instance. */
	DYNAMICMATERIAL_API void SetMaterialModel(UDynamicMaterialModelBase* InMaterialModel);

	/** Event called when the base material is build. */
	DYNAMICMATERIAL_API void OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModel);

	/** Initialises the base MID object with the current Material Model's generated material.*/
	DYNAMICMATERIAL_API void InitializeMIDPublic();

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	DYNAMICMATERIAL_API virtual void PostEditImport() override;
	DYNAMICMATERIAL_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	//~ End UObject
#endif

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDynamicMaterialModelBase> MaterialModelBase;
};
