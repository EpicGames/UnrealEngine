// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/DynamicMaterialModelBase.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Templates/SubclassOf.h"

#if WITH_EDITOR
#include "UObject/ScriptInterface.h"
#endif

#include "DynamicMaterialModel.generated.h"

class UDMMaterialComponent;
class UDMMaterialParameter;
class UDMMaterialValue;
class UDMMaterialValueFloat1;
class UDMTextureUV;
class UMaterialExpression;
class UMaterialInstanceDynamic;
enum class EDMIterationResult : uint8;
enum class EDMMaterialPropertyType : uint8;
enum class EDMMaterialShadingModel : uint8;
enum class EDMUpdateType : uint8;
struct FDMComponentPath;

#if WITH_EDITORONLY_DATA
class IDynamicMaterialModelEditorOnlyDataInterface;
class UMaterial;
enum EBlendMode : int;
#endif

DECLARE_MULTICAST_DELEGATE_TwoParams(FDMOnValueUpdated, UDynamicMaterialModel*, UDMMaterialValue*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FDMOnTextureUVUpdated, UDynamicMaterialModel*, UDMTextureUV*);

UCLASS(MinimalAPI, ClassGroup = "Material Designer", DefaultToInstanced, BlueprintType, meta = (DisplayThumbnail = "true"))
class UDynamicMaterialModel : public UDynamicMaterialModelBase
{
	GENERATED_BODY()

public:
	/** Tokens used for Components. */
	DYNAMICMATERIAL_API static const FString ValuesPathToken;
	DYNAMICMATERIAL_API static const FString ParametersPathToken;

	/** FNames for global parameters values. */
	DYNAMICMATERIAL_API static const FLazyName GlobalBaseColorValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalBaseColorParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalEmissiveColorValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalEmissiveColorParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalOpacityValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalOpacityParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalMetallicValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalMetallicParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalRoughnessValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalRoughnessParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalNormalValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalNormalParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalSpecularValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalSpecularParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalAnisotropyValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalAnisotropyParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalWorldPositionOffsetValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalWorldPositionOffsetParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalAmbientOcclusionValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalAmbientOcclusionParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalRefractionValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalRefractionParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalTangentValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalTangentParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalPixelDepthOffsetValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalPixelDepthOffsetParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalDisplacementValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalDisplacementParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalSubsurfaceColorValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalSubsurfaceColorParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalSurfaceThicknessValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalSurfaceThicknessParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalOffsetValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalOffsetParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalTilingValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalTilingParameterName;
	DYNAMICMATERIAL_API static const FLazyName GlobalRotationValueName;
	DYNAMICMATERIAL_API static const FLazyName GlobalRotationParameterName;

	UDynamicMaterialModel();

	/** Returns this if this is IsValid() and isn't isn't being destroyed. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool IsModelValid() const;

	/** Returns a specific global parameter value (such as global opacity) for the given material property or nullptr. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialValue* GetGlobalParameterValueForMaterialProperty(EDMMaterialPropertyType InProperty) const;

	/** Returns a Cast version of a specific global value (such as global opacity) or nullptr. */
	template<typename InValueClass>
	InValueClass* GetGlobalParameterValueForMaterialProperty(EDMMaterialPropertyType InProperty) const
	{
		return Cast<InValueClass>(GetGlobalParameterValueForMaterialProperty(InProperty));
	}

	/** Returns a specific global parameter value (such as global opacity) of the given object name (see global parameter FNames) or nullptr. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialValue* GetGlobalParameterValue(FName InName) const;

	/** Returns a Cast version of a specific global value (such as global opacity) or nullptr. */
	template<typename InValueClass>
	InValueClass* GetTypedGlobalParameterValue(FName InName) const
	{
		return Cast<InValueClass>(GetGlobalParameterValue(InName));
	}

	void ForEachGlobalParameter(TFunctionRef<void(UDMMaterialValue* InGlobalParameterValue)> InCallable);

	/** Searches the model for a specific component based on a path. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialComponent* GetComponentByPath(const FString& InPath) const;

	/** Searches the model for a specific component based on a path. */
	DYNAMICMATERIAL_API UDMMaterialComponent* GetComponentByPath(FDMComponentPath& InPath) const;

	/** Searches the model for a specific component based on a path and Casts it to the given type. */
	template<typename InComponentClass>
	InComponentClass* GetComponentByPath(FDMComponentPath& InPath) const
	{
		return Cast<InComponentClass>(GetComponentByPath(InPath));
	}

	/** Returns an array of the (non-global parameter) values used in this Model. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<UDMMaterialValue*>& GetValues() const { return Values; }

	/** Returns a specific (non-global parameter) value of the given object name. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialValue* GetValueByName(FName InName) const;

	/** Returns a set of components that require a runtime reference, such as texture uvs. */
	const TSet<TObjectPtr<UDMMaterialComponent>>& GetRuntimeComponents() const { return RuntimeComponents; }

#if WITH_EDITOR
	friend class UDynamicMaterialModelEditorOnlyData;
	friend class UDynamicMaterialModelFactory;

	/** Returns the editor only data for this model. */
	UFUNCTION(BlueprintPure, Category = "Material Designer", Meta = (DisplayName = "Get Editor Only Data"))
	TScriptInterface<IDynamicMaterialModelEditorOnlyDataInterface> BP_GetEditorOnlyData() const;

	/** Returns the editor only data for this model. */
	DYNAMICMATERIAL_API IDynamicMaterialModelEditorOnlyDataInterface* GetEditorOnlyData() const;

	/** Creates a new value of the given class and returns it. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIAL_API UDMMaterialValue* AddValue(TSubclassOf<UDMMaterialValue> InValueClass);

	/** Adds a reference to a component so they don't get garbage collected at runtime. */
	void AddRuntimeComponentReference(UDMMaterialComponent* InValue);

	/** Removes a reference to a component. */
	void RemoveRuntimeComponentReference(UDMMaterialComponent* InValue);

	/** Removes a value based on its parameter name. */
	void RemoveValueByParameterName(FName InName);

	/**
	 * Returns true if a parameter has with the given name exists on this Model.
	 * Will not include automatically generated component parameter names.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIAL_API bool HasParameterName(FName InParameterName) const;

	/** Creates a new parameter and assigns it a unique name. */
	UDMMaterialParameter* CreateUniqueParameter(FName InBaseName);

	/** Updates the name on an existing parameter. */
	void RenameParameter(UDMMaterialParameter* InParameter, FName InBaseName);

	/** Removes parameter by the name assigned to this parameter object. */
	void FreeParameter(UDMMaterialParameter* InParameter);

	/**
	 * Removes this specific object from the parameter map if the name is in use by a different parameter.
	 * Returns true if, after this call, the object is not in the parameter map.
	 */
	bool ConditionalFreeParameter(UDMMaterialParameter* InParameter);
#endif

	/** Called by the value when it updates. Updates Material Designer Material and triggers the delegate. */
	void OnValueUpdated(UDMMaterialValue* InValue, EDMUpdateType InUpdateType);

	/** Called by the texture uv when it updates. Updates Material Designer Material and triggers the delegate. */
	void OnTextureUVUpdated(UDMTextureUV* InTextureUV);

	/** Returns the value update delegate to (un)subscribe. */
	FDMOnValueUpdated::RegistrationType& GetOnValueUpdateDelegate() { return OnValueUpdateDelegate; }

	/** Returns the texture uv update delegate to (un)subscribe. */
	FDMOnTextureUVUpdated::RegistrationType& GetOnTextureUVUpdateDelegate() { return OnTextureUVUpdateDelegate; }

	//~ Begin UDynamicMaterialModelBase
	virtual UDynamicMaterialModel* ResolveMaterialModel() override { return this; }
	virtual UDynamicMaterialInstance* GetDynamicMaterialInstance() const override { return DynamicMaterialInstance; }
	virtual void SetDynamicMaterialInstance(UDynamicMaterialInstance* InDynamicMaterialInstance) override;
	virtual UMaterial* GetGeneratedMaterial() const override { return DynamicMaterial; }
	DYNAMICMATERIAL_API virtual void ApplyComponents(UMaterialInstanceDynamic* InMID) override;
	//~ End UDynamicMaterialModelBase

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostLoad() override;
#if WITH_EDITOR
	DYNAMICMATERIAL_API virtual void PostEditUndo() override;
	DYNAMICMATERIAL_API virtual void PostEditImport() override;
	DYNAMICMATERIAL_API virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	//~ End UObject

	/** Called to ensure that the object hierarchy is correct. */
	void PostEditorDuplicate();
#endif

	UE_DEPRECATED(5.5, "Added to GlobalParameterValues map.")
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMMaterialValueFloat1* GetGlobalOpacityValue() const;

protected:
	/** Global values */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialValue>> Values;

	/** References to runtime components outered to this model which are not otherwise referenced. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TSet<TObjectPtr<UDMMaterialComponent>> RuntimeComponents;

	/** Map of parameter names to the objects representing that parameter. */
	UPROPERTY(VisibleInstanceOnly, TextExportTransient, Category = "Material Designer")
	TMap<FName, TWeakObjectPtr<UDMMaterialParameter>> ParameterMap;

	/** Material generated by the Model. */
	UPROPERTY(VisibleInstanceOnly, Instanced, BlueprintReadOnly, TextExportTransient, Category = "Material Designer")
	TObjectPtr<UMaterial> DynamicMaterial = nullptr;

	/** Material Designer Material representing the MID for this Model. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, TextExportTransient, Category = "Material Designer")
	TObjectPtr<UDynamicMaterialInstance> DynamicMaterialInstance;

#if WITH_EDITORONLY_DATA
	/** Object holding the editor-only data used by this model. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UObject> EditorOnlyDataSI;
#endif

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalBaseColorParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalEmissiveColorParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalOpacityParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalRoughnessParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalSpecularParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalMetallicParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalNormalParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalPixelDepthOffsetParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalWorldPositionOffsetParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalAmbientOcclusionParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalAnisotropyParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalRefractionParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalTangentParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalDisplacementParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalSubsurfaceColorParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalSurfaceThicknessParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalOffsetParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalTilingParameterValue;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> GlobalRotationParameterValue;

	/** Delegate called when a value is updated. @See GetOnValueUpdateDelegate. */
	FDMOnValueUpdated OnValueUpdateDelegate;

	/** Delegate called when a value is updated. @See GetOnValueUpdateDelegate. */
	FDMOnTextureUVUpdated OnTextureUVUpdateDelegate;

	/**
	 * Attempts to fix global opacity to give it a range of 0-1.
	 */
	void FixGlobalParameterValues();

#if WITH_EDITOR
	/** Checks the current parameters and returns the first parameter name that is not in used (BaseName1, 2, etc, etc.) */
	FName CreateUniqueParameterName(FName InBaseName);

	/** Called to ensure that all components are correctly initialized. Also calls the editor only data version, if applicable. */
	void ReinitComponents();

	/* Makes sure the global parameter values have the correct name. */
	void FixGlobalVars();
#endif

	/**
	 * These properties used to exist on the editor-only subclass of this model.
	 * They now exist on the editor-only component of this model.
	 */

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	TEnumAsByte<EBlendMode> BlendMode;

	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	EDMMaterialShadingModel ShadingModel;

	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	TMap<EDMMaterialPropertyType, TObjectPtr<UObject>> Properties;

	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	TMap<EDMMaterialPropertyType, TObjectPtr<UObject>> PropertySlotMap;

	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	TArray<TObjectPtr<UObject>> Slots;

	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	TArray<TObjectPtr<UMaterialExpression>> Expressions;

	UE_DEPRECATED(5.3, "Moved to editor-only subobject.")
	UPROPERTY()
	bool bCreateMaterialPackage;

	UE_DEPRECATED(5.5, "Moved to GlobalParameterValues map.")
	UPROPERTY()
	TObjectPtr<UDMMaterialValueFloat1> GlobalOpacityValue;
#endif
};
