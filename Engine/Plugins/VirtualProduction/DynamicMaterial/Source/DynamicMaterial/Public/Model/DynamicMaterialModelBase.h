// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DynamicMaterialModelBase.generated.h"

class UDynamicMaterialInstance;
class UDynamicMaterialModel;
class UMaterial;
class UMaterialInstanceDynamic;

/**
 * Base version of a dynamic material model.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Model Base"))
class UDynamicMaterialModelBase : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Returns the Material Designer Model that is the base for this Model.
	 * It will be this object for a Model.
	 * It will be the parent Model for a Model Instance.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual UDynamicMaterialModel* ResolveMaterialModel() PURE_VIRTUAL(UDynamicMaterialModelBase::ResolveMaterialModel, return nullptr;)

	/** Returns the Material Designer Material that contains this Model, if there is one. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual UDynamicMaterialInstance* GetDynamicMaterialInstance() const PURE_VIRTUAL(UDynamicMaterialModelBase::GetDynamicMaterialInstance, return nullptr;)

	/** Sets the Material Designer Material for this Model. */
	virtual void SetDynamicMaterialInstance(UDynamicMaterialInstance* InDynamicMaterialInstance) PURE_VIRTUAL(UDynamicMaterialModelBase::SetDynamicMaterialInstance)

	/** Returns the UMaterial from the resolved Material Model. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual UMaterial* GetGeneratedMaterial() const PURE_VIRTUAL(UDynamicMaterialModelBase::GetGeneratedMaterial, return nullptr;)

	/** Apply all registered components to the given MID. */
	virtual void ApplyComponents(UMaterialInstanceDynamic* InMID) PURE_VIRTUAL(UDynamicMaterialModelBase::ApplyComponents)

#if WITH_EDITOR
	bool IsPreviewModified() const { return bPreviewModified; }

	void MarkPreviewModified() { bPreviewModified = true; }

	void MarkOriginalUpdated() { bPreviewModified = false; }
#endif

protected:
#if WITH_EDITORONLY_DATA
	/** Set to true when the material designer makes a change to a preview model. */
	bool bPreviewModified = false;
#endif
};
