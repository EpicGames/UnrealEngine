// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/DynamicMaterialModelBase.h"

#include "Components/DMMaterialValueDynamic.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"

#include "DynamicMaterialModelDynamic.generated.h"

class FString;
class UDMMaterialComponentDynamic;
class UDMMaterialValueDynamic;
class UDMTextureUVDynamic;
struct FDMComponentPath;

DECLARE_MULTICAST_DELEGATE_TwoParams(FDMOnValueDynamicUpdated, UDynamicMaterialModelDynamic*, UDMMaterialValueDynamic*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FDMOnTextureUVDynamicUpdated, UDynamicMaterialModelDynamic*, UDMTextureUVDynamic*);

/**
 * Represents a MID-like version of a Material Designer Model. Uses dynamic values/texture uvs to link to the original model.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Model Instance"))
class UDynamicMaterialModelDynamic : public UDynamicMaterialModelBase
{
	GENERATED_BODY()

public:
	DYNAMICMATERIAL_API static const FString ParentModelPathToken;
	DYNAMICMATERIAL_API static const FString DynamicComponentsPathToken;

#if WITH_EDITOR
	/**
	 * Create a new Material Designer Model Instance based on a parent Model.
	 * @param InOuter Could be the transient package, an asset package or a Material Designer Material.
	 * @return A new Material Designer Model Instance with its components already initialized.
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Design")
	static DYNAMICMATERIAL_API UDynamicMaterialModelDynamic* Create(UObject* InOuter, UDynamicMaterialModel* InParentModel);
#endif

	UDynamicMaterialModelDynamic();

	/** Resolves and returns the parent model from ParentModelSoft, saving it in the transient ParentModel. */
	UFUNCTION(BlueprintPure, Category = "Motion Design")
	DYNAMICMATERIAL_API UDynamicMaterialModel* GetParentModel() const;

#if WITH_EDITOR
	/** Returns the component with the given name or nullptr. */
	UFUNCTION(BlueprintPure, Category = "Motion Design")
	DYNAMICMATERIAL_API UDMMaterialComponentDynamic* GetComponentDynamic(FName InName);

	/**
	 * Adds the given component. Won't add if a component with the same name already exists.
	 * @return True if the component was added.
	 */
	bool AddComponentDynamic(UDMMaterialComponentDynamic* InValueDynamic);

	/**
	 * Removes the given component. Won't remove if the name doesn't exist or the components don't match.
	 * @return True if the component was removed.
	 */
	bool RemoveComponentDynamic(UDMMaterialComponentDynamic* InValueDynamic);

	/**
	 * Checks the parent model to make sure that all components that exist on the parent model are added
	 * and that all components that no longer exist on the parent model are removed.
	 */
	DYNAMICMATERIAL_API void EnsureComponents();
#endif

	/** Called when a value changes so that the Material Designer Material can be updated and the event broadcast */
	void OnValueUpdated(UDMMaterialValueDynamic* InValueDynamic);

	/** Called when a texture uv changes so that the Material Designer Material can be updated and the event broadcast */
	void OnTextureUVUpdated(UDMTextureUVDynamic* InTextureUVDynamic);

	/** Returns the on value update method so it can (un)subscribed to. */
	FDMOnValueDynamicUpdated::RegistrationType& GetOnValueDynamicUpdateDelegate() { return OnValueDynamicUpdateDelegate; }

	/** Returns the on value update method so it can (un)subscribed to. */
	FDMOnTextureUVDynamicUpdated::RegistrationType& GetOnTextureUVDynamicUpdateDelegate() { return OnTextureUVDynamicUpdateDelegate; }

	/** Finds the component with the given path. */
	UFUNCTION(BlueprintPure, Category = "Motion Design")
	DYNAMICMATERIAL_API UDMMaterialComponent* GetComponentByPath(const FString& InPath) const;

	/** Finds the component with the given path. */
	DYNAMICMATERIAL_API UDMMaterialComponent* GetComponentByPath(FDMComponentPath& InPath) const;

	/** Returns the component map. */
	DYNAMICMATERIAL_API const TMap<FName, TObjectPtr<UDMMaterialComponentDynamic>>& GetComponentMap() const;

#if WITH_EDITOR
	/** Converts this model dynamic to a new model and returns it. */
	DYNAMICMATERIAL_API UDynamicMaterialModel* ToEditable(UObject* InOuter);
#endif

	//~ Begin UDynamicMaterialModelBase
	DYNAMICMATERIAL_API virtual UDynamicMaterialModel* ResolveMaterialModel() override;
	DYNAMICMATERIAL_API virtual UDynamicMaterialInstance* GetDynamicMaterialInstance() const override;
	virtual void SetDynamicMaterialInstance(UDynamicMaterialInstance* InDynamicMaterialInstance) override;
	DYNAMICMATERIAL_API virtual UMaterial* GetGeneratedMaterial() const override;
	DYNAMICMATERIAL_API virtual void ApplyComponents(UMaterialInstanceDynamic* InMID) override;
	//~ End UDynamicMaterialModelBase

	//~ Begin UObject
	DYNAMICMATERIAL_API virtual void PostLoad() override;
	//~ End UObject

protected:
	/** Soft reference to the parent model. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TSoftObjectPtr<UDynamicMaterialModel> ParentModelSoft;

	/** Hard reference to the parent model, loaded when the model is first accessed. */
	UPROPERTY(Transient)
	TObjectPtr<UDynamicMaterialModel> ParentModel;

	/** Map of the dynamic components that reference the parent model. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TMap<FName, TObjectPtr<UDMMaterialComponentDynamic>> DynamicComponents;

	/** Hard reference to the instance, if it exists. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, TextExportTransient, Category = "Material Designer")
	TObjectPtr<UDynamicMaterialInstance> DynamicMaterialInstance;

	/** Delegate called when values update. @See GetOnValueDynamicUpdateDelegate */
	FDMOnValueDynamicUpdated OnValueDynamicUpdateDelegate;

	/** Delegate called when texture uvs update. @See GetOnTextureUVDynamicUpdateDelegate */
	FDMOnTextureUVDynamicUpdated OnTextureUVDynamicUpdateDelegate;

	/**
	 * Loads the parent model from the soft reference, if it is not already loaded.
	 * @return The loaded parent model or nullptr.
	 */
	UDynamicMaterialModel* EnsureParentModel();

#if WITH_EDITOR
	/** Scans the parent Material Model and adds all the components from there as Instance Components. Should not be called twice. */
	void InitComponents();
#endif
};
