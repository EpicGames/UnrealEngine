// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/ContainersFwd.h"
#include "Engine/EngineTypes.h"

class UDMMaterialValue;
class UDynamicMaterialModel;
class UMaterial;
class UMaterialExpression;
enum class EDMMaterialPropertyType : uint8;
struct FExpressionInput;
struct IDMMaterialBuildUtilsInterface;

/**
 * BuildState is a class that stores the current state of a material that is being built.
 * It stores useful lists of UMaterialExpressions relating to various object within the builder, such a Stages or Sources.
 * It is an entirely transient object. It is not meant to be saved outside of the material building processs.
 */
struct IDMMaterialBuildStateInterface
{
	virtual ~IDMMaterialBuildStateInterface() = default;

	/** Returns the dynamic material being built. */
	virtual UMaterial* GetDynamicMaterial() const = 0;

	/** Returns the material model for the material being build. */
	virtual UDynamicMaterialModel* GetMaterialModel() const = 0;

	/** Returns the build utils object for this build state, holding helpful material utilities. */
	virtual IDMMaterialBuildUtilsInterface& GetBuildUtils() const = 0;

	/** Material Values */
	virtual bool HasValue(const UDMMaterialValue* InValue) const = 0;

	virtual const TArray<UMaterialExpression*>& GetValueExpressions(const UDMMaterialValue* InValue) const = 0;

	virtual UMaterialExpression* GetLastValueExpression(const UDMMaterialValue* InValue) const = 0;

	virtual void AddValueExpressions(const UDMMaterialValue* InValue, const TArray<UMaterialExpression*>& InValueExpressions) = 0;

	virtual TArray<const UDMMaterialValue*> GetValues() const = 0;

	virtual const TMap<const UDMMaterialValue*, TArray<UMaterialExpression*>>& GetValueMap() const = 0;

	/** Other Expressions. */
	virtual void AddOtherExpressions(const TArray<UMaterialExpression*>& InOtherExpressions) = 0;

	virtual const TSet<UMaterialExpression*>& GetOtherExpressions() = 0;
};

#endif // WITH_EDITOR
