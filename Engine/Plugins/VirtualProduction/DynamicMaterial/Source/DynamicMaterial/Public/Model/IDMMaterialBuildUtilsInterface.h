// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/ContainersFwd.h"
#include "Model/IDMMaterialBuildStateInterface.h"
#include "Templates/SubclassOf.h"

class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialStageInput;
class UDMMaterialStageSource;
class UDMMaterialStageThroughput;
class UDynamicMaterialModel;
class UMaterial;
class UMaterialExpression;
class UMaterialExpressionAppendVector;
class UMaterialExpressionComponentMask;
struct FDMExpressionInput;
struct FDMMaterialStageConnection;

enum class EDMMaterialParameterGroup : uint8
{
	Property,
	Global,
	NotExposed
};

/**
 * BuildUtils provides some helper functions for creating UMaterialExpressions during the material build process.
 */
struct IDMMaterialBuildUtilsInterface
{
	virtual ~IDMMaterialBuildUtilsInterface() = default;

	/**
	 * Creates an expression of the given class and adds it to the material.
	 * @param InComment Comment added to the material node.
	 * @param InAsset Object assigned to the node, such as a texture.
	 */
	virtual UMaterialExpression* CreateExpression(TSubclassOf<UMaterialExpression> InExpressionClass, const FString& InComment, 
		UObject* InAsset = nullptr) const = 0;

	/**
	 * Creates an expression of the given class as a parameter and adds it to the material.
	 * @param InParameterName The name of the parameter.
	 * @param InComment Comment added to the material node.
	 * @param InAsset Object assigned to the node, such as a texture.
	 */
	virtual UMaterialExpression* CreateExpressionParameter(TSubclassOf<UMaterialExpression> InExpressionClass, FName InParameterName,
		EDMMaterialParameterGroup InParameterGroup, const FString& InComment, UObject* InAsset = nullptr) const = 0;

	template<typename InExpressionClass
		UE_REQUIRES(std::derived_from<InExpressionClass, UMaterialExpression>)>
	InExpressionClass* CreateExpression(const FString& InComment, UObject* InAsset = nullptr) const
	{
		return Cast<InExpressionClass>(CreateExpression(TSubclassOf<UMaterialExpression>(InExpressionClass::StaticClass()), InComment, InAsset));
	}

	template<typename InExpressionClass
		UE_REQUIRES(std::derived_from<InExpressionClass, UMaterialExpression>)>
	InExpressionClass* CreateExpressionParameter(FName InParameterName, EDMMaterialParameterGroup InParameterGroup, const FString& InComment, UObject* InAsset = nullptr) const
	{
		return Cast<InExpressionClass>(CreateExpressionParameter(TSubclassOf<UMaterialExpression>(InExpressionClass::StaticClass()), 
			InParameterName, InParameterGroup, InComment, InAsset));
	}

	/** Creates a set of expressions merging all the inputs for each channel into a single output */
	virtual TArray<UMaterialExpression*> CreateExpressionInputs(const TArray<FDMMaterialStageConnection>& InInputConnectionMap,
		int32 InStageSourceInputIdx, const TArray<UDMMaterialStageInput*>& InStageInputs, int32& OutOutputIndex,
		int32& OutOutputChannel) const = 0;

	/** Creates a set of expressions that display this material stage input. */
	virtual TArray<UMaterialExpression*> CreateExpressionInput(UDMMaterialStageInput* InInput) const = 0;

	/** Searches the outputs of an expression to see if there is an appropriate output to match a requested channel mask. Returns INDEX_NONE on failure. */
	virtual int32 FindOutputForBitmask(UMaterialExpression* InExpression, int32 InOutputChannels) const = 0;

	/** Creates a set of expressions merging all the inputs for each channel into a single output */
	virtual UMaterialExpressionComponentMask* CreateExpressionBitMask(UMaterialExpression* InExpression, int32 InOutputIndex, 
		int32 InOutputChannels) const = 0;

	/** Creates an append expression, joining the output of 2 other expressions into a single vector. */
	virtual UMaterialExpressionAppendVector* CreateExpressionAppend(UMaterialExpression* InExpressionA, int32 InOutputIndexA,
		UMaterialExpression* InExpressionB, int32 InOutputIndexB) const = 0;

	/** Updates a preview material, assigning the output of the "last expression" to an appropriate material property. */
	virtual void UpdatePreviewMaterial(UMaterialExpression* InLastExpression, int32 InOutputIndex, int32 InOutputChannel, 
		int32 InSize) const = 0;
};
#endif