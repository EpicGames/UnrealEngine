// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SharedPointer.h"

#if WITH_EDITOR
#include "Model/IDMMaterialBuildStateInterface.h"
#endif

#include "IDynamicMaterialModelEditorOnlyDataInterface.generated.h"

class UDMMaterialComponent;
class UDMMaterialValue;
class UDMTextureUV;
class UDynamicMaterialModel;
class UMaterial;
enum class EDMUpdateType : uint8;
struct FDMComponentPathSegment;
struct FDMComponentPath;

UENUM(BlueprintType)
enum class EDMBuildRequestType : uint8
{
	/** Compile the material immediately. */
	Immediate,

	/** Will try to add to the build queue or fallback to immediate compile. */
	Async,

	/**
	 * If bAutomaticallyCompilePreviewMaterial is false, will just mark the material as changed,
	 * but not actually try to compile. It will fall back to Async if automatic compile is true.
	 */
	Preview
};

/** Interface for the editor-only data so that editor-only parts of runtime components can interact with editor-only features. */
UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UDynamicMaterialModelEditorOnlyDataInterface : public UInterface
{
	GENERATED_BODY()
};

class IDynamicMaterialModelEditorOnlyDataInterface
{
	GENERATED_BODY()

	friend class UDynamicMaterialModel;

public:
	/** Called to ensure that the object hierarchy is correct. */
	virtual void PostEditorDuplicate() = 0;

	/** Called when a value is updated. */
	virtual void OnValueUpdated(UDMMaterialValue* InValue, EDMUpdateType InUpdateType) = 0;

	/** Called when a value is added or removed. */
	virtual void OnValueListUpdate() = 0;

	/** Called when a texture uv is updated. */
	virtual void OnTextureUVUpdated(UDMTextureUV* InTextureUV) = 0;

#if WITH_EDITOR
	/** Called when the model needs to have the material rebuild. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void RequestMaterialBuild(EDMBuildRequestType InRequestType = EDMBuildRequestType::Preview) PURE_VIRTUAL(UDynamicMaterialModelEditorOnlyDataInterface::RequestMaterialBuild)

	/** Called to create the build state for building materials. */
	virtual TSharedRef<IDMMaterialBuildStateInterface> CreateBuildStateInterface(UMaterial* InMaterialToBuild) const = 0;

	/** Sets the component on a material property (such as a global parameter). */
	virtual void SetPropertyComponent(EDMMaterialPropertyType InPropertyType, FName InComponentName, UDMMaterialComponent* InComponent) = 0;
#endif

	/** Searches the model editor only data for a specific component based on a path. */
	virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath) const = 0;

	/** Searches the model editor only data for a specific component based on a path. */
	virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const = 0;

protected:
	/** Called to ensure that all editor only data is correctly initialized. */
	virtual void ReinitComponents() = 0;
};