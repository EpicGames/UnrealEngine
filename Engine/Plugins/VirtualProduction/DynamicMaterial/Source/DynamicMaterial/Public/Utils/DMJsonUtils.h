// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/BaseStructureProvider.h"
#include "Containers/StringFwd.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/Interface.h"
#include "Dom/JsonObject.h"
#include "DMJsonUtils.generated.h"

class FJsonValue;
class UClass;
class UTexture;

/** Offers custom serialization options for class. */
UINTERFACE(MinimalAPI, Blueprintable, BlueprintType)
class UDMJsonSerializable : public UInterface
{
	GENERATED_BODY()
};

class IDMJsonSerializable
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<FJsonValue> JsonSerialize() const PURE_VIRTUAL(UDMMaterialValue::JsonSerialize, return nullptr;)
	virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) PURE_VIRTUAL(UDMMaterialValue::JsonDeserialize, return false;)
};

#if WITH_EDITOR
/** Wrapper library to allow quicker (de)serialization to JSON. */
class FDMJsonUtils
{
public:
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(bool bInValue);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const FString& InString);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const FText& InText);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const FName& InName);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const UClass* InClass);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const UScriptStruct* InScriptStruct, const void* InData);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const UObject* InObject);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const FObjectPtr& InObject);
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> Serialize(const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& InMap);

	template<typename InArithmeticType
		UE_REQUIRES(TIsArithmetic<InArithmeticType>::Value)>
	static TSharedPtr<FJsonValue> Serialize(InArithmeticType InValue)
	{
		return SerializeNumber(static_cast<double>(InValue));
	}

	template<typename InEnumType
		UE_REQUIRES(TIsEnum<InEnumType>::Value)>
	static TSharedPtr<FJsonValue> Serialize(InEnumType InValue)
	{
		return SerializeNumber(static_cast<double>(InValue));
	}

	template<typename InStructType
		UE_REQUIRES(TModels_V<CBaseStructureProvider, InStructType>)>
	static TSharedPtr<FJsonValue> Serialize(const InStructType& InStruct)
	{
		return Serialize(TBaseStructure<InStructType>::Get(), static_cast<const void*>(&InStruct));
	}

	template<typename InClassType
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassType>)>
	static TSharedPtr<FJsonValue> Serialize(const TSubclassOf<InClassType>& InClass)
	{
		return Serialize(InClass.Get());
	}

	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, bool& bOutValue);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FString& OutString);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FText& OutText);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FName& OutName);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, UClass*& OutClass);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, const UScriptStruct* InScriptStruct, void* OutData);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, const UEnum* InEnum, int64& OutValue);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, UObject*& OutObject, UObject* InOuter = nullptr);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, FObjectPtr& OutObject, UObject* InOuter = nullptr);
	DYNAMICMATERIAL_API static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& OutMap);

	template<typename InArithmeticType
		UE_REQUIRES(TIsArithmetic<InArithmeticType>::Value)>
	static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, InArithmeticType& OutValue)
	{
		double Value;

		if (DeserializeNumber(InJsonValue, Value))
		{
			if constexpr (TIsIntegral<InArithmeticType>::Value)
			{
				OutValue = FMath::RoundToInt(Value);
			}
			else
			{
				OutValue = static_cast<InArithmeticType>(Value);
			}
			return true;
		}

		return false;
	}

	template<typename InEnumType
		UE_REQUIRES(TIsEnum<InEnumType>::Value)>
	static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, InEnumType& OutValue)
	{
		int64 Value;

		if (Deserialize(InJsonValue, StaticEnum<InEnumType>(), Value))
		{
			OutValue = static_cast<InEnumType>(Value);
			return true;
		}

		return false;
	}

	template<typename InStructType
		UE_REQUIRES(TModels_V<CBaseStructureProvider, InStructType>)>
	static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, InStructType& OutStruct)
	{
		return Deserialize(InJsonValue, TBaseStructure<InStructType>::Get(), static_cast<void*>(&OutStruct));
	}

	template<typename InClassType
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassType>)>
	static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, InClassType*& OutObject, UObject* InOuter = nullptr)
	{
		UObject* Object;

		if (Deserialize(InJsonValue, Object, InOuter))
		{
			if (!Object || Object->IsA<InClassType>())
			{
				OutObject = Cast<InClassType>(Object);
				return true;
			}
		}

		return false;
	}

	template<typename InClassType
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassType>)>
	static bool Deserialize(const TSharedPtr<FJsonValue>& InJsonValue, TSubclassOf<InClassType>& OutSubclass)
	{
		UClass* Class = nullptr;

		if (Deserialize(InJsonValue, Class))
		{
			if (!Class || Class->IsChildOf(InClassType::StaticClass()))
			{
				OutSubclass = Class;
				return true;
			}
		}

		return false;
	}

private:
	DYNAMICMATERIAL_API static TSharedPtr<FJsonValue> SerializeNumber(double InNumber);
	DYNAMICMATERIAL_API static bool DeserializeNumber(const TSharedPtr<FJsonValue>& InJsonValue, double& OutNumber);
};
#endif
