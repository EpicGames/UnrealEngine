// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/UnrealString.h"

enum class EDMMaterialPropertyType : uint8;
enum class EDMTextureSetMaterialProperty : uint8;
enum EMaterialProperty : int;

/** General utilities for use with the Material Designer. */
struct FDMUtils
{
#if WITH_EDITOR
	/** Designed to be used with preprocessor macros. */
	DYNAMICMATERIAL_API static FString CreateNodeComment(const ANSICHAR* InFile, int InLine, const ANSICHAR* InFunction, const FString* InComment = nullptr);
#endif

	/** Converts EDMMaterialPropertyType to EDMTextureSetMaterialProperty. */
	DYNAMICMATERIAL_API static EDMTextureSetMaterialProperty MaterialPropertyTypeToTextureSetMaterialProperty(EDMMaterialPropertyType InPropertyType);

	/** Converts EDMTextureSetMaterialProperty to EDMMaterialPropertyType. */
	DYNAMICMATERIAL_API static EDMMaterialPropertyType TextureSetMaterialPropertyToMaterialPropertyType(EDMTextureSetMaterialProperty InPropertyType);

	/** Converts EDMMaterialPropertyType to EMaterialProperty. */
	DYNAMICMATERIAL_API static EMaterialProperty MaterialPropertyTypeToMaterialProperty(EDMMaterialPropertyType InPropertyType);

	/** Converts EMaterialProperty to EDMMaterialPropertyType. */
	DYNAMICMATERIAL_API static EDMMaterialPropertyType MaterialPropertyToMaterialPropertyType(EMaterialProperty InPropertyType);
};

#if WITH_EDITOR
#define UE_DM_NodeComment_Default FDMUtils::CreateNodeComment(__FILE__, __LINE__, __FUNCTION__)
#define UE_DM_NodeComment(Comment) FDMUtils::CreateNodeComment(__FILE__, __LINE__, __FUNCTION__, &Comment)
#endif
