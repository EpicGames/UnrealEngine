// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterialEditor : ModuleRules
{
	public DynamicMaterialEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DynamicMaterial",
				"Engine",
				"MaterialEditor"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedPreviewScene",
				"AppFramework",
				"ApplicationCore",
				"AssetDefinition",
				"ContentBrowser",
				"ContentBrowserData",
				"CustomDetailsView",
				"DeveloperSettings",
				"DynamicMaterialShaders",
				"DynamicMaterialTextureSet",
				"DynamicMaterialTextureSetEditor",
				"EditorSubsystem",
				"EditorWidgets",
				"InputCore",
				"Json",
				"JsonUtilities",
				"MaterialEditor",
				"Projects",
				"PropertyEditor",
				"RenderCore",
				"Renderer",
				"RHI",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TypedElementRuntime",
				"UMG",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);

		ShortName = "DynMatEd";
	}
}
