// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialStage.h"
#include "DMComponentPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialEffect)

#define LOCTEXT_NAMESPACE "DMMaterialEffect"

EDMMaterialEffectTarget UDMMaterialEffect::StageTypeToEffectType(EDMMaterialLayerStage InStageType)
{
	if (EnumHasAnyFlags(InStageType, EDMMaterialLayerStage::Base))
	{
		return EDMMaterialEffectTarget::BaseStage;
	}

	if (EnumHasAnyFlags(InStageType, EDMMaterialLayerStage::Mask))
	{
		return EDMMaterialEffectTarget::MaskStage;
	}

	return EDMMaterialEffectTarget::None;
}

UDMMaterialEffect* UDMMaterialEffect::CreateEffect(UDMMaterialEffectStack* InEffectStack, TSubclassOf<UDMMaterialEffect> InEffectClass)
{
	check(InEffectStack);
	check(InEffectClass.Get() != UDMMaterialEffect::StaticClass());

	return NewObject<UDMMaterialEffect>(InEffectStack, InEffectClass, NAME_None, RF_Transactional);
}

UDMMaterialEffect::UDMMaterialEffect()
	: EffectTarget(EDMMaterialEffectTarget::None)
	, bEnabled(true)
{
}

UDMMaterialEffectStack* UDMMaterialEffect::GetEffectStack() const
{
	return Cast<UDMMaterialEffectStack>(GetOuterSafe());
}

int32 UDMMaterialEffect::FindIndex() const
{
	if (UDMMaterialEffectStack* EffectStack = GetEffectStack())
	{
		return EffectStack->GetEffects().IndexOfByPredicate(
			[this](const TObjectPtr<UDMMaterialEffect>& InElement)
			{
				return this == InElement;
			}
		);
	}

	return INDEX_NONE;
}

bool UDMMaterialEffect::IsEnabled() const
{
	return bEnabled;
}

bool UDMMaterialEffect::SetEnabled(bool bInIsEnabled)
{
	if (bEnabled == bInIsEnabled)
	{
		return false;
	}

	bEnabled = bInIsEnabled;

	Update(this, EDMUpdateType::Structure);

	return true;
}

EDMMaterialEffectTarget UDMMaterialEffect::GetEffectTarget() const
{
	return EffectTarget;
}

FText UDMMaterialEffect::GetEffectName() const
{
	return GetComponentDescription();
}

UDMMaterialComponent* UDMMaterialEffect::GetParentComponent() const
{
	return GetEffectStack();
}

FString UDMMaterialEffect::GetComponentPathComponent() const
{
	return FString::Printf(
		TEXT("%s%c%i%c"),
		*UDMMaterialEffectStack::EffectsPathToken,
		FDMComponentPath::ParameterOpen,
		FindIndex(),
		FDMComponentPath::ParameterClose
	);
}

FText UDMMaterialEffect::GetComponentDescription() const
{
	static const FText Description = LOCTEXT("Effect", "Effect");
	return Description;
}

void UDMMaterialEffect::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	const int32 Index = FindIndex();

	if (Index != INDEX_NONE && EffectStack->GetEffects().IsValidIndex(Index + 1))
	{
		EffectStack->GetEffects()[Index + 1]->Update(InSource, InUpdateType);
	}
	else
	{
		EffectStack->Update(InSource, InUpdateType);
	}

	Super::Update(InSource, InUpdateType);

	if (UDMMaterialLayerObject* Layer = EffectStack->GetLayer())
	{
		if (EffectTarget == EDMMaterialEffectTarget::BaseStage || EffectTarget == EDMMaterialEffectTarget::TextureUV)
		{
			if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base))
			{
				BaseStage->Update(InSource, InUpdateType);
			}
		}

		if (EffectTarget == EDMMaterialEffectTarget::MaskStage || EffectTarget == EDMMaterialEffectTarget::TextureUV)
		{
			if (UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask))
			{
				MaskStage->Update(InSource, InUpdateType);
			}
		}
	}
}

void UDMMaterialEffect::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);	

	if (GetOuter() != InParent)
	{
		Rename(nullptr, InParent, UE::DynamicMaterial::RenameFlags);
	}
}

void UDMMaterialEffect::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = GetEffectStack();

	if (!EffectStack)
	{
		SetComponentState(EDMComponentLifetimeState::Removed);
		return;
	}

	MarkComponentDirty();

	Update(this, EDMUpdateType::Structure);
}

#undef LOCTEXT_NAMESPACE
