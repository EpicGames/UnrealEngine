// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialEffectFunction.h"

#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DMComponentPath.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "DynamicMaterialEditorSettings.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"
#include "MaterialValueType.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialFunctionFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialEffectFunction)

#define LOCTEXT_NAMESPACE "DMMaterialEffectFunction"

const FString UDMMaterialEffectFunction::InputsPathToken = TEXT("Inputs");

UDMMaterialEffectFunction::UDMMaterialEffectFunction()
{
}

UMaterialFunctionInterface* UDMMaterialEffectFunction::GetMaterialFunction() const
{
	return MaterialFunctionPtr;
}

bool UDMMaterialEffectFunction::SetMaterialFunction(UMaterialFunctionInterface* InFunction)
{
	if (MaterialFunctionPtr == InFunction)
	{
		return false;
	}

	MaterialFunctionPtr = InFunction;

	OnMaterialFunctionChanged();

	Update(this, EDMUpdateType::Structure);

	return true;
}

UDMMaterialValue* UDMMaterialEffectFunction::GetInputValue(int32 InIndex) const
{
	if (InputValues.IsValidIndex(InIndex))
	{
		return InputValues[InIndex];
	}

	return nullptr;
}

TArray<UDMMaterialValue*> UDMMaterialEffectFunction::BP_GetInputValues() const
{
	TArray<UDMMaterialValue*> Values;
	Values.Reserve(InputValues.Num());

	for (const TObjectPtr<UDMMaterialValue>& Value : InputValues)
	{
		Values.Add(Value);
	}

	return Values;
}

const TArray<TObjectPtr<UDMMaterialValue>>& UDMMaterialEffectFunction::GetInputValues() const
{
	return InputValues;
}

TSharedPtr<FJsonValue> UDMMaterialEffectFunction::JsonSerialize() const
{
	TArray<TSharedPtr<FJsonValue>> ValueArray;

	if (MaterialFunctionPtr)
	{
		for (int32 InputIndex = 0; InputIndex < InputValues.Num(); ++InputIndex)
		{
			// Index 0 is always null
			if (InputIndex == 0)
			{
				ValueArray.Add(MakeShared<FJsonValueNull>());
			}
			else if (InputValues[InputIndex])
			{
				ValueArray.Add(InputValues[InputIndex]->JsonSerialize());
			}
			else
			{
				UE::DynamicMaterialEditor::Private::LogError(TEXT("Null input found when serializing material effect function."), true, this);
			}
		}
	}

	return FDMJsonUtils::Serialize({
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, bEnabled)), FDMJsonUtils::Serialize(bEnabled)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MaterialFunctionPtr)), FDMJsonUtils::Serialize(MaterialFunctionPtr)},
		{FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, InputValues)), MakeShared<FJsonValueArray>(ValueArray)},
	});
}

bool UDMMaterialEffectFunction::JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue)
{
	TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>> Data;

	if (!FDMJsonUtils::Deserialize(InJsonValue, Data))
	{
		return false;
	}

	bool bSuccess = false;

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, bEnabled))))
	{
		bool bEnabledJson = false;

		if (FDMJsonUtils::Deserialize(*JsonValue, bEnabledJson))
		{
			const FDMUpdateGuard Guard;
			SetEnabled(bEnabledJson);
			bSuccess = true;
		}
	}

	if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MaterialFunctionPtr))))
	{
		UMaterialFunctionInterface* MaterialFunctionJson = nullptr;

		if (FDMJsonUtils::Deserialize(*JsonValue, MaterialFunctionJson))
		{
			const FDMUpdateGuard Guard;
			SetMaterialFunction(MaterialFunctionJson);
			bSuccess = true;
		}
	}

	// If there's no effect function, there's no input values
	if (MaterialFunctionPtr)
	{
		if (const TSharedPtr<FJsonValue>* JsonValue = Data.Find(FJsonObject::FStringType(GET_MEMBER_NAME_STRING_CHECKED(ThisClass, InputValues))))
		{
			const TArray<TSharedPtr<FJsonValue>>* InputValuesJson = nullptr;

			if ((*JsonValue)->TryGetArray(InputValuesJson))
			{
				if (InputValuesJson->Num() != InputValues.Num())
				{
					UE::DynamicMaterialEditor::Private::LogError(TEXT("Mismatched input value count deserializing effect function."), true, this);
					bSuccess = false;
				}
				else
				{
					// Index 0 is always nullptr, ignore it.
					for (int32 InputIndex = 1; InputIndex < InputValues.Num(); ++InputIndex)
					{
						if (!InputValues[InputIndex]->JsonDeserialize((*InputValuesJson)[InputIndex]))
						{
							UE::DynamicMaterialEditor::Private::LogError(TEXT("Unable to deserialize input value while deserializing material effect function."), true, this);
							bSuccess = false;
						}
					}
				}
			}
		}
	}

	if (bSuccess)
	{
		Update(this, EDMUpdateType::Structure);
	}

	return bSuccess;
}

FText UDMMaterialEffectFunction::GetEffectName() const
{
	if (UMaterialFunctionInterface* MaterialFunction = MaterialFunctionPtr.Get())
	{
		const FString Caption = MaterialFunction->GetUserExposedCaption();

		if (!Caption.IsEmpty())
		{
			return FText::FromString(Caption);
		}
	}

	static const FText Name = LOCTEXT("EffectFunction", "Effect Function");
	return Name;
}

FText UDMMaterialEffectFunction::GetEffectDescription() const
{
	if (UMaterialFunctionInterface* MaterialFunction = MaterialFunctionPtr.Get())
	{
		const FString& Description = MaterialFunction->GetDescription();

		if (!Description.IsEmpty())
		{
			return FText::FromString(Description);
		}
	}

	return FText::GetEmpty();
}

bool UDMMaterialEffectFunction::IsCompatibleWith(UDMMaterialEffect* InEffect) const
{
	if (!MaterialFunctionPtr)
	{
		return false;
	}

	UDMMaterialEffectFunction* EffectFunction = Cast<UDMMaterialEffectFunction>(InEffect);

	return !EffectFunction || EffectFunction->GetMaterialFunction() != MaterialFunctionPtr;
}

void UDMMaterialEffectFunction::ApplyTo(const TSharedRef<FDMMaterialBuildState>& InBuildState, TArray<UMaterialExpression*>& InOutStageExpressions,
	int32& InOutLastExpressionOutputChannel, int32& InOutLastExpressionOutputIndex) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InOutStageExpressions.IsEmpty())
	{
		return;
	}

	UMaterialFunctionInterface* MaterialFunction = MaterialFunctionPtr;

	if (!IsValid(MaterialFunction))
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMaterialFunctionCall>(UE_DM_NodeComment_Default);
	FunctionCall->SetMaterialFunction(MaterialFunction);
	FunctionCall->UpdateFromFunctionResource();

	if (FunctionCall->FunctionInputs.Num() != InputValues.Num())
	{
		return;
	}

	UMaterialExpression* LastStageExpression = InOutStageExpressions.Last();

	TArray<UMaterialExpression*> LastInputExpressions;
	LastInputExpressions.Reserve(InputValues.Num() + 1);

	for (const TObjectPtr<UDMMaterialValue>& InputValue : InputValues)
	{
		// Certain inputs (such as index 0) are intentionally nullptr just to align input values with function inputs.
		if (!InputValue)
		{
			LastInputExpressions.Add(nullptr);
			continue;
		}

		InputValue->GenerateExpression(InBuildState);

		if (InBuildState->HasValue(InputValue))
		{
			const TArray<UMaterialExpression*>& ValueExpressions = InBuildState->GetValueExpressions(InputValue);

			if (!ValueExpressions.IsEmpty())
			{
				LastInputExpressions.Add(ValueExpressions.Last());
				InOutStageExpressions.Append(ValueExpressions);
				continue;
			}
		}

		LastInputExpressions.Add(nullptr);
	}

	LastInputExpressions[0] = LastStageExpression;

	for (int32 InputIndex = 0; InputIndex < InputValues.Num(); ++InputIndex)
	{
		if (LastInputExpressions[InputIndex])
		{
			if (InputIndex == 0)
			{
				if (InOutLastExpressionOutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
				{
					const int32 MaskedOutput = InBuildState->GetBuildUtils().FindOutputForBitmask(
						LastInputExpressions[0],
						InOutLastExpressionOutputChannel
					);

					if (MaskedOutput == INDEX_NONE)
					{
						UMaterialExpression* Mask = InBuildState->GetBuildUtils().CreateExpressionBitMask(
							LastInputExpressions[0],
							InOutLastExpressionOutputIndex,
							InOutLastExpressionOutputChannel
						);

						FunctionCall->FunctionInputs[InputIndex].Input.Connect(0, Mask);

						InOutStageExpressions.Add(Mask);
					}
					else
					{
						FunctionCall->FunctionInputs[InputIndex].Input.Connect(
							MaskedOutput,
							LastInputExpressions[0]
						);
					}
				}
				else
				{
					FunctionCall->FunctionInputs[InputIndex].Input.Connect(
						InOutLastExpressionOutputIndex,
						LastInputExpressions[0]
					);
				}
			}
			else
			{
				FunctionCall->FunctionInputs[InputIndex].Input.Connect(0, LastInputExpressions[InputIndex]);
			}
		}
		else
		{
			FunctionCall->FunctionInputs[InputIndex].Input.Expression = nullptr;
			FunctionCall->FunctionInputs[InputIndex].Input.OutputIndex = 0;
		}
	}

	InOutStageExpressions.Add(FunctionCall);

	// Output index from an effect function is always the first output.
	InOutLastExpressionOutputIndex = 0;
	InOutLastExpressionOutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
}

UObject* UDMMaterialEffectFunction::GetAsset() const
{
	return GetMaterialFunction();
}

FText UDMMaterialEffectFunction::GetComponentDescription() const
{
	return GetEffectName();
}

void UDMMaterialEffectFunction::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	for (const TObjectPtr<UDMMaterialValue>& Value : InputValues)
	{
		if (Value)
		{
			Value->PostEditorDuplicate(InMaterialModel, this);
		}
	}
}

bool UDMMaterialEffectFunction::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	for (const TObjectPtr<UDMMaterialValue>& Value : InputValues)
	{
		if (Value)
		{
			Value->Modify(bInAlwaysMarkDirty);
		}
	}

	return bSaved;
}

void UDMMaterialEffectFunction::PostLoad()
{
	Super::PostLoad();

	if (NeedsFunctionInit())
	{
		InitFunction();
	}
}

void UDMMaterialEffectFunction::OnMaterialFunctionChanged()
{
	DeinitFunction();
	InitFunction();
}

void UDMMaterialEffectFunction::DeinitFunction()
{
	for (const TObjectPtr<UDMMaterialValue>& Value : InputValues)
	{
		if (Value)
		{
			Value->SetComponentState(EDMComponentLifetimeState::Removed);
		}
	}

	InputValues.Empty();
}

bool UDMMaterialEffectFunction::NeedsFunctionInit() const
{
	if (!IsValid(MaterialFunctionPtr))
	{
		// If we have no function, but we do have inputs, they need to be refreshed (removed).
		return !InputValues.IsEmpty();
	}

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	MaterialFunctionPtr->GetInputsAndOutputs(Inputs, Outputs);

	if (Outputs.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function must have at least one output."), true, this);
		return false;
	}

	const EMaterialValueType InputType = Inputs[0].ExpressionInput ? Inputs[0].ExpressionInput->GetInputValueType(0) : MCT_Unknown;
	const bool bInputTypeIsFloat = (InputType & MCT_Float) != 0;

	const EMaterialValueType OutputType = Outputs[0].ExpressionOutput ? Outputs[0].ExpressionOutput->GetOutputValueType(0) : MCT_Unknown;
	const bool bOutputTypeIsFloat = (InputType & MCT_Float) != 0;

	/**
	 * First input and output types must match.
	 * Previously MCT_Float (a combination of float 1, 2, 3 and 4) caused a equality check to fail. It is now more rigorous.
	 */
	bool bValidThroughput = true;

	if (InputType == MCT_Unknown || OutputType == MCT_Unknown)
	{
		bValidThroughput = false;
	}
	else if (bInputTypeIsFloat && bOutputTypeIsFloat)
	{
		// If they are both float types and either of them are MCT_Float then they are a match.
		if (InputType != MCT_Float && OutputType != MCT_Float)
		{
			bValidThroughput = InputType == OutputType;
		}
	}
	else if (InputType != OutputType)
	{
		bValidThroughput = false;
	}

	if (!bValidThroughput)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function's first input must match its first output."), true, this);
		return false;
	}

	if (Inputs.Num() != InputValues.Num())
	{
		return true;
	}

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		UMaterialExpressionFunctionInput* FunctionInput = Inputs[InputIndex].ExpressionInput.Get();

		if (!IsValid(FunctionInput))
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function has missing input object."), true, this);
			return false;
		}

		// First input must be a scalar or vector.
		if (InputIndex == 0)
		{
			switch (FunctionInput->InputType)
			{
				case EFunctionInputType::FunctionInput_Scalar:
				case EFunctionInputType::FunctionInput_Vector2:
				case EFunctionInputType::FunctionInput_Vector3:
				case EFunctionInputType::FunctionInput_Vector4:
					break;

				default:
					UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function has invalid first input -  must be a scalar or vector."), true, this);
					return false;
			}

			continue;
		}

		const EDMValueType ValueType = UDMMaterialFunctionFunctionLibrary::GetInputValueType(FunctionInput);

		if (ValueType == EDMValueType::VT_None)
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function has invalid input type -  must be a scalar, vector or texture."), true, this);
			return false;
		}

		if (ValueType != InputValues[InputIndex]->GetType())
		{
			return true;
		}
	}

	return false;
}

void UDMMaterialEffectFunction::InitFunction()
{
	if (!IsValid(MaterialFunctionPtr))
	{
		return;
	}

	UDMMaterialSlot* Slot = GetTypedParent<UDMMaterialSlot>(/* bAllowSubclasses */ true);

	if (!Slot)
	{
		MaterialFunctionPtr = nullptr;
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

	if (!EditorOnlyData)
	{
		MaterialFunctionPtr = nullptr;
		return;
	}

	UDynamicMaterialModel* MaterialModel = EditorOnlyData->GetMaterialModel();

	if (!MaterialModel)
	{
		MaterialFunctionPtr = nullptr;
		return;
	}

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	MaterialFunctionPtr->GetInputsAndOutputs(Inputs, Outputs);

	if (Inputs.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function must have at least one input."), true, this);
		MaterialFunctionPtr = nullptr;
		return;
	}

	if (Outputs.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function must have at least one output."), true, this);
		MaterialFunctionPtr = nullptr;
		return;
	}

	InputValues.Reserve(Inputs.Num());

	const bool bSetValueAdded = IsComponentAdded();

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		UMaterialExpressionFunctionInput* FunctionInput = Inputs[InputIndex].ExpressionInput.Get();

		if (!IsValid(FunctionInput))
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function has missing input object."), true, this);
			MaterialFunctionPtr = nullptr;
			return;
		}

		// First input must be a scalar or vector.
		if (InputIndex == 0)
		{
			switch (FunctionInput->InputType)
			{
				case EFunctionInputType::FunctionInput_Scalar:
					EffectTarget = EDMMaterialEffectTarget::MaskStage;
					break;

				case EFunctionInputType::FunctionInput_Vector2:
					EffectTarget = EDMMaterialEffectTarget::TextureUV;
					break;

				case EFunctionInputType::FunctionInput_Vector3:
					EffectTarget = EDMMaterialEffectTarget::BaseStage;
					break;

				case EFunctionInputType::FunctionInput_Vector4:
					EffectTarget = EDMMaterialEffectTarget::Slot;
					break;

				default:
					UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function has invalid first input -  must be a scalar or vector."), true, this);
					MaterialFunctionPtr = nullptr;
					return;
			}

			InputValues.Add(nullptr);
			continue;
		}

		const EDMValueType ValueType = UDMMaterialFunctionFunctionLibrary::GetInputValueType(FunctionInput);

		if (ValueType == EDMValueType::VT_None)
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Effect Function has invalid input type -  must be a scalar, vector or texture."), true, this);
			MaterialFunctionPtr = nullptr;
			return;
		}

		UDMMaterialValue* Value = UDMMaterialValue::CreateMaterialValue(
			MaterialModel, 
			TEXT(""), 
			UDMValueDefinitionLibrary::GetValueDefinition(ValueType).GetValueClass(), 
			/* bInLocal */ true
		);
		check(Value);

		InputValues.Add(Value);

		UDMMaterialFunctionFunctionLibrary::SetInputDefault(FunctionInput, Value);

		if (bSetValueAdded)
		{
			Value->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}
}

UDMMaterialComponent* UDMMaterialEffectFunction::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == InputsPathToken)
	{
		int32 InputIndex;

		if (InPathSegment.GetParameter(InputIndex))
		{
			if (InputValues.IsValidIndex(InputIndex))
			{
				return InputValues[InputIndex]->GetComponentByPath(InPath);
			}
		}
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialEffectFunction::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	if (NeedsFunctionInit())
	{
		InitFunction();
	}

	Super::OnComponentAdded();
}

void UDMMaterialEffectFunction::OnComponentRemoved()
{
	DeinitFunction();

	Super::OnComponentRemoved();
}

#undef LOCTEXT_NAMESPACE
