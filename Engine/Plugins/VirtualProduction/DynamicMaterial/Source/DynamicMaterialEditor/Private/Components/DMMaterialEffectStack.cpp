// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialEffectStack.h"

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "DMComponentPath.h"
#include "UObject/Package.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialEffectStack)

#define LOCTEXT_NAMESPACE "DMMaterialEffectStack"

const FString UDMMaterialEffectStack::EffectsPathToken = FString(TEXT("Effect"));

UDMMaterialEffectStack* UDMMaterialEffectStack::CreateEffectStack(UDMMaterialSlot* InSlot)
{
	check(InSlot);

	return NewObject<UDMMaterialEffectStack>(InSlot, NAME_None, RF_Transactional);
}

UDMMaterialEffectStack* UDMMaterialEffectStack::CreateEffectStack(UDMMaterialLayerObject* InLayer)
{
	check(InLayer);

	return NewObject<UDMMaterialEffectStack>(InLayer, NAME_None, RF_Transactional);
}

UDMMaterialEffectStack::UDMMaterialEffectStack()
	: bEnabled(true)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialEffectStack, Effects));
}

UDMMaterialSlot* UDMMaterialEffectStack::GetSlot() const
{
	return Cast<UDMMaterialSlot>(GetOuterSafe());
}

UDMMaterialLayerObject* UDMMaterialEffectStack::GetLayer() const
{
	return Cast<UDMMaterialLayerObject>(GetOuterSafe());
}

bool UDMMaterialEffectStack::IsEnabled() const
{
	return bEnabled;
}

bool UDMMaterialEffectStack::SetEnabled(bool bInIsEnabled)
{
	if (bEnabled == bInIsEnabled)
	{
		return false;
	}

	bEnabled = bInIsEnabled;

	Update(this, EDMUpdateType::Structure);

	return true;
}

UDMMaterialEffect* UDMMaterialEffectStack::GetEffect(int32 InIndex) const
{
	if (Effects.IsValidIndex(InIndex))
	{
		return Effects[InIndex];
	}

	return nullptr;
}

TArray<UDMMaterialEffect*> UDMMaterialEffectStack::BP_GetEffects() const
{
	TArray<UDMMaterialEffect*> EffectObjects;
	EffectObjects.Reserve(Effects.Num());

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		EffectObjects.Add(Effect);
	}

	return EffectObjects;
}

const TArray<TObjectPtr<UDMMaterialEffect>>& UDMMaterialEffectStack::GetEffects() const
{
	return Effects;
}

bool UDMMaterialEffectStack::HasEffect(const UDMMaterialEffect* InEffect) const
{
	if (!IsValid(InEffect))
	{
		return false;
	}

	for (const TObjectPtr<UDMMaterialEffect>& EffectPtr : ReverseIterate(Effects))
	{
		if (UDMMaterialEffect* Effect = EffectPtr.Get())
		{
			if (Effect == InEffect)
			{
				return true;
			}
		}
	}

	return false;
}

bool UDMMaterialEffectStack::AddEffect(UDMMaterialEffect* InEffect)
{
	if (!IsValid(InEffect))
	{
		return false;
	}

	if (GUndo)
	{
		Modify();
		InEffect->Modify();
	}

	Effects.Remove(nullptr);
	TArray<UDMMaterialEffect*> IncompatibleEffects = GetIncompatibleEffects(InEffect);

	if (IncompatibleEffects.Num() == 1)
	{
		for (int32 ReplaceIndex = 0; ReplaceIndex < Effects.Num(); ++ReplaceIndex)
		{
			if (Effects[ReplaceIndex] == IncompatibleEffects[0])
			{
				SetEffect(ReplaceIndex, InEffect);
				return true;
			}
		}
	}

	RemoveIncompatibleEffects(InEffect);

	if (UDMMaterialEffectStack* OldStack = InEffect->GetEffectStack())
	{
		if (GUndo)
		{
			OldStack->Modify();
		}

		OldStack->RemoveEffect(InEffect);
	}

	Effects.Add(InEffect);

	if (IsComponentAdded())
	{
		InEffect->SetComponentState(EDMComponentLifetimeState::Added);
	}

	InEffect->Update(this, EDMUpdateType::Structure);

	return true;
}

UDMMaterialEffect* UDMMaterialEffectStack::SetEffect(int32 InIndex, UDMMaterialEffect* InEffect)
{
	if (!Effects.IsValidIndex(InIndex) || !IsValid(InEffect))
	{
		return nullptr;
	}

	UDMMaterialEffect* OldEffect = Effects[InIndex];

	if (OldEffect)
	{
		if (GUndo)
		{
			OldEffect->Modify();
		}

		OldEffect->SetEnabled(false);
		OldEffect->SetComponentState(EDMComponentLifetimeState::Removed);
	}

	if (GUndo)
	{
		InEffect->Modify();
	}

	InEffect->SetEnabled(true);
	InEffect->SetComponentState(EDMComponentLifetimeState::Added);

	Effects[InIndex] = InEffect;
	Effects[InIndex]->Update(this, EDMUpdateType::Structure);

	return OldEffect;
}

bool UDMMaterialEffectStack::MoveEffect(int32 InIndex, int32 InNewIndex)
{
	if (!IsComponentValid())
	{
		return false;
	}

	check(Effects.IsValidIndex(InIndex));

	InNewIndex = FMath::Clamp(InNewIndex, 0, Effects.Num() - 1);

	if (InNewIndex == InIndex)
	{
		return false;
	}

	UDMMaterialEffect* MovedEffect = Effects[InIndex];

	if (GUndo)
	{
		Modify();
	}

	Effects.RemoveAt(InIndex, EAllowShrinking::No); // Don't allow shrinking.
	Effects.Insert(MovedEffect, InNewIndex);

	const int MinIndex = FMath::Min(InIndex, InNewIndex);

	Effects[MinIndex]->Update(this, EDMUpdateType::Structure);

	return true;
}

bool UDMMaterialEffectStack::MoveEffect(UDMMaterialEffect* InEffect, int32 InNewIndex)
{
	if (!IsValid(InEffect))
	{
		return false;
	}

	const int32 EffectIndex = Effects.IndexOfByPredicate(
		[InEffect](const TObjectPtr<UDMMaterialEffect>& InElement)
		{
			return InEffect == InElement;
		}
	);

	if (EffectIndex != INDEX_NONE)
	{
		return MoveEffect(EffectIndex, InNewIndex);
	}

	return false;
}

UDMMaterialEffect* UDMMaterialEffectStack::RemoveEffect(int32 InIndex)
{
	if (!Effects.IsValidIndex(InIndex))
	{
		return nullptr;
	}

	UDMMaterialEffect* Effect = Effects[InIndex];

	if (GUndo)
	{
		Modify();
		Effect->Modify();
	}

	Effect->SetEnabled(false);
	Effect->SetComponentState(EDMComponentLifetimeState::Removed);

	Effects.RemoveAt(InIndex);

	Update(this, EDMUpdateType::Structure);

	return Effect;
}

bool UDMMaterialEffectStack::RemoveEffect(UDMMaterialEffect* InEffect)
{
	if (!IsValid(InEffect))
	{
		return false;
	}

	const int32 EffectIndex = Effects.IndexOfByPredicate(
		[InEffect](const TObjectPtr<UDMMaterialEffect>& InElement)
		{
			return InEffect == InElement;
		}
	);

	if (EffectIndex != INDEX_NONE)
	{
		return !!RemoveEffect(EffectIndex);
	}

	return false;
}

bool UDMMaterialEffectStack::ApplyEffects(const TSharedRef<FDMMaterialBuildState>& InBuildState, EDMMaterialEffectTarget InEffectTarget,
	TArray<UMaterialExpression*>& InOutStageExpressions, int32& InOutLastExpressionOutputChannel, int32& InOutLastExpressionOutputIndex) const
{
	bool bAppliedEffect = false;

	for (const TObjectPtr<UDMMaterialEffect>& EffectPtr : Effects)
	{
		if (UDMMaterialEffect* Effect = EffectPtr.Get())
		{
			if (IsValid(Effect) && InEffectTarget == Effect->GetEffectTarget() && Effect->IsEnabled())
			{
				Effect->ApplyTo(InBuildState, InOutStageExpressions, InOutLastExpressionOutputChannel, InOutLastExpressionOutputIndex);
				bAppliedEffect = true;
			}
		}
	}

	return bAppliedEffect;
}

FDMMaterialEffectStackJson UDMMaterialEffectStack::CreatePreset()
{
	FDMMaterialEffectStackJson Preset;
	Preset.bEnabled = bEnabled;

	Preset.Effects.Reserve(Effects.Num());

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		FDMMaterialEffectJson& EffectJson = Preset.Effects.AddDefaulted_GetRef();
		EffectJson.Class = Effect->GetClass();
		EffectJson.Data = Effect->JsonSerialize();
	}

	return Preset;
}

void UDMMaterialEffectStack::ApplyPreset(const FDMMaterialEffectStackJson& InPreset)
{
	SetEnabled(InPreset.bEnabled);

	for (const FDMMaterialEffectJson& EffectJson : InPreset.Effects)
	{
		if (!EffectJson.Class.Get())
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid class when applying effect preset."), true, this);
			continue;
		}

		UDMMaterialEffect* Effect = UDMMaterialEffect::CreateEffect(this, EffectJson.Class);

		if (!Effect)
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed creating class when applying effect preset."), true, this);
			continue;
		}

		Effect->JsonDeserialize(EffectJson.Data);

		AddEffect(Effect);
	}
}

UDMMaterialComponent* UDMMaterialEffectStack::GetParentComponent() const
{
	UObject* Outer = GetOuterSafe();

	if (UDMMaterialSlot* Slot = Cast<UDMMaterialSlot>(Outer))
	{
		return Slot;
	}

	if (UDMMaterialLayerObject* Layer = Cast<UDMMaterialLayerObject>(Outer))
	{
		return Layer;
	}

	return nullptr;
}

FString UDMMaterialEffectStack::GetComponentPathComponent() const
{
	return UDMMaterialLayerObject::EffectStackPathToken;
}

FText UDMMaterialEffectStack::GetComponentDescription() const
{
	static const FText Description = LOCTEXT("EffectStack", "Effect Stack");
	return Description;
}

void UDMMaterialEffectStack::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	Super::Update(InSource, InUpdateType);

	if (UDMMaterialComponent* Parent = GetParentComponent())
	{
		Parent->Update(InSource, InUpdateType);
	}
}

void UDMMaterialEffectStack::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (GetOuter() != InParent)
	{
		Rename(nullptr, InParent, UE::DynamicMaterial::RenameFlags);
	}

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		if (Effect)
		{
			Effect->PostEditorDuplicate(InMaterialModel, this);
		}
	}
}

bool UDMMaterialEffectStack::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		if (Effect)
		{
			Effect->Modify(bInAlwaysMarkDirty);
		}
	}

	return bSaved;
}

void UDMMaterialEffectStack::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialComponent* Parent = GetParentComponent();

	if (!Parent)
	{
		SetComponentState(EDMComponentLifetimeState::Removed);
		return;
	}

	MarkComponentDirty();

	Update(this, EDMUpdateType::Structure);
}

TArray<UDMMaterialEffect*> UDMMaterialEffectStack::GetIncompatibleEffects(UDMMaterialEffect* InEffect)
{
	if (!InEffect)
	{
		return {};
	}

	TArray<UDMMaterialEffect*> IncompatibleEffects;

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		if (!Effect || !Effect->IsCompatibleWith(InEffect))
		{
			IncompatibleEffects.Add(Effect);
		}
	}

	return IncompatibleEffects;
}

TArray<UDMMaterialEffect*> UDMMaterialEffectStack::RemoveIncompatibleEffects(UDMMaterialEffect* InEffect)
{
	if (!InEffect)
	{
		return {};
	}

	TArray<UDMMaterialEffect*> IncompatibleEffects;

	for (int32 Index = Effects.Num() - 1; Index >= 0; --Index)
	{
		UDMMaterialEffect* Effect = Effects[Index];

		if (!Effect || !Effect->IsCompatibleWith(InEffect))
		{
			IncompatibleEffects.Add(Effect);
			RemoveEffect(Index);
		}
	}

	return IncompatibleEffects;
}

UDMMaterialComponent* UDMMaterialEffectStack::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == EffectsPathToken)
	{
		int32 EffectIndex;

		if (InPathSegment.GetParameter(EffectIndex))
		{
			if (Effects.IsValidIndex(EffectIndex))
			{
				return Effects[EffectIndex]->GetComponentByPath(InPath);
			}
		}
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialEffectStack::OnComponentAdded()
{
	Super::OnComponentAdded();

	if (!IsComponentValid())
	{
		return;
	}

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		if (Effect)
		{
			Effect->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}
}

void UDMMaterialEffectStack::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	for (const TObjectPtr<UDMMaterialEffect>& Effect : Effects)
	{
		if (Effect)
		{
			Effect->SetComponentState(EDMComponentLifetimeState::Removed);
		}
	}
}

#undef LOCTEXT_NAMESPACE
