// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialLayer.h"

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "DMComponentPath.h"
#include "Dom/JsonObject.h"
#include "Factories.h"
#include "JsonObjectConverter.h"
#include "Misc/ReverseIterate.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialLayer)

#define LOCTEXT_NAMESPACE "DMMaterialLayer"

namespace UE::DynamicMaterialEditor::Private
{
	static constexpr int32 BaseIndex = 0;
	static constexpr int32 MaskIndex = 1;
}

struct FDMMaterialLayerObjectFactory : public FCustomizableTextObjectFactory
{
	TArray<UDMMaterialLayerObject*> CreatedLayers;

	FDMMaterialLayerObjectFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory interface
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		// Only allow layers to be created
		return ObjectClass->IsChildOf(UDMMaterialLayerObject::StaticClass());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		CreatedLayers.Add(CastChecked<UDMMaterialLayerObject>(NewObject));
	}
	// End of FCustomizableTextObjectFactory interface
};

const FString UDMMaterialLayerObject::StagesPathToken = FString(TEXT("Stages"));
const FString UDMMaterialLayerObject::BasePathToken = FString(TEXT("Base"));
const FString UDMMaterialLayerObject::MaskPathToken = FString(TEXT("Mask"));
const FString UDMMaterialLayerObject::EffectStackPathToken = FString(TEXT("Effects"));

UDMMaterialLayerObject* UDMMaterialLayerObject::CreateLayer(UDMMaterialSlot* InSlot, EDMMaterialPropertyType InMaterialProperty, 
	const TArray<UDMMaterialStage*>& InStages)
{
	check(InSlot);
	check(InMaterialProperty != EDMMaterialPropertyType::None);

	UDMMaterialLayerObject* NewLayer = NewObject<UDMMaterialLayerObject>(InSlot, NAME_None, RF_Transactional);
	NewLayer->MaterialProperty = InMaterialProperty;
	NewLayer->Stages = InStages;

	for (UDMMaterialStage* Stage : InStages)
	{
		Stage->Rename(nullptr, NewLayer, UE::DynamicMaterial::RenameFlags);
	}

	return NewLayer;
}

UDMMaterialLayerObject::UDMMaterialLayerObject()
	: bEnabled(true)
	, bLinkedUVs(true)
{
	EffectStack = CreateDefaultSubobject<UDMMaterialEffectStack>("EffectStack");
}

UDMMaterialSlot* UDMMaterialLayerObject::GetSlot() const
{
	return Cast<UDMMaterialSlot>(GetOuterSafe());
}

int32 UDMMaterialLayerObject::FindIndex() const
{
	if (!IsComponentValid())
	{
		return INDEX_NONE;
	}

	if (UDMMaterialSlot* Slot = GetSlot())
	{
		return Slot->GetLayers().IndexOfByPredicate(
			[this](const TObjectPtr<UDMMaterialLayerObject>& InElement)
			{
				return InElement == this;
			}
		);
	}

	return INDEX_NONE;
}

EDMMaterialLayerStage UDMMaterialLayerObject::GetStageType(const UDMMaterialStage* InStage) const
{
	const int32 Index = Stages.IndexOfByPredicate(
		[InStage](const TObjectPtr<UDMMaterialStage> InElement)
		{
			return InElement == InStage;
		}
	);

	using namespace UE::DynamicMaterialEditor::Private;

	switch (Index)
	{
		case BaseIndex:
			return EDMMaterialLayerStage::Base;

		case MaskIndex:
			return EDMMaterialLayerStage::Mask;

		default:
			return EDMMaterialLayerStage::None;
	}
}

void UDMMaterialLayerObject::SetLayerName(const FText& InName)
{
	LayerName = InName;
}

bool UDMMaterialLayerObject::AreAllStagesValid(EDMMaterialLayerStage InStageScope) const
{
	for (UDMMaterialStage* Stage : GetStages(InStageScope))
	{
		if (!IsValid(Stage))
		{
			return false;
		}
	}

	return true;
}

bool UDMMaterialLayerObject::AreAllStagesEnabled(EDMMaterialLayerStage InStageScope) const
{
	for (UDMMaterialStage* Stage : GetStages(InStageScope))
	{
		if (!IsValid(Stage) || !Stage->IsEnabled())
		{
			return false;
		}
	}

	return true;
}

EDMMaterialPropertyType UDMMaterialLayerObject::GetMaterialProperty() const
{
	return MaterialProperty;
}

bool UDMMaterialLayerObject::SetMaterialProperty(EDMMaterialPropertyType InMaterialProperty)
{
	if (MaterialProperty == InMaterialProperty)
	{
		return false;
	}

	MaterialProperty = InMaterialProperty;

	Update(this, EDMUpdateType::Structure);

	return true;
}

bool UDMMaterialLayerObject::IsTextureUVLinkEnabled() const
{
	return bLinkedUVs;
}

bool UDMMaterialLayerObject::SetTextureUVLinkEnabled(bool bInValue)
{
	if (bLinkedUVs == bInValue)
	{
		return false;
	}

	bLinkedUVs = bInValue;

	Update(this, EDMUpdateType::Structure);

	return true;
}

bool UDMMaterialLayerObject::ToggleTextureUVLinkEnabled()
{
	return SetTextureUVLinkEnabled(!bLinkedUVs);
}

UDMMaterialLayerObject* UDMMaterialLayerObject::GetPreviousLayer(EDMMaterialPropertyType InUsingProperty, EDMMaterialLayerStage InSearchFor) const
{
	if (!IsComponentValid())
	{
		return nullptr;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return nullptr;
	}

	int32 Index = FindIndex();

	if (Index == INDEX_NONE)
	{
		return nullptr;
	}

	const TArray<UDMMaterialLayerObject*>& Layers = Slot->GetLayers();

	for (Index = Index - 1; Index >= 0; --Index)
	{
		if ((InUsingProperty == EDMMaterialPropertyType::Any || Layers[Index]->GetMaterialProperty() == InUsingProperty)
			&& Layers[Index]->IsEnabled()
			&& Layers[Index]->GetStage(InSearchFor, /* bEnabledOnly */ true))
		{
			return Layers[Index];
		}
	}

	return nullptr;
}

UDMMaterialLayerObject* UDMMaterialLayerObject::GetNextLayer(EDMMaterialPropertyType InUsingProperty, EDMMaterialLayerStage InSearchFor) const
{
	if (!IsComponentValid())
	{
		return nullptr;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return nullptr;
	}

	int32 Index = FindIndex();

	if (Index == INDEX_NONE)
	{
		return nullptr;
	}

	const TArray<UDMMaterialLayerObject*>& Layers = Slot->GetLayers();

	for (Index = Index + 1; Index < Layers.Num(); ++Index)
	{
		if ((InUsingProperty == EDMMaterialPropertyType::Any || Layers[Index]->GetMaterialProperty() == InUsingProperty)
			&& Layers[Index]->IsEnabled()
			&& Layers[Index]->GetStage(InSearchFor, /* bEnabledOnly */ true))
		{
			return Layers[Index];
		}
	}

	return nullptr;
}

UDMMaterialStage* UDMMaterialLayerObject::GetFirstValidStage(EDMMaterialLayerStage InStageScope) const
{
	for (UDMMaterialStage* Stage : GetStages(InStageScope))
	{
		if (IsValid(Stage))
		{
			return Stage;
		}
	}

	return nullptr;
}

UDMMaterialStage* UDMMaterialLayerObject::GetLastValidStage(EDMMaterialLayerStage InStageScope) const
{
	TArray<UDMMaterialStage*> ScopedStages = GetStages(InStageScope);

	for (UDMMaterialStage* Stage : ReverseIterate(ScopedStages))
	{
		if (IsValid(Stage))
		{
			return Stage;
		}
	}

	return nullptr;
}

UDMMaterialStage* UDMMaterialLayerObject::GetFirstEnabledStage(EDMMaterialLayerStage InStageScope) const
{
	for (UDMMaterialStage* Stage : GetStages(InStageScope))
	{
		if (IsValid(Stage) && Stage->IsEnabled())
		{
			return Stage;
		}
	}

	return nullptr;
}

UDMMaterialStage* UDMMaterialLayerObject::GetLastEnabledStage(EDMMaterialLayerStage InStageScope) const
{
	TArray<UDMMaterialStage*> ScopedStages = GetStages(InStageScope);

	for (UDMMaterialStage* Stage : ReverseIterate(ScopedStages))
	{
		if (IsValid(Stage) && Stage->IsEnabled())
		{
			return Stage;
		}
	}

	return nullptr;
}

bool UDMMaterialLayerObject::SetStage(EDMMaterialLayerStage InStageType, UDMMaterialStage* InStage)
{
	if (!IsComponentValid())
	{
		return false;
	}

	using namespace UE::DynamicMaterialEditor::Private;

	int32 Index = INDEX_NONE;

	if (EnumHasAnyFlags(InStageType, EDMMaterialLayerStage::Base))
	{
		Index = BaseIndex;
	}
	else if (EnumHasAnyFlags(InStageType, EDMMaterialLayerStage::Mask))
	{
		Index = MaskIndex;
	}

	if (Index == INDEX_NONE)
	{
		return false;
	}

	if (!Stages.IsValidIndex(Index))
	{
		Stages.SetNum(Index + 1);
	}
	else if (IsValid(Stages[Index]))
	{
		Stages[Index]->SetComponentState(EDMComponentLifetimeState::Removed);
	}

	Stages[Index] = InStage;

	if (IsValid(InStage))
	{
		InStage->Rename(nullptr, this, UE::DynamicMaterial::RenameFlags);

		if (IsComponentAdded())
		{
			InStage->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}

	return true;
}

void UDMMaterialLayerObject::ForEachValidStage(EDMMaterialLayerStage InStageScope, FStageCallbackFunc InCallback) const
{
	for (UDMMaterialStage* Stage : GetStages(InStageScope))
	{
		if (IsValid(Stage))
		{
			InCallback(Stage);
		}
	}
}

void UDMMaterialLayerObject::ForEachEnabledStage(EDMMaterialLayerStage InStageScope, FStageCallbackFunc InCallback) const
{
	for (UDMMaterialStage* Stage : GetStages(InStageScope))
	{
		if (IsValid(Stage) && Stage->IsEnabled())
		{
			InCallback(Stage);
		}
	}
}

UDMMaterialEffectStack* UDMMaterialLayerObject::GetEffectStack() const
{
	return EffectStack;
}

UDMMaterialStage* UDMMaterialLayerObject::GetStage(EDMMaterialLayerStage InStageType, bool bInCheckEnabled) const
{
	TArray<UDMMaterialStage*> FilteredStages = GetStages(InStageType, bInCheckEnabled);

	if (!FilteredStages.IsEmpty())
	{
		return FilteredStages[0];
	}

	return nullptr;
}

TArray<UDMMaterialStage*> UDMMaterialLayerObject::GetStages(EDMMaterialLayerStage InStageType, bool bInCheckEnabled) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	TArray<UDMMaterialStage*> FilteredStages;
	FilteredStages.Reserve(Stages.Num());

	if (EnumHasAnyFlags(InStageType, EDMMaterialLayerStage::Base))
	{
		if (Stages.IsValidIndex(BaseIndex))
		{
			if ((!bInCheckEnabled || Stages[BaseIndex]->IsEnabled()) && IsValid(Stages[BaseIndex]))
			{
				FilteredStages.Add(Stages[BaseIndex]);
			}
		}
	}

	if (EnumHasAnyFlags(InStageType, EDMMaterialLayerStage::Mask))
	{
		if (Stages.IsValidIndex(MaskIndex))
		{
			if ((!bInCheckEnabled || Stages[MaskIndex]->IsEnabled()) && IsValid(Stages[MaskIndex]))
			{
				FilteredStages.Add(Stages[MaskIndex]);
			}
		}
	}

	return FilteredStages;
}

const TArray<TObjectPtr<UDMMaterialStage>>& UDMMaterialLayerObject::GetAllStages() const
{
	return Stages;
}

bool UDMMaterialLayerObject::HasValidStage(const UDMMaterialStage* InStage) const
{
	return GetStageType(InStage) != EDMMaterialLayerStage::None;
}

bool UDMMaterialLayerObject::HasValidStageOfType(EDMMaterialLayerStage InStageScope) const
{
	return IsValid(GetStage(InStageScope));
}

bool UDMMaterialLayerObject::IsEnabled() const
{
	return bEnabled;
}

bool UDMMaterialLayerObject::SetEnabled(bool bInIsEnabled)
{
	if (bEnabled == bInIsEnabled)
	{
		return false;
	}

	bEnabled = bInIsEnabled;

	Update(this, EDMUpdateType::Structure);

	return true;
}

bool UDMMaterialLayerObject::IsStageEnabled(EDMMaterialLayerStage InStageType) const
{
	if (UDMMaterialStage* Stage = GetStage(InStageType))
	{
		return Stage->IsEnabled();
	}

	return false;
}

bool UDMMaterialLayerObject::CanMoveLayerAbove(UDMMaterialLayerObject* InLayer) const
{
	if (!IsComponentValid())
	{
		return false;
	}

	if (!InLayer || InLayer == this)
	{
		return false;
	}

	const int32 ThisIndex = FindIndex();

	// Already top level - or invalid.
	if (ThisIndex == 0 || ThisIndex == INDEX_NONE)
	{
		return false;
	}

	const int32 DraggedLayerIndex = InLayer->FindIndex();

	if (DraggedLayerIndex == INDEX_NONE)
	{
		return false;
	}

	const UDMMaterialSlot* const ThisSlot = GetSlot();

	if (!IsValid(ThisSlot))
	{
		return false;
	}

	const UDMMaterialSlot* const DraggedLayerSlot = InLayer->GetSlot();

	if (!IsValid(DraggedLayerSlot))
	{
		return false;
	}

	if (ThisSlot != DraggedLayerSlot)
	{
		return false;
	}

	UDMMaterialStage* ThisBase = GetStage(EDMMaterialLayerStage::Base);

	if (!ThisBase)
	{
		return false;
	}

	UDMMaterialStage* DraggedLayerBase = InLayer->GetStage(EDMMaterialLayerStage::Base);

	if (!DraggedLayerBase)
	{
		return false;
	}

	UDMMaterialLayerObject* PreviousLayer = GetPreviousLayer(EDMMaterialPropertyType::Any, EDMMaterialLayerStage::All);
	UDMMaterialLayerObject* NextLayer = GetNextLayer(EDMMaterialPropertyType::Any, EDMMaterialLayerStage::All);

	// Can't move because it's the only layer
	if (!PreviousLayer && !NextLayer)
	{
		return true;
	}

	if (ThisIndex < DraggedLayerIndex)
	{
		if (PreviousLayer)
		{
			const bool bIsCompatibleWithBase = DraggedLayerBase->IsCompatibleWithPreviousStage(PreviousLayer->GetStage(EDMMaterialLayerStage::Base));
			const bool bIsCompatibleWithNextStage = DraggedLayerBase->IsCompatibleWithNextStage(ThisBase);

			if (!bIsCompatibleWithBase || !bIsCompatibleWithNextStage)
			{
				return false;
			}
		}
	}
	else
	{
		if (NextLayer)
		{
			const bool bIsCompatibleWithPreviousStage = DraggedLayerBase->IsCompatibleWithPreviousStage(ThisBase);
			const bool bIsCompatibleWithNextStage = DraggedLayerBase->IsCompatibleWithNextStage(NextLayer->GetStage(EDMMaterialLayerStage::Base));

			if (!bIsCompatibleWithPreviousStage || !bIsCompatibleWithNextStage)
			{
				return false;
			}
		}
	}

	PreviousLayer = InLayer->GetPreviousLayer(EDMMaterialPropertyType::Any, EDMMaterialLayerStage::All);
	NextLayer = InLayer->GetNextLayer(EDMMaterialPropertyType::Any, EDMMaterialLayerStage::All);

	if (PreviousLayer && NextLayer)
	{
		UDMMaterialStage* PreviousLayerBase = PreviousLayer->GetStage(EDMMaterialLayerStage::Base);
		UDMMaterialStage* NextLayerBase = NextLayer->GetStage(EDMMaterialLayerStage::Base);

		if (PreviousLayerBase && NextLayerBase)
		{
			if (!PreviousLayerBase->IsCompatibleWithNextStage(NextLayerBase))
			{
				return false;
			}

			if (!NextLayerBase->IsCompatibleWithPreviousStage(PreviousLayerBase))
			{
				return false;
			}
		}
	}
	
	return true;
}

bool UDMMaterialLayerObject::CanMoveLayerBelow(UDMMaterialLayerObject* InLayer) const
{
	if (!IsComponentValid())
	{
		return false;
	}

	if (!InLayer || InLayer == this)
	{
		return false;
	}

	const UDMMaterialSlot* const ThisSlot = GetSlot();

	if (!IsValid(ThisSlot))
	{
		return false;
	}

	const int32 ThisIndex = FindIndex();

	// Already bottom level - or invalid.
	if (ThisIndex == (ThisSlot->GetLayers().Num() - 1) || ThisIndex == INDEX_NONE)
	{
		return false;
	}

	const int32 DraggedLayerIndex = InLayer->FindIndex();

	if (DraggedLayerIndex == INDEX_NONE)
	{
		return false;
	}

	const UDMMaterialSlot* const DraggedLayerSlot = InLayer->GetSlot();

	if (!IsValid(DraggedLayerSlot))
	{
		return false;
	}

	if (ThisSlot != DraggedLayerSlot)
	{
		return false;
	}

	return true;
}

FString UDMMaterialLayerObject::SerializeToString() const
{
	TSharedPtr<FJsonObject> LayerJson = MakeShared<FJsonObject>();
	FJsonObjectConverter::UStructToJsonObject(UDMMaterialLayerObject::StaticClass(), this, LayerJson.ToSharedRef(), 0, CPF_Deprecated | CPF_Transient | CPF_TextExportTransient);

	FString SerializedString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&SerializedString);
	FJsonSerializer::Serialize(LayerJson.ToSharedRef(), Writer);

	return SerializedString;
}

void UDMMaterialLayerObject::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasLayer(this) || Stages.IsEmpty())
	{
		return;
	}

	UDMMaterialStage* BaseStage = GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true);

	// Layers with no base stage are dealt with by the alpha layer oh an activated based layer.
	if (!BaseStage)
	{
		return;
	}

	TArray<UMaterialExpression*> LayerExpressions;

	for (UDMMaterialStage* Stage : GetStages(EDMMaterialLayerStage::All, /* bEnabledOnly*/ true))
	{
		Stage->GenerateExpressions(InBuildState);
		LayerExpressions.Append(InBuildState->GetStageExpressions(Stage));
	}

	if (LayerExpressions.IsEmpty())
	{
		return;
	}

	InBuildState->AddLayerExpressions(this, LayerExpressions);
}

bool UDMMaterialLayerObject::ApplyEffects(const TSharedRef<FDMMaterialBuildState>& InBuildState, const UDMMaterialStage* InStage,
	TArray<UMaterialExpression*>& InOutStageExpressions, int32& InOutLastExpressionOutputChannel, int32& InOutLastExpressionOutputIndex) const
{
	if (InOutStageExpressions.IsEmpty())
	{
		return false;
	}

	if (!IsValid(EffectStack))
	{
		return false;
	}

	EDMMaterialLayerStage StageType = GetStageType(InStage);

	if (StageType == EDMMaterialLayerStage::None)
	{
		return false;
	}

	return EffectStack->ApplyEffects(InBuildState, UDMMaterialEffect::StageTypeToEffectType(StageType), InOutStageExpressions, 
		InOutLastExpressionOutputChannel, InOutLastExpressionOutputIndex);
}

UDMMaterialComponent* UDMMaterialLayerObject::GetParentComponent() const
{
	return GetSlot();
}

FString UDMMaterialLayerObject::GetComponentPathComponent() const
{
	return FString::Printf(
		TEXT("%s%c%i%c"),
		*UDMMaterialSlot::LayersPathToken,
		FDMComponentPath::ParameterOpen,
		FindIndex(),
		FDMComponentPath::ParameterClose
	);
}

FText UDMMaterialLayerObject::GetComponentDescription() const
{
	if (LayerName.IsEmpty())
	{
		static const FText NameFormat = LOCTEXT("LayerName", "Layer {0}");

		return FText::Format(NameFormat, FText::AsNumber(FindIndex()));
	}

	return LayerName;
}

void UDMMaterialLayerObject::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (UDMMaterialLayerObject* NextLayer = GetNextLayer(MaterialProperty, EDMMaterialLayerStage::All))
	{
		if (UDMMaterialStage* FirstStage = NextLayer->GetFirstEnabledStage(EDMMaterialLayerStage::All))
		{
			FirstStage->Update(InSource, InUpdateType);
		}
		else if (UDMMaterialEffectStack* NextEffectStack = NextLayer->GetEffectStack())
		{
			NextEffectStack->Update(InSource, InUpdateType);
		}
		else
		{
			NextLayer->Update(InSource, InUpdateType);
		}
	}
	else if (UDMMaterialSlot* Slot = GetSlot())
	{
		Slot->Update(InSource, InUpdateType);
	}

	Super::Update(InSource, InUpdateType);
}

void UDMMaterialLayerObject::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (GetOuter() != InParent)
	{
		Rename(nullptr, InParent, UE::DynamicMaterial::RenameFlags);
	}

	for (const TObjectPtr<UDMMaterialStage>& Stage : Stages)
	{
		if (Stage)
		{
			Stage->PostEditorDuplicate(InMaterialModel, this);
		}
	}
}

bool UDMMaterialLayerObject::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	for (const TObjectPtr<UDMMaterialStage>& Stage : Stages)
	{
		if (Stage)
		{
			Stage->Modify(bInAlwaysMarkDirty);
		}
	}

	return bSaved;
}

void UDMMaterialLayerObject::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		SetComponentState(EDMComponentLifetimeState::Removed);
		return;
	}

	MarkComponentDirty();

	Update(this, EDMUpdateType::Structure);
}

UDMMaterialComponent* UDMMaterialLayerObject::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == BasePathToken)
	{
		return GetStage(EDMMaterialLayerStage::Base, /* bCheckEnabled */ false);
	}

	if (InPathSegment.GetToken() == MaskPathToken)
	{
		return GetStage(EDMMaterialLayerStage::Mask, /* bCheckEnabled */ false);
	}

	if (InPathSegment.GetToken() == EffectStackPathToken)
	{
		return EffectStack;
	}

	if (InPathSegment.GetToken() == StagesPathToken)
	{
		int32 StageIndex;

		if (InPathSegment.GetParameter(StageIndex))
		{
			if (Stages.IsValidIndex(StageIndex))
			{
				return Stages[StageIndex]->GetComponentByPath(InPath);
			}
		}
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialLayerObject::OnComponentAdded()
{
	Super::OnComponentAdded();

	if (!IsComponentValid())
	{
		return;
	}

	for (const TObjectPtr<UDMMaterialStage>& Stage : Stages)
	{
		if (Stage)
		{
			Stage->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}

	if (IsValid(EffectStack))
	{
		EffectStack->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialLayerObject::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	for (const TObjectPtr<UDMMaterialStage>& Stage : Stages)
	{
		if (Stage)
		{
			Stage->SetComponentState(EDMComponentLifetimeState::Removed);
		}
	}

	if (IsValid(EffectStack))
	{
		EffectStack->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

UDMMaterialLayerObject* UDMMaterialLayerObject::DeserializeFromString(UDMMaterialSlot* InOuter, const FString& InSerializedString)
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = InOuter->GetMaterialModelEditorOnlyData();

	if (!EditorOnlyData)
	{
		return nullptr;
	}

	UDynamicMaterialModel* MaterialModel = EditorOnlyData->GetMaterialModel();

	if (!MaterialModel)
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject> LayerJson;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InSerializedString);

	UDMMaterialLayerObject* PastedLayer = NewObject<UDMMaterialLayerObject>(InOuter, NAME_None, RF_Transactional);

	if (!FJsonSerializer::Deserialize(Reader, LayerJson))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Unable to deserialise clipboard data."));
		return nullptr;
	}

	FJsonObjectConverter::JsonObjectToUStruct(LayerJson.ToSharedRef(), StaticClass(), PastedLayer);

	return PastedLayer;
}

#undef LOCTEXT_NAMESPACE
