// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMDefs.h"
#include "DMMaterialLayer_Deprecated.generated.h"

class UDMMaterialStage;

USTRUCT(BlueprintType, Category = "Material Designer")
struct FDMMaterialLayer
{
	GENERATED_BODY()

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMMaterialPropertyType MaterialProperty = EDMMaterialPropertyType::None;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FText LayerName;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialStage> Base = nullptr;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialStage> Mask = nullptr;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bEnabled = true;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bBaseEnabled = true;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bMaskEnabled = true;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	mutable bool bLinkedUVs = true;
};
