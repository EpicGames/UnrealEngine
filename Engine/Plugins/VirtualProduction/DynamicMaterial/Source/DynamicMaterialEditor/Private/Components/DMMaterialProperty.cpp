// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialProperty.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageBlends/DMMSBNormal.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageExpressions/DMMSETextureSampleBase.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DMComponentPath.h"
#include "DMValueDefinition.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialEditorSettings.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionMax.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialProperty)

#define LOCTEXT_NAMESPACE "DMMaterialProperty"

const FString UDMMaterialProperty::ComponentsPathToken = TEXT("Components");

UDMMaterialProperty::UDMMaterialProperty()
	: UDMMaterialProperty(EDMMaterialPropertyType::None, EDMValueType::VT_Float1)
{
}

UDMMaterialProperty::UDMMaterialProperty(EDMMaterialPropertyType InMaterialProperty, EDMValueType InInputConnectorType)
	: MaterialProperty(InMaterialProperty)
	, bEnabled(true)
	, InputConnectorType(InInputConnectorType)
{
}

FString UDMMaterialProperty::GetComponentPathComponent() const
{
	return StaticEnum<EDMMaterialPropertyType>()->GetNameStringByValue(static_cast<int64>(MaterialProperty));
}

UDMMaterialProperty* UDMMaterialProperty::CreateCustomMaterialPropertyDefaultSubobject(UDynamicMaterialModelEditorOnlyData* InModelEditorOnlyData, 
	EDMMaterialPropertyType InMaterialProperty, const FName& InSubObjName)
{
	check(InModelEditorOnlyData);
	check(InMaterialProperty >= EDMMaterialPropertyType::Custom1 && InMaterialProperty <= EDMMaterialPropertyType::Custom4);

	UDMMaterialProperty* NewMaterialProperty = InModelEditorOnlyData->CreateDefaultSubobject<UDMMaterialProperty>(InSubObjName);
	NewMaterialProperty->MaterialProperty = InMaterialProperty;
	NewMaterialProperty->InputConnectorType = EDMValueType::VT_None;

	return NewMaterialProperty;
}

UDynamicMaterialModelEditorOnlyData* UDMMaterialProperty::GetMaterialModelEditorOnlyData() const
{
	return Cast<UDynamicMaterialModelEditorOnlyData>(GetOuterSafe());
}

void UDMMaterialProperty::SetEnabled(bool bInEnabled)
{
	if (bEnabled == bInEnabled)
	{
		return;
	}

	bEnabled = bInEnabled;

	Update(this, EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}

FText UDMMaterialProperty::GetDescription() const
{
	return UE::DynamicMaterialEditor::Private::GetMaterialPropertyLongDisplayName(MaterialProperty);
}

bool UDMMaterialProperty::IsMaterialPin() const
{
	switch (MaterialProperty)
	{
		case EDMMaterialPropertyType::None:
		case EDMMaterialPropertyType::Any:
		case EDMMaterialPropertyType::Custom1:
		case EDMMaterialPropertyType::Custom2:
		case EDMMaterialPropertyType::Custom3:
		case EDMMaterialPropertyType::Custom4:
				return false;

		default:
			return true;
	}
}

void UDMMaterialProperty::ResetInputConnectionMap()
{
	if (!IsComponentValid())
	{
		return;
	}

	InputConnectionMap.Channels.Empty();

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDMMaterialSlot* Slot = ModelEditorOnlyData->GetSlotForMaterialProperty(MaterialProperty);

	if (!Slot || Slot->GetLayers().IsEmpty())
	{
		return;
	}

	const TArray<EDMValueType>& SlotOutputTypes = Slot->GetOutputConnectorTypesForMaterialProperty(MaterialProperty);

	for (int32 SlotOutputIdx = 0; SlotOutputIdx < SlotOutputTypes.Num(); ++SlotOutputIdx)
	{
		if (UDMValueDefinitionLibrary::AreTypesCompatible(SlotOutputTypes[SlotOutputIdx], InputConnectorType))
		{
			InputConnectionMap.Channels.Add({
				FDMMaterialStageConnectorChannel::PREVIOUS_STAGE,
				MaterialProperty,
				SlotOutputIdx,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			});

			break;
		}
	}
}

UMaterialExpression* UDMMaterialProperty::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return nullptr;
}

TEnumAsByte<EMaterialSamplerType> UDMMaterialProperty::GetTextureSamplerType() const
{
	return EMaterialSamplerType::SAMPLERTYPE_LinearColor;
}

void UDMMaterialProperty::OnSlotAdded(UDMMaterialSlot* InSlot)
{
	if (!IsValid(InSlot))
	{
		return;
	}

	InSlot->AddDefaultLayer(MaterialProperty);
}

void UDMMaterialProperty::AddDefaultBaseStage(UDMMaterialLayerObject* InLayer)
{
	if (!IsValid(InLayer))
	{
		return;
	}

	if (InLayer->GetMaterialProperty() == EDMMaterialPropertyType::None)
	{
		InLayer->SetMaterialProperty(MaterialProperty);
	}

	UDMMaterialStage* DefaultStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	check(DefaultStage);

	InLayer->SetStage(EDMMaterialLayerStage::Base, DefaultStage);

	const FDMDefaultMaterialPropertySlotValue& DefaultValue = UDynamicMaterialEditorSettings::Get()->GetDefaultSlotValue(MaterialProperty);

	switch (DefaultValue.DefaultType)
	{
		case EDMDefaultMaterialPropertySlotValueType::Texture:
		{
			UDMMaterialStageInputExpression* BaseInputExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
				DefaultStage,
				UDMMaterialStageExpressionTextureSample::StaticClass(),
				UDMMaterialStageBlendNormal::InputB,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
				0,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);

			break;
		}

		case EDMDefaultMaterialPropertySlotValueType::Color:
		{
			UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
				DefaultStage,
				UDMMaterialStageBlendNormal::InputB,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
				EDMValueType::VT_Float3_RGB,
				0
			);

			break;
		}
	}
}

void UDMMaterialProperty::AddDefaultMaskStage(UDMMaterialLayerObject* InLayer)
{
	if (!IsValid(InLayer))
	{
		return;
	}

	UDMMaterialStage* MaskStage = UDMMaterialStageThroughputLayerBlend::CreateStage();
	check(MaskStage);

	InLayer->SetStage(EDMMaterialLayerStage::Mask, MaskStage);
}

UDMMaterialComponent* UDMMaterialProperty::AddComponent(FName InName, UDMMaterialComponent* InComponent)
{
	if (!IsValid(InComponent))
	{
		InComponent = nullptr;
	}

	const TObjectPtr<UDMMaterialComponent>* CurrentComponentPtr = Components.Find(InName);

	if (CurrentComponentPtr && IsValid(*CurrentComponentPtr))
	{
		if ((*CurrentComponentPtr) == InComponent)
		{
			return nullptr;
		}

		(*CurrentComponentPtr)->SetComponentState(EDMComponentLifetimeState::Removed);
	}
	else if (!InComponent)
	{
		return nullptr;
	}

	if (InComponent)
	{
		Components.FindOrAdd(InName) = InComponent;
		InComponent->SetComponentState(EDMComponentLifetimeState::Added);
	}
	else if (CurrentComponentPtr)
	{
		Components.Remove(InName);
	}

	if (CurrentComponentPtr)
	{
		return *CurrentComponentPtr;
	}

	return nullptr;
}

bool UDMMaterialProperty::HasComponent(FName InName) const
{
	return Components.Contains(InName);
}

UDMMaterialComponent* UDMMaterialProperty::GetComponent(FName InName) const
{
	if (const TObjectPtr<UDMMaterialComponent>* CurrentComponentPtr = Components.Find(InName))
	{
		return *CurrentComponentPtr;
	}

	return nullptr;
}

UDMMaterialComponent* UDMMaterialProperty::RemoveComponent(FName InName)
{
	if (const TObjectPtr<UDMMaterialComponent>* CurrentComponentPtr = Components.Find(InName))
	{
		Components.Remove(InName);
		return *CurrentComponentPtr;
	}

	return nullptr;
}

void UDMMaterialProperty::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetMaterialModelEditorOnlyData();

	if (!EditorOnlyData || !IsValidForModel(*EditorOnlyData))
	{
		return;
	}

	// For now we don't have channel remapping!
	FExpressionInput* MaterialPropertyPtr = InBuildState->GetMaterialProperty(MaterialProperty);

	if (!MaterialPropertyPtr)
	{
		return;
	}

	MaterialPropertyPtr->Expression = nullptr;
	MaterialPropertyPtr->OutputIndex = 0;

	UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForMaterialProperty(MaterialProperty);

	if (!Slot || Slot->GetLayers().IsEmpty())
	{
		return;
	}

	InBuildState->SetCurrentMaterialProperty(this);

	Slot->GenerateExpressions(InBuildState);

	if (InBuildState->GetSlotExpressions(Slot).IsEmpty())
	{
		return;
	}

	UMaterialExpression* LastPropertyExpression = InBuildState->GetLastSlotPropertyExpression(Slot, MaterialProperty);

	if (!LastPropertyExpression)
	{
		return;
	}

	MaterialPropertyPtr->Expression = LastPropertyExpression;

	if (InputConnectionMap.Channels.IsEmpty() == false)
	{
		MaterialPropertyPtr->OutputIndex = InputConnectionMap.Channels[0].OutputIndex;
	}
	else
	{
		MaterialPropertyPtr->OutputIndex = 0;
	}
}

void UDMMaterialProperty::GenerateOpacityExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState, UDMMaterialSlot* InFromSlot, 
	EDMMaterialPropertyType InFromProperty, UMaterialExpression*& OutExpression, int32& OutOutputIndex, int32& OutOutputChannel)
{
	const TArray<TObjectPtr<UDMMaterialLayerObject>>& SlotLayers = InFromSlot->GetLayers();
	OutExpression = nullptr;

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : SlotLayers)
	{
		if (!IsValid(Layer))
		{
			continue;
		}

		// Although we are working with masks, if the base is disabled, this is handled by the GenerateExpressions
		// of the LayerBlend code (to multiply alpha together, instead of maxing it).
		if (Layer->GetMaterialProperty() != InFromProperty || !Layer->IsEnabled() || !Layer->IsStageEnabled(EDMMaterialLayerStage::Base))
		{
			continue;
		}

		UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base);
		UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask);

		if (!MaskStage->IsEnabled())
		{
			continue;
		}

		MaskStage->GenerateExpressions(InBuildState);
		UDMMaterialStageThroughputLayerBlend* LayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(MaskStage->GetSource());

		if (!LayerBlend)
		{
			continue;
		}

		UMaterialExpression* MaskOutputExpression;
		int32 MaskOutputIndex;
		int32 MaskOutputChannel;
		LayerBlend->GetMaskOutput(InBuildState, MaskOutputExpression, MaskOutputIndex, MaskOutputChannel);

		if (!MaskOutputExpression)
		{
			continue;
		}

		if (LayerBlend->UsePremultiplyAlpha())
		{
			if (UDMMaterialStageSource* Source = BaseStage->GetSource())
			{
				UMaterialExpression* LayerAlphaOutputExpression;
				int32 LayerAlphaOutputIndex;
				int32 LayerAlphaOutputChannel;

				Source->GetMaskAlphaBlendNode(InBuildState, LayerAlphaOutputExpression, LayerAlphaOutputIndex, LayerAlphaOutputChannel);

				if (LayerAlphaOutputExpression)
				{
					UMaterialExpressionMultiply* AlphaMultiply = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMultiply>(UE_DM_NodeComment_Default);

					AlphaMultiply->A.Expression = MaskOutputExpression;
					AlphaMultiply->A.OutputIndex = MaskOutputIndex;
					AlphaMultiply->A.Mask = 0;

					if (MaskOutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
					{
						AlphaMultiply->A.Mask = 1;
						AlphaMultiply->A.MaskR = !!(MaskOutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
						AlphaMultiply->A.MaskG = !!(MaskOutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
						AlphaMultiply->A.MaskB = !!(MaskOutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
						AlphaMultiply->A.MaskA = !!(MaskOutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);
					}

					AlphaMultiply->B.Expression = LayerAlphaOutputExpression;
					AlphaMultiply->B.OutputIndex = LayerAlphaOutputIndex;
					AlphaMultiply->B.Mask = 0;

					if (LayerAlphaOutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
					{
						AlphaMultiply->B.Mask = 1;
						AlphaMultiply->B.MaskR = !!(LayerAlphaOutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
						AlphaMultiply->B.MaskG = !!(LayerAlphaOutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
						AlphaMultiply->B.MaskB = !!(LayerAlphaOutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
						AlphaMultiply->B.MaskA = !!(LayerAlphaOutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);
					}

					MaskOutputExpression = AlphaMultiply;
					MaskOutputIndex = 0;
					MaskOutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
				}
			}
		}

		if (OutExpression == nullptr)
		{
			OutExpression = MaskOutputExpression;

			// The first output will use the node's output info.
			OutOutputIndex = MaskOutputIndex;
			OutOutputChannel = MaskOutputChannel;
			continue;
		}

		TArray<UMaterialExpression*> BaseExpressions;
		int32 BlendedOutputIndex;
		int32 BlendedOutputChannel;

		if (UDMMaterialStageBlend* BaseBlend = Cast<UDMMaterialStageBlend>(BaseStage->GetSource()))
		{
			BaseBlend->BlendOpacityLayer(InBuildState, OutExpression, OutOutputIndex, OutOutputChannel, MaskOutputExpression,
				MaskOutputIndex, MaskOutputChannel, BaseExpressions, BlendedOutputIndex, BlendedOutputChannel);
		}
		else
		{
			UDMMaterialStageBlend::CreateBlendOpacityLayer<UMaterialExpressionMax>(InBuildState, OutExpression, OutOutputIndex, 
				OutOutputChannel, MaskOutputExpression, MaskOutputIndex, MaskOutputChannel, BaseExpressions, BlendedOutputIndex,
				BlendedOutputChannel);
		}

		OutExpression = BaseExpressions.Last();
		OutOutputIndex = BlendedOutputIndex;
		OutOutputChannel = BlendedOutputChannel;
	}
}

void UDMMaterialProperty::AddAlphaMultiplier(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	UDMMaterialValueFloat1* AlphaValue = GetTypedComponent<UDMMaterialValueFloat1>(UDynamicMaterialModelEditorOnlyData::AlphaValueName);

	if (!AlphaValue)
	{
		return;
	}

	FExpressionInput* PropertyInputExpression = InBuildState->GetMaterialProperty(MaterialProperty);

	if (!PropertyInputExpression || !PropertyInputExpression->Expression)
	{
		return;
	}

	AlphaValue->GenerateExpression(InBuildState);

	UMaterialExpression* GlobalOpacityExpression = InBuildState->GetLastValueExpression(AlphaValue);

	if (!GlobalOpacityExpression)
	{
		return;
	}

	UMaterialExpressionMultiply* OpacityMultiply = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMultiply>(UE_DM_NodeComment_Default);
	OpacityMultiply->A.Expression = PropertyInputExpression->Expression;
	OpacityMultiply->A.Mask = PropertyInputExpression->Mask;
	OpacityMultiply->A.MaskR = PropertyInputExpression->MaskR;
	OpacityMultiply->A.MaskG = PropertyInputExpression->MaskG;
	OpacityMultiply->A.MaskB = PropertyInputExpression->MaskB;
	OpacityMultiply->A.MaskA = PropertyInputExpression->MaskA;
	OpacityMultiply->A.OutputIndex = PropertyInputExpression->OutputIndex;

	OpacityMultiply->B.Expression = GlobalOpacityExpression;
	OpacityMultiply->B.SetMask(1, 1, 0, 0, 0);
	OpacityMultiply->B.OutputIndex = 0;

	PropertyInputExpression->Expression = OpacityMultiply;
}

void UDMMaterialProperty::AddOutputProcessor(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!OutputProcessor)
	{
		return;
	}

	FExpressionInput* MaterialPropertyPtr = InBuildState->GetMaterialProperty(MaterialProperty);

	if (!MaterialPropertyPtr)
	{
		return;
	}

	UMaterialExpression* LastPropertyExpression = MaterialPropertyPtr->Expression;

	if (!LastPropertyExpression)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall = FDMMaterialFunctionLibrary::Get().MakeExpression(
		InBuildState->GetDynamicMaterial(),
		OutputProcessor,
		UE_DM_NodeComment_Default
	);

	FExpressionInput* FirstInput = MaterialFunctionCall->GetInput(0);
	if (!FirstInput)
	{
		return;
	}

	LastPropertyExpression->ConnectExpression(FirstInput, MaterialPropertyPtr->OutputIndex);
	MaterialFunctionCall->ConnectExpression(MaterialPropertyPtr, 0);

	MaterialPropertyPtr->OutputIndex = 0;
}

void UDMMaterialProperty::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetMaterialModelEditorOnlyData();

	if (!EditorOnlyData)
	{
		return;
	}

	InPreviewMaterial->BlendMode = EBlendMode::BLEND_Translucent;

	TSharedRef<FDMMaterialBuildState> BuildState = EditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);

	UE_LOGF(LogDynamicMaterialEditor, Display, "Building Material Designer Property Preview (%ls)...", *GetName());

	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &BuildState](EDMMaterialPropertyType InType)
		{
			if (FExpressionInput* Input = BuildState->GetMaterialProperty(InType))
			{
				Input->Expression = nullptr;
			}

			return EDMIterationResult::Continue;
		});


	if (!IsEnabled() || !EditorOnlyData->GetSlotForMaterialProperty(MaterialProperty))
	{
		return;
	}

	GenerateExpressions(BuildState);

	AddAlphaMultiplier(BuildState);

	// Move output to the emissive channel.
	if (MaterialProperty != EDMMaterialPropertyType::EmissiveColor)
	{
		if (FExpressionInput* EmissiveExpressionInput = BuildState->GetMaterialProperty(EDMMaterialPropertyType::EmissiveColor))
		{
			FExpressionInput* MyFirstExpressionInput = nullptr;

			UE::DynamicMaterial::ForEachMaterialPropertyType(
				[this, &BuildState, &MyFirstExpressionInput](EDMMaterialPropertyType InType)
				{
					using namespace UE::DynamicMaterialEditor::Private;

					MyFirstExpressionInput = BuildState->GetMaterialProperty(InType);

					if (!MyFirstExpressionInput || MyFirstExpressionInput->Expression == nullptr)
					{
						MyFirstExpressionInput = nullptr;
						return EDMIterationResult::Continue;
					}

					return EDMIterationResult::Break;
				});

			if (MyFirstExpressionInput)
			{
				// Swap inputs to emissive channel
				EmissiveExpressionInput->Expression = MyFirstExpressionInput->Expression;
				EmissiveExpressionInput->Mask = MyFirstExpressionInput->Mask;
				EmissiveExpressionInput->MaskR = MyFirstExpressionInput->MaskR;
				EmissiveExpressionInput->MaskG = MyFirstExpressionInput->MaskG;
				EmissiveExpressionInput->MaskB = MyFirstExpressionInput->MaskB;
				EmissiveExpressionInput->MaskA = MyFirstExpressionInput->MaskA;
				EmissiveExpressionInput->OutputIndex = MyFirstExpressionInput->OutputIndex;

				MyFirstExpressionInput->Expression = nullptr;
			}
		}
	}

	// Attempt to create an opacity channel.
	FExpressionInput* OpacityExpression = BuildState->GetMaterialProperty(EDMMaterialPropertyType::Opacity);

	if (OpacityExpression && !OpacityExpression->Expression)
	{
		UDMMaterialSlot* MySlot = EditorOnlyData->GetSlotForMaterialProperty(MaterialProperty);
		UMaterialExpression* OpacityOutputNode;
		int32 OutputIndex;
		int32 OutputChannel;
		GenerateOpacityExpressions(BuildState, MySlot, MaterialProperty, OpacityOutputNode, OutputIndex, OutputChannel);

		if (OpacityOutputNode)
		{
			OpacityExpression->Expression = OpacityOutputNode;
			OpacityExpression->OutputIndex = 0;
			OpacityExpression->Mask = 0;

			if (OutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
			{
				OpacityExpression->Mask = 1;
				OpacityExpression->MaskR = !!(OutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
				OpacityExpression->MaskG = !!(OutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
				OpacityExpression->MaskB = !!(OutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
				OpacityExpression->MaskA = !!(OutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);
			}
		}
	}
}

void UDMMaterialProperty::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	Super::Update(InSource, InUpdateType);

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
		check(ModelEditorOnlyData);

		ModelEditorOnlyData->OnPropertyUpdate(this);
	}
}

void UDMMaterialProperty::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(InMaterialModel);

	if (ModelEditorOnlyData && GetOuter() != ModelEditorOnlyData)
	{
		Rename(nullptr, ModelEditorOnlyData, UE::DynamicMaterial::RenameFlags);
	}
}

void UDMMaterialProperty::PreEditChange(FProperty* InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);

	static const FName OutputProcessorName = GET_MEMBER_NAME_CHECKED(UDMMaterialProperty, OutputProcessor);

	if (InPropertyAboutToChange->GetFName() == OutputProcessorName)
	{
		OutputProcessor_PreUpdate = OutputProcessor;
	}
}

void UDMMaterialProperty::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName OutputProcessorName = GET_MEMBER_NAME_CHECKED(UDMMaterialProperty, OutputProcessor);

	if (InPropertyChangedEvent.Property && InPropertyChangedEvent.Property->GetFName() == OutputProcessorName)
	{
		OnOutputProcessorUpdated();
	}
}

void UDMMaterialProperty::LoadDeprecatedModelData(UDMMaterialProperty* InOldProperty)
{
	InputConnectionMap = InOldProperty->InputConnectionMap;
	OutputProcessor = InOldProperty->OutputProcessor;
}

void UDMMaterialProperty::SetOutputProcessor(UMaterialFunctionInterface* InFunction)
{
	if (OutputProcessor == InFunction)
	{
		return;
	}

	OutputProcessor_PreUpdate = OutputProcessor;
	OutputProcessor = InFunction;

	OnOutputProcessorUpdated();
}

bool UDMMaterialProperty::IsValidForModel(const UDynamicMaterialModelEditorOnlyData& InEditorOnlyData) const
{
	if (UMaterial* GeneratedMaterial = InEditorOnlyData.GetGeneratedMaterial())
	{
		return GeneratedMaterial->IsPropertyActiveInEditor(FDMUtils::MaterialPropertyTypeToMaterialProperty(MaterialProperty));
	}

	return false;
}

void UDMMaterialProperty::OnOutputProcessorUpdated()
{
	if (!OutputProcessor)
	{
		if (OutputProcessor_PreUpdate)
		{
			Update(this, EDMUpdateType::Structure);
		}

		OutputProcessor = nullptr;
		OutputProcessor_PreUpdate = nullptr;
		return;
	}

	using namespace UE::DynamicMaterialEditor;

	bool bValid = true;

	do
	{
		TArray<FFunctionExpressionInput> Inputs;
		TArray<FFunctionExpressionOutput> Outputs;

		OutputProcessor->GetInputsAndOutputs(Inputs, Outputs);

		if (Inputs.IsEmpty() || Outputs.IsEmpty())
		{
			bValid = false;
			break;
		}
	}
	while (false);

	if (!bValid)
	{
		if (IsValid(OutputProcessor_PreUpdate))
		{
			// No update has occurred
			OutputProcessor = OutputProcessor_PreUpdate;
			OutputProcessor_PreUpdate = nullptr;
			return;
		}
		else
		{
			// Possible update has occurred
			OutputProcessor = nullptr;
			OutputProcessor_PreUpdate = nullptr;
		}
	}

	Update(this, EDMUpdateType::Structure);
}

UDMMaterialComponent* UDMMaterialProperty::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == ComponentsPathToken)
	{
		FString ComponentString;

		if (InPathSegment.GetParameter(ComponentString))
		{
			const FName ComponentName = *ComponentString;

			if (const TObjectPtr<UDMMaterialComponent>* CurrentComponentPtr = Components.Find(ComponentName))
			{
				return *CurrentComponentPtr;
			}
		}
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialProperty::OnComponentAdded()
{
	Super::OnComponentAdded();

	for (const TPair<FName, TObjectPtr<UDMMaterialComponent>>& Pair : Components)
	{
		if (IsValid(Pair.Value))
		{
			Pair.Value->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}
}

void UDMMaterialProperty::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	for (const TPair<FName, TObjectPtr<UDMMaterialComponent>>& Pair : Components)
	{
		if (IsValid(Pair.Value))
		{
			Pair.Value->SetComponentState(EDMComponentLifetimeState::Removed);
		}
	}
}

UMaterialExpression* UDMMaterialProperty::CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	float InDefaultValue)
{
	UMaterialExpressionConstant* Constant = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant>(UE_DM_NodeComment_Default);
	Constant->R = InDefaultValue;

	InBuildState->AddOtherExpressions({Constant});

	return Constant;
}

UMaterialExpression* UDMMaterialProperty::CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	const FVector2d& InDefaultValue)
{
	UMaterialExpressionConstant2Vector* Constant = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant2Vector>(UE_DM_NodeComment_Default);
	Constant->R = InDefaultValue.X;
	Constant->G = InDefaultValue.Y;

	InBuildState->AddOtherExpressions({Constant});

	return Constant;
}

UMaterialExpression* UDMMaterialProperty::CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	const FVector3d& InDefaultValue)
{
	UMaterialExpressionConstant3Vector* Constant = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant3Vector>(UE_DM_NodeComment_Default);
	Constant->Constant.R = InDefaultValue.X;
	Constant->Constant.G = InDefaultValue.Y;
	Constant->Constant.B = InDefaultValue.Z;
	Constant->Constant.A = 0.f;

	InBuildState->AddOtherExpressions({Constant});

	return Constant;
}

UMaterialExpression* UDMMaterialProperty::CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	const FVector4d& InDefaultValue)
{
	UMaterialExpressionConstant4Vector* Constant = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant4Vector>(UE_DM_NodeComment_Default);
	Constant->Constant.R = InDefaultValue.X;
	Constant->Constant.G = InDefaultValue.Y;
	Constant->Constant.B = InDefaultValue.Z;
	Constant->Constant.A = InDefaultValue.W;

	InBuildState->AddOtherExpressions({Constant});

	return Constant;
}

#undef LOCTEXT_NAMESPACE
