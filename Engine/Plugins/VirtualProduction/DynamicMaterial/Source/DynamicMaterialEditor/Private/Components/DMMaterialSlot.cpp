// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialSlot.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialLayer_Deprecated.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialSubStage.h"
#include "CoreGlobals.h"
#include "DMComponentPath.h"
#include "DMDefs.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "MaterialValueType.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialSlot)

#define LOCTEXT_NAMESPACE "DMMaterialStage"

const FString UDMMaterialSlot::LayersPathToken = FString(TEXT("Layers"));

bool UDMMaterialSlot::MoveLayer(UDMMaterialLayerObject* InLayer, int32 InNewIndex)
{
	if (!IsComponentValid())
	{
		return false;
	}

	check(InLayer);

	InNewIndex = FMath::Clamp(InNewIndex, 0, LayerObjects.Num() - 1);
	const int32 CurrentIndex = InLayer->FindIndex();

	if (InNewIndex == CurrentIndex)
	{
		return false;
	}

	if (InNewIndex == 0 && !InLayer->IsStageEnabled(EDMMaterialLayerStage::Base))
	{
		if (UDMMaterialStage* Stage = InLayer->GetStage(EDMMaterialLayerStage::Base))
		{
			if (GUndo)
			{
				Stage->Modify();
			}

			Stage->SetEnabled(true);
		}
	}
		
	const int MinIndex = FMath::Min(CurrentIndex, InNewIndex);
	const int MaxIndex = FMath::Max(CurrentIndex, InNewIndex);

	LayerObjects.RemoveAt(CurrentIndex, EAllowShrinking::No); // Don't allow shrinking.
	LayerObjects.Insert(InLayer, InNewIndex);

	for (int32 LayerIndex = MinIndex; LayerIndex <= MaxIndex; ++LayerIndex)
	{
		LayerObjects[LayerIndex]->ForEachValidStage(
			EDMMaterialLayerStage::All,
			[](UDMMaterialStage* InStage)
			{
				if (GUndo)
				{
					InStage->Modify();
				}

				InStage->ResetInputConnectionMap();
			});
	}

	if (InNewIndex == (LayerObjects.Num() - 1))
	{
		UpdateOutputConnectorTypes();
	}

	if (UDMMaterialStage* Stage = LayerObjects[MinIndex]->GetFirstEnabledStage(EDMMaterialLayerStage::All))
	{
		Stage->Update(this, EDMUpdateType::Structure);
	}
	else
	{
		Update(this, EDMUpdateType::Structure);
	}

	OnLayersUpdateDelegate.Broadcast(this);

	return true;
}

bool UDMMaterialSlot::MoveLayerBefore(UDMMaterialLayerObject* InLayer, UDMMaterialLayerObject* InBeforeLayer)
{
	check(InLayer);

	if (InBeforeLayer == nullptr)
	{
		return MoveLayer(InLayer, 0);
	}
	else
	{
		return MoveLayer(InLayer, InBeforeLayer->FindIndex() - 1);
	}
}

bool UDMMaterialSlot::MoveLayerAfter(UDMMaterialLayerObject* InLayer, UDMMaterialLayerObject* InAfterLayer)
{
	check(InLayer);

	if (InAfterLayer == nullptr)
	{
		return MoveLayer(InLayer, LayerObjects.Num());
	}
	else
	{
		return MoveLayer(InLayer, InAfterLayer->FindIndex() + 1);
	}
}

UDMMaterialLayerObject* UDMMaterialSlot::FindLayer(const UDMMaterialStage* InBaseOrMask) const
{
	if (const UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(InBaseOrMask))
	{
		InBaseOrMask = SubStage->GetParentMostStage();
	}

	const TObjectPtr<UDMMaterialLayerObject>* FoundLayer = LayerObjects.FindByPredicate(
		[InBaseOrMask](const TObjectPtr<UDMMaterialLayerObject>& Element)
		{
			return IsValid(Element) && Element->HasValidStage(InBaseOrMask);
		}
	);

	if (FoundLayer)
	{
		return FoundLayer->Get();
	}

	return nullptr;
}

TArray<UDMMaterialLayerObject*> UDMMaterialSlot::BP_GetLayers() const
{
	TArray<UDMMaterialLayerObject*> OutLayerObjects;
	OutLayerObjects.Reserve(LayerObjects.Num());

	Algo::Transform(LayerObjects, OutLayerObjects, [](const TObjectPtr<UDMMaterialLayerObject> InElement) { return InElement; });

	return OutLayerObjects;
}

UDMMaterialLayerObject* UDMMaterialSlot::GetLastLayerForMaterialProperty(EDMMaterialPropertyType InMaterialProperty) const
{
	for (int32 LayerIndex = LayerObjects.Num() - 1; LayerIndex >= 0; --LayerIndex)
	{
		if (!LayerObjects[LayerIndex]->IsStageEnabled(EDMMaterialLayerStage::Base))
		{
			continue;
		}

		if (LayerObjects[LayerIndex]->GetMaterialProperty() != InMaterialProperty)
		{
			continue;
		}

		return LayerObjects[LayerIndex];
	}

	return nullptr;
}

void UDMMaterialSlot::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		MarkComponentDirty();
		UpdateMaterialProperties();
	}

	Super::Update(InSource, InUpdateType);

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
		check(ModelEditorOnlyData);

		ModelEditorOnlyData->RequestMaterialBuild();
	}
}

void UDMMaterialSlot::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	Super::OnComponentAdded();

	for (const TObjectPtr<UDMMaterialLayerObject>& LayerObj : LayerObjects)
	{
		if (GUndo)
		{
			LayerObj->Modify();
		}

		LayerObj->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialSlot::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	for (const TObjectPtr<UDMMaterialLayerObject>& LayerObj : LayerObjects)
	{
		if (GUndo)
		{
			LayerObj->Modify();
		}

		LayerObj->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

void UDMMaterialSlot::UpdateOutputConnectorTypes()
{
	if (!IsComponentValid())
	{
		return;
	}

	OutputConnectorTypes.Empty();

	if (LayerObjects.IsEmpty())
	{
		return;
	}

	TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> LastOutputForProperty;

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		LastOutputForProperty.FindOrAdd(Layer->GetMaterialProperty()) = Layer;
	}

	for (const TPair<EDMMaterialPropertyType, UDMMaterialLayerObject*>& PropStage : LastOutputForProperty)
	{
		if (UDMMaterialStage* Mask = PropStage.Value->GetStage(EDMMaterialLayerStage::Mask, /* Enabled Only */ true))
		{
			if (UDMMaterialStageSource* Source = Mask->GetSource())
			{
				TArray<EDMValueType> MaterialPropertyORutputConnectorTypes;
				const TArray<FDMMaterialStageConnector>& LastStageOutputConnectors = Source->GetOutputConnectors();

				for (const FDMMaterialStageConnector& Connector : LastStageOutputConnectors)
				{
					MaterialPropertyORutputConnectorTypes.Add(Connector.Type);
				}

				FDMMaterialSlotOutputConnectorTypes ConnectorTypes = {MaterialPropertyORutputConnectorTypes};
				OutputConnectorTypes.Emplace(PropStage.Key, ConnectorTypes);
			}
		}
	}

	UpdateMaterialProperties();

	OnConnectorsUpdateDelegate.Broadcast(this);
}

void UDMMaterialSlot::UpdateMaterialProperties()
{
	if (!IsComponentValid())
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	if (LayerObjects.IsEmpty())
	{
		return;
	}

	TSet<EDMMaterialPropertyType> CurrentStageMaterialProperties;

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();

		if (StageProperty != EDMMaterialPropertyType::None && StageProperty != EDMMaterialPropertyType::Any)
		{
			CurrentStageMaterialProperties.Add(StageProperty);
		}
	}

	const TArray<EDMMaterialPropertyType> CurrentSlotMaterialProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(this);

	for (EDMMaterialPropertyType MaterialProperty : CurrentSlotMaterialProperties)
	{
		if (!CurrentStageMaterialProperties.Contains(MaterialProperty))
		{
			UDMMaterialSlot* CurrentSlot = ModelEditorOnlyData->GetSlotForMaterialProperty(MaterialProperty);
			check(CurrentSlot == nullptr || CurrentSlot == this);

			if (CurrentSlot == this)
			{
				if (GUndo)
				{
					ModelEditorOnlyData->Modify();
				}

				ModelEditorOnlyData->UnassignMaterialProperty(MaterialProperty);
			}
		}
	}

	for (EDMMaterialPropertyType MaterialProperty : CurrentStageMaterialProperties)
	{
		if (!CurrentSlotMaterialProperties.Contains(MaterialProperty))
		{
			UDMMaterialSlot* CurrentSlot = ModelEditorOnlyData->GetSlotForMaterialProperty(MaterialProperty);
			check(CurrentSlot == nullptr || CurrentSlot == this);

			if (CurrentSlot == nullptr)
			{
				if (GUndo)
				{
					ModelEditorOnlyData->Modify();
				}

				ModelEditorOnlyData->AssignMaterialPropertyToSlot(MaterialProperty, this);
			}
		}
	}
}

UDMMaterialSlot::UDMMaterialSlot()
	: Index(INDEX_NONE)
{
}

UDynamicMaterialModelEditorOnlyData* UDMMaterialSlot::GetMaterialModelEditorOnlyData() const
{
	return Cast<UDynamicMaterialModelEditorOnlyData>(GetOuterSafe());
}

FText UDMMaterialSlot::GetDescription() const
{
	static const FText Template = LOCTEXT("StageInputSlotTempate", "Slot {0}");

	return FText::Format(Template, FText::AsNumber(Index));
}

UDMMaterialLayerObject* UDMMaterialSlot::GetLayer(int32 InLayerIndex) const
{
	if (!LayerObjects.IsValidIndex(InLayerIndex))
	{
		return nullptr;
	}

	return LayerObjects[InLayerIndex];
}

const TArray<EDMValueType>& UDMMaterialSlot::GetOutputConnectorTypesForMaterialProperty(EDMMaterialPropertyType InMaterialProperty) const
{
	static const TArray<EDMValueType> NoConnectors;

	const FDMMaterialSlotOutputConnectorTypes* ConnectorTypes = OutputConnectorTypes.Find(InMaterialProperty);

	if (!ConnectorTypes)
	{
		return NoConnectors;
	}

	return ConnectorTypes->ConnectorTypes;
}

TSet<EDMValueType> UDMMaterialSlot::GetAllOutputConnectorTypes() const
{
	TSet<EDMValueType> AllOutputTypes;

	for (const TPair<EDMMaterialPropertyType, FDMMaterialSlotOutputConnectorTypes>& OutputTypes : OutputConnectorTypes)
	{
		for (EDMValueType OutputType : OutputTypes.Value.ConnectorTypes)
		{
			AllOutputTypes.Emplace(OutputType);
		}
	}

	return AllOutputTypes;
}

UDMMaterialLayerObject* UDMMaterialSlot::AddDefaultLayer(EDMMaterialPropertyType InMaterialProperty)
{
	if (!IsComponentValid())
	{
		return nullptr;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDMMaterialProperty* Property = ModelEditorOnlyData->GetMaterialProperty(InMaterialProperty);
	check(Property);

	UDMMaterialLayerObject* NewLayer = UDMMaterialLayerObject::CreateLayer(this, InMaterialProperty, {});
	LayerObjects.Add(NewLayer);

	if (IsComponentAdded())
	{
		NewLayer->SetComponentState(EDMComponentLifetimeState::Added);
	}

	{
		const FDMUpdateGuard Guard;
		Property->AddDefaultBaseStage(NewLayer);
		Property->AddDefaultMaskStage(NewLayer);
	}

	UpdateOutputConnectorTypes();

	NewLayer->Update(this, EDMUpdateType::Structure);

	OnLayersUpdateDelegate.Broadcast(this);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlot::AddLayer(EDMMaterialPropertyType InMaterialProperty, UDMMaterialStage* InNewBase)
{
	if (!IsComponentValid())
	{
		return nullptr;
	}

	check(InNewBase);
	check(InNewBase->GetSource());
	check(InNewBase->GetSource()->GetOutputConnectors().IsEmpty() == false);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDMMaterialProperty* Property = ModelEditorOnlyData->GetMaterialProperty(InMaterialProperty);
	check(Property);

	if (GUndo)
	{
		InNewBase->Modify();
	}

	UDMMaterialLayerObject* NewLayer = UDMMaterialLayerObject::CreateLayer(this, InMaterialProperty, {InNewBase});
	LayerObjects.Add(NewLayer);

	if (IsComponentAdded())
	{
		NewLayer->SetComponentState(EDMComponentLifetimeState::Added);
	}

	{
		const FDMUpdateGuard Guard;
		Property->AddDefaultMaskStage(NewLayer);
	}

	UpdateOutputConnectorTypes();

	NewLayer->Update(this, EDMUpdateType::Structure);

	OnLayersUpdateDelegate.Broadcast(this);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlot::AddLayerWithMask(EDMMaterialPropertyType InMaterialProperty, UDMMaterialStage* InNewBase, 
	UDMMaterialStage* InNewMask)
{
	if (!IsComponentValid())
	{
		return nullptr;
	}

	check(InNewBase);
	check(InNewBase->GetSource());
	check(InNewBase->GetSource()->GetOutputConnectors().IsEmpty() == false);

	check(InNewMask);
	check(InNewMask->GetSource());
	check(InNewMask->GetSource()->GetOutputConnectors().IsEmpty() == false);

	if (GUndo)
	{
		InNewBase->Modify();
		InNewMask->Modify();
	}

	UDMMaterialLayerObject* NewLayer = UDMMaterialLayerObject::CreateLayer(this, InMaterialProperty, {InNewBase, InNewMask});
	LayerObjects.Add(NewLayer);

	if (IsComponentAdded())
	{
		NewLayer->SetComponentState(EDMComponentLifetimeState::Added);
	}

	UpdateOutputConnectorTypes();

	NewLayer->Update(this, EDMUpdateType::Structure);

	OnLayersUpdateDelegate.Broadcast(this);

	return NewLayer;
}

bool UDMMaterialSlot::PasteLayer(UDMMaterialLayerObject* InLayer)
{
	if (!InLayer)
	{
		return false;
	}

	if (GUndo)
	{
		InLayer->Modify();
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();

	if (!ModelEditorOnlyData)
	{
		return false;
	}

	EDMMaterialPropertyType NewPropertyType = EDMMaterialPropertyType::None;

	if (!LayerObjects.IsEmpty())
	{
		for (UDMMaterialLayerObject* CurrentLayer : UE::Core::Private::TReverseIterationAdapter(LayerObjects))
		{
			if (CurrentLayer->IsEnabled())
			{
				NewPropertyType = CurrentLayer->GetMaterialProperty();
				break;
			}
		}

		if (NewPropertyType == EDMMaterialPropertyType::None)
		{
			if (LayerObjects.Num() > 0)
			{
				NewPropertyType = LayerObjects.Last()->GetMaterialProperty();
			}
		}
	}

	if (NewPropertyType == EDMMaterialPropertyType::None)
	{
		TArray<EDMMaterialPropertyType> SlotProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(this);

		if (SlotProperties.IsEmpty())
		{
			return false;
		}

		NewPropertyType = SlotProperties[0];
	}

	{
		const FDMUpdateGuard Guard;
		InLayer->SetMaterialProperty(NewPropertyType);

		UDynamicMaterialModel* MaterialModel = nullptr;

		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetMaterialModelEditorOnlyData())
		{
			MaterialModel = EditorOnlyData->GetMaterialModel();
		}

		LayerObjects.Add(InLayer);

		InLayer->PostEditorDuplicate(MaterialModel, this);
	}

	if (IsComponentAdded())
	{
		InLayer->SetComponentState(EDMComponentLifetimeState::Added);
	}

	UpdateOutputConnectorTypes();

	if (UDMMaterialStage* Stage = InLayer->GetFirstEnabledStage(EDMMaterialLayerStage::All))
	{
		Stage->Update(this, EDMUpdateType::Structure);
	}
	else
	{
		Update(this, EDMUpdateType::Structure);
	}

	OnLayersUpdateDelegate.Broadcast(this);

	return true;
}

bool UDMMaterialSlot::CanRemoveLayer(UDMMaterialLayerObject* InLayer) const
{
	if (!IsComponentValid())
	{
		return false;
	}

	check(InLayer);
	check(InLayer->GetSlot() == this);

	const EDMMaterialPropertyType LayerProperty = InLayer->GetMaterialProperty();
	int32 LayerPropertyCount = 0;

	for (UDMMaterialLayerObject* Layer : LayerObjects)
	{
		if (Layer->GetMaterialProperty() == LayerProperty)
		{
			++LayerPropertyCount;
		}
	}

	if (LayerPropertyCount == 1)
	{
		return false;
	}

	return InLayer->FindIndex() != INDEX_NONE;
}

bool UDMMaterialSlot::RemoveLayer(UDMMaterialLayerObject* InLayer)
{
	if (!CanRemoveLayer(InLayer))
	{
		return false;
	}

	const int32 LayerIndex = InLayer->FindIndex();

	if (LayerIndex == INDEX_NONE)
	{
		return false;
	}

	LayerObjects.RemoveAt(LayerIndex);

	if (LayerIndex == 0 && LayerObjects.IsEmpty() == false)
	{
		if (UDMMaterialStage* Stage = LayerObjects[0]->GetStage(EDMMaterialLayerStage::Base))
		{
			if (GUndo)
			{
				Stage->Modify();
			}

			Stage->SetEnabled(true);
		}
	}

	if (GUndo)
	{
		InLayer->Modify();
	}

	InLayer->SetComponentState(EDMComponentLifetimeState::Removed);

	if (!LayerObjects.IsEmpty())
	{
		if (UDMMaterialStage* Stage = LayerObjects[0]->GetFirstEnabledStage(EDMMaterialLayerStage::All))
		{
			Stage->Update(this, EDMUpdateType::Structure);
		}
		else
		{
			Update(this, EDMUpdateType::Structure);
		}
	}

	OnLayersUpdateDelegate.Broadcast(this);

	return true;
}

void UDMMaterialSlot::OnPropertiesUpdated()
{
	OnPropertiesUpdateDelegate.Broadcast(this);
}

void UDMMaterialSlot::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasSlot(this) || LayerObjects.IsEmpty())
	{
		return;
	}

	TArray<UMaterialExpression*> SlotExpressions;
	TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>> SlotPropertyExpressions;

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		if (!Layer->IsEnabled())
		{
			continue;
		}

		Layer->GenerateExpressions(InBuildState);

		const TArray<UMaterialExpression*>& LayerExpressions = InBuildState->GetLayerExpressions(Layer);

		if (LayerExpressions.IsEmpty())
		{
			continue;
		}

		SlotExpressions.Append(LayerExpressions);
		SlotPropertyExpressions.FindOrAdd(Layer->GetMaterialProperty()).Append(LayerExpressions);
	}

	if (SlotExpressions.IsEmpty())
	{
		return;
	}

	InBuildState->AddSlotExpressions(this, SlotExpressions);
	InBuildState->AddSlotPropertyExpressions(this, SlotPropertyExpressions);
}

TArray<UDMMaterialSlot*> UDMMaterialSlot::K2_GetSlotsReferencedBy() const
{
	TArray<TWeakObjectPtr<UDMMaterialSlot>> WeakKeys;
	SlotsReferencedBy.GetKeys(WeakKeys);

	TArray<UDMMaterialSlot*> Keys;
	Keys.SetNum(Keys.Num());

	for (int32 KeyIndex = 0; KeyIndex < WeakKeys.Num(); ++KeyIndex)
	{
		Keys[KeyIndex] = WeakKeys[KeyIndex].Get();
	}

	return Keys;
}

bool UDMMaterialSlot::ReferencedBySlot(UDMMaterialSlot* InOtherSlot)
{
	if (!IsComponentValid())
	{
		return false;
	}

	check(InOtherSlot);
	check(InOtherSlot != this);

	if (int32* CountPtr = SlotsReferencedBy.Find(InOtherSlot))
	{
		++(*CountPtr);
		return false;
	}
	else
	{
		SlotsReferencedBy.Emplace(InOtherSlot, 1);
		OnPropertiesUpdateDelegate.Broadcast(this);
		return true;
	}
}

bool UDMMaterialSlot::UnreferencedBySlot(UDMMaterialSlot* InOtherSlot)
{
	if (!IsComponentValid())
	{
		return false;
	}

	check(InOtherSlot);
	check(InOtherSlot != this);

	int32* CountPtr = SlotsReferencedBy.Find(InOtherSlot);
	check(CountPtr);

	--(*CountPtr);

	if ((*CountPtr) == 0)
	{
		SlotsReferencedBy.Remove(InOtherSlot);
		OnPropertiesUpdateDelegate.Broadcast(this);
		return true;
	}
	else
	{
		return false;
	}
}

bool UDMMaterialSlot::SetLayerMaterialPropertyAndReplaceOthers(UDMMaterialLayerObject* InLayer, EDMMaterialPropertyType InPropertyFrom, 
	EDMMaterialPropertyType InPropertyTo)
{
	if (!IsComponentValid())
	{
		return false;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDMMaterialSlot* CurrentSlot = ModelEditorOnlyData->GetSlotForMaterialProperty(InPropertyFrom);

	if (!CurrentSlot || CurrentSlot == this)
	{
		if (GUndo)
		{
			InLayer->Modify();
		}

		InLayer->SetMaterialProperty(InPropertyFrom);
		return false; // Could be caused by asynchronous input
	}

	{
		FDMUpdateGuard Guard;
		
		for (TObjectPtr<UDMMaterialLayerObject>& Layer : CurrentSlot->LayerObjects)
		{
			if (Layer->GetMaterialProperty() == InPropertyFrom)
			{
				if (GUndo)
				{
					Layer->Modify();
				}

				Layer->SetMaterialProperty(InPropertyTo);
			}

			if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base))
			{
				TArray<FDMMaterialStageConnection>& CurrentSlotStageInputMap = BaseStage->GetInputConnectionMap();

				for (int32 InputIdx = 0; InputIdx < CurrentSlotStageInputMap.Num(); ++InputIdx)
				{
					for (int32 ChannelIdx = 0; ChannelIdx < CurrentSlotStageInputMap[InputIdx].Channels.Num(); ++ChannelIdx)
					{
						FDMMaterialStageConnectorChannel& Channel = CurrentSlotStageInputMap[InputIdx].Channels[ChannelIdx];

						if (Channel.SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE
							&& Channel.MaterialProperty == InPropertyFrom)
						{
							// Delve into class internals, avoiding the const issues above.
							Channel.MaterialProperty = InPropertyTo;
						}
					}
				}
			}
		}
	}

	if (!CurrentSlot->LayerObjects.IsEmpty())
	{
		if (UDMMaterialStage* Stage = CurrentSlot->LayerObjects[0]->GetFirstEnabledStage(EDMMaterialLayerStage::All))
		{
			Stage->Update(this, EDMUpdateType::Structure);
		}
	}

	return InLayer->SetMaterialProperty(InPropertyFrom);
}

bool UDMMaterialSlot::ChangeMaterialProperty(EDMMaterialPropertyType InPropertyFrom, EDMMaterialPropertyType InReplaceWithProperty)
{
	if (!IsComponentValid())
	{
		return false;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	FDMUpdateGuard Guard;
	UDMMaterialLayerObject* FirstLayerObject = nullptr;

	for (TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		if (Layer->GetMaterialProperty() == InPropertyFrom)
		{
			if (GUndo)
			{
				Layer->Modify();
			}

			Layer->SetMaterialProperty(InReplaceWithProperty);

			if (!FirstLayerObject)
			{
				FirstLayerObject = Layer;
			}
		}

		if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base))
		{
			TArray<FDMMaterialStageConnection>& CurrentSlotStageInputMap = BaseStage->GetInputConnectionMap();

			for (int32 InputIdx = 0; InputIdx < CurrentSlotStageInputMap.Num(); ++InputIdx)
			{
				for (int32 ChannelIdx = 0; ChannelIdx < CurrentSlotStageInputMap[InputIdx].Channels.Num(); ++ChannelIdx)
				{
					FDMMaterialStageConnectorChannel& Channel = CurrentSlotStageInputMap[InputIdx].Channels[ChannelIdx];

					if (Channel.SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE
						&& Channel.MaterialProperty == InPropertyFrom)
					{
						// Delve into class internals, avoiding the const issues above.
						Channel.MaterialProperty = InReplaceWithProperty;
					}
				}
			}
		}
	}

	ModelEditorOnlyData->UnassignMaterialProperty(InPropertyFrom);
	ModelEditorOnlyData->AssignMaterialPropertyToSlot(InReplaceWithProperty, this);

	if (const FDMMaterialSlotOutputConnectorTypes* ConnectorTypes = OutputConnectorTypes.Find(InPropertyFrom))
	{
		OutputConnectorTypes.Emplace(InReplaceWithProperty, *ConnectorTypes);
		OutputConnectorTypes.Remove(InPropertyFrom);
	}

	if (FirstLayerObject)
	{
		FirstLayerObject->Update(this, EDMUpdateType::Structure);
	}

	return true;
}

FString UDMMaterialSlot::GetComponentPathComponent() const
{
	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetMaterialModelEditorOnlyData())
	{
		TArray<EDMMaterialPropertyType> SlotProperties = EditorOnlyData->GetMaterialPropertiesForSlot(this);

		if (SlotProperties.Num() == 1)
		{
			const FText Token = UE::DynamicMaterialEditor::Private::GetMaterialPropertyShortDisplayName(SlotProperties[0]);

			return FString::Printf(
				TEXT("%s%c%s%c"),
				*UDynamicMaterialModelEditorOnlyData::SlotsPathToken,
				FDMComponentPath::ParameterOpen,
				*Token.ToString(),
				FDMComponentPath::ParameterClose
			);
		}
	}

	return FString::Printf(
		TEXT("%s%c%i%c"),
		*UDynamicMaterialModelEditorOnlyData::SlotsPathToken,
		FDMComponentPath::ParameterOpen,
		Index,
		FDMComponentPath::ParameterClose
	);
}

UDMMaterialComponent* UDMMaterialSlot::GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == LayersPathToken)
	{
		int32 LayerIndex = INDEX_NONE;

		if (InPathSegment.GetParameter(LayerIndex))
		{
			if (LayerObjects.IsValidIndex(LayerIndex))
			{
				return LayerObjects[LayerIndex];
			}
		}
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialSlot::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(InMaterialModel);

	if (ModelEditorOnlyData && GetOuter() != ModelEditorOnlyData)
	{
		Rename(nullptr, ModelEditorOnlyData, UE::DynamicMaterial::RenameFlags);
	}

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		Layer->PostEditorDuplicate(InMaterialModel, this);
	}
}

bool UDMMaterialSlot::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		Layer->Modify(bInAlwaysMarkDirty);
	}

	return bSaved;
}

void UDMMaterialSlot::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : LayerObjects)
	{
		if (GUndo)
		{
			Layer->Modify();
		}

		Layer->PostEditorDuplicate(MaterialModel, this);
	}

	MarkComponentDirty();

	Update(this, EDMUpdateType::Structure);

	// Fire all of these to make sure everything is updated.
	OnPropertiesUpdateDelegate.Broadcast(this);
	OnLayersUpdateDelegate.Broadcast(this);

	UpdateOutputConnectorTypes();
}

void UDMMaterialSlot::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (!Layers.IsEmpty())
	{
		ConvertDeprecatedLayers(Layers);
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDMMaterialSlot::ConvertDeprecatedLayers(TArray<FDMMaterialLayer>& InLayers)
{
	FDMUpdateGuard Guard;

	MarkPackageDirty();

	UDynamicMaterialModel* MaterialModel = GetMaterialModelEditorOnlyData()->GetMaterialModel();

	for (const FDMMaterialLayer& Layer : InLayers)
	{
		UDMMaterialLayerObject* NewLayer = AddLayerWithMask(Layer.MaterialProperty, Layer.Base, Layer.Mask);
		NewLayer->SetLayerName(Layer.LayerName);
		NewLayer->SetEnabled(Layer.bEnabled);
		NewLayer->SetTextureUVLinkEnabled(Layer.bLinkedUVs);

		if (Layer.Base)
		{
			Layer.Base->SetEnabled(Layer.bBaseEnabled);
		}

		if (Layer.Mask)
		{
			Layer.Mask->SetEnabled(Layer.bMaskEnabled);
		}

		NewLayer->PostEditorDuplicate(MaterialModel, this);
	}

	InLayers.Empty();	

	if (!LayerObjects.IsEmpty())
	{
		LayerObjects[0]->Update(this, EDMUpdateType::Structure);
	}
	else
	{
		Update(this, EDMUpdateType::Structure);
	}
}

#undef LOCTEXT_NAMESPACE
