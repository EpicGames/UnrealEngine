// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStage.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStageInput.h"
#include "Components/DMMaterialStageSource.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "DMComponentPath.h"
#include "DynamicMaterialEditorModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialValueType.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UObject/Package.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStage)

#define LOCTEXT_NAMESPACE "DMMaterialStage"

const FString UDMMaterialStage::SourcePathToken = FString(TEXT("Source"));
const FString UDMMaterialStage::InputsPathToken = FString(TEXT("Inputs"));

UDMMaterialStage* UDMMaterialStage::CreateMaterialStage(UDMMaterialLayerObject* InLayer)
{
	UObject* Outer = IsValid(InLayer) ? (UObject*)InLayer : (UObject*)GetTransientPackage();

	return NewObject<UDMMaterialStage>(Outer, NAME_None, RF_Transactional);
}

UDMMaterialStage::UDMMaterialStage()
	: Source(nullptr)
	, bEnabled(true)
	, bCanChangeSource(true)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStage, Source));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStage, Inputs));
}

void UDMMaterialStage::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	Super::OnComponentAdded();

	ResetInputConnectionMap();

	for (UDMMaterialStageInput* Input : Inputs)
	{
		Input->SetComponentState(EDMComponentLifetimeState::Added);
	}
		
	if (Source)
	{
		if (GUndo)
		{
			Source->Modify();
		}

		Source->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialStage::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	for (UDMMaterialStageInput* Input : Inputs)
	{
		Input->SetComponentState(EDMComponentLifetimeState::Removed);
	}
		
	if (Source)
	{
		if (GUndo)
		{
			Source->Modify();
		}

		Source->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

UDMMaterialComponent* UDMMaterialStage::GetParentComponent() const
{
	return GetLayer();
}

void UDMMaterialStage::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (GetOuter() != InParent)
	{
		Rename(nullptr, InParent, UE::DynamicMaterial::RenameFlags);
	}

	if (Source)
	{
		if (GUndo)
		{
			Source->Modify();
		}

		Source->PostEditorDuplicate(InMaterialModel, this);
	}

	for (const TObjectPtr<UDMMaterialStageInput>& Input : Inputs)
	{
		if (Input)
		{
			Input->PostEditorDuplicate(InMaterialModel, this);
		}
	}
}

bool UDMMaterialStage::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	if (Source)
	{
		Source->Modify(bInAlwaysMarkDirty);
	}

	for (const TObjectPtr<UDMMaterialStageInput>& Input : Inputs)
	{
		if (Input)
		{
			Input->Modify(bInAlwaysMarkDirty);
		}
	}

	return bSaved;
}

FString UDMMaterialStage::GetComponentPathComponent() const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		FString TypeStr = TEXT("?");

		switch (Layer->GetStageType(this))
		{
			case EDMMaterialLayerStage::Base:
				TypeStr = UDMMaterialLayerObject::BasePathToken;
				break;

			case EDMMaterialLayerStage::Mask:
				TypeStr = UDMMaterialLayerObject::MaskPathToken;
				break;

			default:
				TypeStr = FString::FromInt(Layer->GetAllStages().IndexOfByKey(this));
				break;
		}

		return FString::Printf(
			TEXT("%s%c%s%c"),
			*UDMMaterialLayerObject::StagesPathToken,
			FDMComponentPath::ParameterOpen,
			*TypeStr,
			FDMComponentPath::ParameterClose
		);
	}

	return Super::GetComponentPathComponent();
}

void UDMMaterialStage::GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const
{
	// Strip off the type index of the substage
	if (OutChildComponentPathComponents.IsEmpty() == false && Source)
	{
		if (OutChildComponentPathComponents.Last() == Source->GetComponentPathComponent())
		{
			OutChildComponentPathComponents.Last() = Source->GetClass()->GetName();
		}
	}

	Super::GetComponentPathInternal(OutChildComponentPathComponents);
}

UDMMaterialComponent* UDMMaterialStage::GetSubComponentByPath(FDMComponentPath& InPath,
	const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == SourcePathToken)
	{
		return Source;
	}

	if (InPathSegment.GetToken() == InputsPathToken)
	{
		int32 InputIndex = INDEX_NONE;
		FString InputType;

		if (InPathSegment.GetParameter(InputIndex))
		{
			if (Inputs.IsValidIndex(InputIndex))
			{
				return Inputs[InputIndex];
			}
		}
		else if (InPathSegment.GetParameter(InputType))
		{
			for (UDMMaterialStageInput* Input : Inputs)
			{
				if (Input)
				{
					const FString InputClassName = Input->GetClass()->GetName();
					
					if (InputClassName.Equals(InputType) || InputClassName.Equals(UDMMaterialStageInput::StageInputPrefixStr + InputType))
					{
						return Input;
					}
				}
			}
		}
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

UDMMaterialLayerObject* UDMMaterialStage::GetLayer() const
{
	return Cast<UDMMaterialLayerObject>(GetOuterSafe());
}

bool UDMMaterialStage::SetEnabled(bool bInEnabled)
{
	if (bEnabled == bInEnabled)
	{
		return false;
	}

	bEnabled = bInEnabled;

	Update(this, EDMUpdateType::Structure);

	return true;
}

void UDMMaterialStage::SetSource(UDMMaterialStageSource* InSource)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (!bCanChangeSource)
	{
		return;
	}

	if (Source)
	{
		if (GUndo)
		{
			Source->Modify();
		}

		Source->SetComponentState(EDMComponentLifetimeState::Removed);
	}

	Source = InSource;

	ResetInputConnectionMap();

	if (IsComponentAdded())
	{
		if (GUndo)
		{
			Source->Modify();
		}

		Source->SetComponentState(EDMComponentLifetimeState::Added);
	}

	Update(this, EDMUpdateType::Structure);
}

FText UDMMaterialStage::GetComponentDescription() const
{
	if (Source)
	{
		if (Source->IsComponentValid())
		{
			return Source->GetStageDescription();
		}
	}

	return LOCTEXT("StageDescription", "Material Stage");
}

FSlateIcon UDMMaterialStage::GetComponentIcon() const
{
	if (Source && Source->IsComponentValid())
	{
		return Source->GetComponentIcon();
	}

	return Super::GetComponentIcon();
}

EDMValueType UDMMaterialStage::GetSourceType(const FDMMaterialStageConnectorChannel& InChannel) const
{
	if (InChannel.SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
	{
		UDMMaterialLayerObject* Layer = GetLayer();
		check(Layer);

		EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
		check(StageProperty != EDMMaterialPropertyType::None && StageProperty != EDMMaterialPropertyType::Any);

		UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base);
		check(PreviousLayer);
		check(PreviousLayer->GetStage(EDMMaterialLayerStage::Mask));
		check(PreviousLayer->GetStage(EDMMaterialLayerStage::Mask)->GetSource());

		const TArray<FDMMaterialStageConnector>& OutputConnectors = PreviousLayer->GetStage(EDMMaterialLayerStage::Mask)->GetSource()->GetOutputConnectors();
		check(OutputConnectors.IsValidIndex(InChannel.OutputIndex));

		return OutputConnectors[InChannel.OutputIndex].Type;
	}
	else
	{
		const int32 StageInputIdx = InChannel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
		check(Inputs.IsValidIndex(StageInputIdx));

		const TArray<FDMMaterialStageConnector>& OutputConnectors = Inputs[StageInputIdx]->GetOutputConnectors();
		check(OutputConnectors.IsValidIndex(InChannel.OutputIndex));

		return OutputConnectors[InChannel.OutputIndex].Type;
	}
}

bool UDMMaterialStage::IsInputMapped(int32 InputIndex) const
{
	if (!InputConnectionMap.IsValidIndex(InputIndex))
	{
		return false;
	}

	for (const FDMMaterialStageConnectorChannel& Channel : InputConnectionMap[InputIndex].Channels)
	{
		if (Channel.SourceIndex != FDMMaterialStageConnectorChannel::NO_SOURCE)
		{
			return true;
		}
	}

	return false;
}

void UDMMaterialStage::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		MarkComponentDirty();
		VerifyAllInputMaps();
	}

	Super::Update(InSource, InUpdateType);

	if (UDMMaterialStage* NextStage = GetNextStage())
	{
		NextStage->Update(InSource, InUpdateType);
	}
	else if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		Layer->Update(InSource, InUpdateType);
	}	
}

void UDMMaterialStage::InputUpdated(UDMMaterialStageInput* InInput, EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);

	if (!Throughput)
	{
		return;
	}

	int32 InputIdx = Inputs.Find(InInput);

	if (InputIdx == INDEX_NONE)
	{
		return;
	}

	InputIdx += FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

	for (int32 InputMapIdx = 0; InputMapIdx < Inputs.Num() && InputMapIdx < InputConnectionMap.Num(); ++InputMapIdx)
	{
		bool bIsUsedInInput = false;

		for (const FDMMaterialStageConnectorChannel& Channel : InputConnectionMap[InputMapIdx].Channels)
		{
			if (Channel.SourceIndex == InputIdx)
			{
				bIsUsedInInput = true;
				break;
			}
		}

		if (bIsUsedInInput)
		{
			Throughput->OnInputUpdated(InputMapIdx, InUpdateType);
		}
	}
}

void UDMMaterialStage::ResetInputConnectionMap()
{
	if (!IsComponentValid())
	{
		return;
	}

	VerifyAllInputMaps();
}

void UDMMaterialStage::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (!IsValid(Source))
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "Stage with no source attempted to generate material expressions.");
		return;
	}

	if (InBuildState->HasStage(this))
	{
		return;
	}

	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return;
	}

	TArray<UMaterialExpression*> StageExpressions;

	Source->GenerateExpressions(InBuildState);
	const TArray<UMaterialExpression*>& StageSourceExpressions = InBuildState->GetStageSourceExpressions(Source);

	if (!StageSourceExpressions.IsEmpty())
	{
		if (UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source))
		{
			const TArray<FDMMaterialStageConnector>& ThroughputInputs = Throughput->GetInputConnectors();

			for (int32 ThroughputInputIdx = 0; ThroughputInputIdx < ThroughputInputs.Num() && ThroughputInputIdx < InputConnectionMap.Num(); ++ThroughputInputIdx)
			{
				FDMMaterialStageConnectorChannel Channel;
				TArray<UMaterialExpression*> Expressions;

				const int32 OutputChannelOverride = Throughput->GetOutputChannelOverride(Channel.OutputIndex);

				if (OutputChannelOverride != INDEX_NONE)
				{
					Channel.OutputChannel = OutputChannelOverride;
				}

				const int32 NodeOutputIndex = Throughput->ResolveInput(
					InBuildState, 
					ThroughputInputIdx, 
					Channel, 
					Expressions
				);
				
				if (!Expressions.IsEmpty() && NodeOutputIndex != INDEX_NONE)
				{
					StageExpressions.Append(Expressions);

					Throughput->ConnectOutputToInput(
						InBuildState, 
						ThroughputInputIdx,
						ThroughputInputs[ThroughputInputIdx].Index, 
						Expressions.Last(), 
						NodeOutputIndex, 
						Channel.OutputChannel
					);
				}
			}
		}

		StageExpressions.Append(InBuildState->GetStageSourceExpressions(Source));		
	}

	if (InBuildState->GetPreviewObject() == this)
	{
		int32 OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
		int32 OutputIndex = 0;

		const UDMMaterialStage* Stage = this;

		if (const UDMMaterialSubStage* SubStage = Cast<const UDMMaterialSubStage>(Stage))
		{
			Stage = SubStage->GetParentMostStage();
		}

		Layer->ApplyEffects(
			InBuildState,
			Stage,
			StageExpressions,
			OutputChannel,
			OutputIndex
		);
	}

	InBuildState->AddStageExpressions(this, StageExpressions);
}

TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> UDMMaterialStage::GetPreviousStagesPropertyMap()
{
	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return {};
	}

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> PropertyMap;
	const TArray<TObjectPtr<UDMMaterialLayerObject>> Layers = Slot->GetLayers();

	for (UDMMaterialLayerObject* LayerIter : Layers)
	{
		if (LayerIter == Layer)
		{
			break;
		}

		PropertyMap.FindOrAdd(LayerIter->GetMaterialProperty()) = LayerIter;
	}

	return PropertyMap;
}

TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> UDMMaterialStage::GetPropertyMap()
{
	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return {};
	}

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> PropertyMap;
	const TArray<TObjectPtr<UDMMaterialLayerObject>> Layers = Slot->GetLayers();

	for (UDMMaterialLayerObject* LayerIter : Layers)
	{
		PropertyMap.FindOrAdd(LayerIter->GetMaterialProperty()) = LayerIter;

		if (Layer->HasValidStage(this))
		{
			break;
		}
	}

	return PropertyMap;
}

void UDMMaterialStage::RemoveUnusedInputs()
{
	if (!IsComponentValid())
	{
		return;
	}

	TArray<UDMMaterialStageInput*> UnusedInputs = Inputs;

	for (const FDMMaterialStageConnection& Connection : InputConnectionMap)
	{
		for (const FDMMaterialStageConnectorChannel& Channel : Connection.Channels)
		{
			if (Channel.SourceIndex == 0)
			{
				continue;
			}

			const int32 InputIdx = Channel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			if (!Inputs.IsValidIndex(InputIdx))
			{
				continue;
			}

			UnusedInputs.Remove(Inputs[InputIdx]);
		}
	}

	for (UDMMaterialStageInput* Input : UnusedInputs)
	{
		int32 InputIdx;

		if (!Inputs.Find(Input, InputIdx))
		{
			continue;
		}

		InputIdx += FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

		for (FDMMaterialStageConnection& Connection : InputConnectionMap)
		{
			for (int32 ChannelIdx = 0; ChannelIdx < Connection.Channels.Num(); ++ChannelIdx)
			{
				// Delete channel if we're using a deleted input
				if (Connection.Channels[ChannelIdx].SourceIndex == InputIdx)
				{
					Connection.Channels.RemoveAt(ChannelIdx);
					--ChannelIdx;
					continue;
				}

				if (Connection.Channels[ChannelIdx].SourceIndex > InputIdx)
				{
					Connection.Channels[ChannelIdx].SourceIndex -= 1;
				}
			}
		}

		InputIdx -= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
		Inputs.RemoveAt(InputIdx);

		Input->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

UDMMaterialStageInput* UDMMaterialStage::ChangeInput(TSubclassOf<UDMMaterialStageInput> InInputClass, int32 InInputIdx,
	int32 InInputChannel, int32 InOutputIdx, int32 InOutputChannel, FInputInitFunctionPtr InPreInit)
{
	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));

	check(InInputClass.Get());
	check(!InInputClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	if (UDMMaterialStageThroughput* ThroughputCDO = Cast<UDMMaterialStageThroughput>(InInputClass->GetDefaultObject(true)))
	{
		check(!ThroughputCDO->IsInputRequired() || ThroughputCDO->AllowsNestedInputs());

		const TArray<FDMMaterialStageConnector>& OutputConnectors = ThroughputCDO->GetOutputConnectors();
		check(Throughput->CanInputConnectTo(InInputIdx, OutputConnectors[InOutputIdx], InOutputChannel));
	}

	UDMMaterialStageInput* NewInput = NewObject<UDMMaterialStageInput>(this, InInputClass, NAME_None, RF_Transactional);
	check(NewInput);

	if (InPreInit)
	{
		InPreInit(this, NewInput);
	}

	AddInput(NewInput);

	UpdateInputMap(
		InInputIdx,
		FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT + Inputs.Num() - 1,
		InInputChannel,
		InOutputIdx,
		InOutputChannel,
		EDMMaterialPropertyType::None
	);

	Throughput->OnPostInputAdded(InInputIdx);

	return NewInput;
}

UDMMaterialStageSource* UDMMaterialStage::ChangeInput_PreviousStage(int32 InInputIdx, int32 InInputChannel, EDMMaterialPropertyType InPreviousStageProperty, int32 InOutputIdx, int32 InOutputChannel)
{
	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));

	UDMMaterialLayerObject* Layer = GetLayer();
	check(Layer);

	EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
	check(StageProperty != EDMMaterialPropertyType::None);

	UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base);
	UDMMaterialStageSource* PreviousSource = nullptr;
	
	if (PreviousLayer)
	{
		if (UDMMaterialStage* Stage = PreviousLayer->GetLastEnabledStage(EDMMaterialLayerStage::All))
		{
			PreviousSource = Stage->GetSource();
		}
	}

	if (PreviousSource)
	{
		const TArray<FDMMaterialStageConnector>& PreviousStageOutputs = PreviousSource->GetOutputConnectors();
		check(PreviousStageOutputs.IsValidIndex(InOutputIdx));
		check(Throughput->CanInputConnectTo(InInputIdx, PreviousStageOutputs[InOutputIdx], InOutputChannel));
	}
	else
	{
		check(InOutputIdx == 0);
		check(Throughput->GetInputConnectors().IsEmpty() == false);
		check(Throughput->GetInputConnectors()[0].IsCompatibleWith(EDMValueType::VT_Float3_RGB));
	}

	UpdateInputMap(
		InInputIdx,
		FDMMaterialStageConnectorChannel::PREVIOUS_STAGE,
		InInputChannel,
		InOutputIdx,
		InOutputChannel,
		InPreviousStageProperty
	);

	return PreviousSource;
}

void UDMMaterialStage::UpdateInputMap(int32 InInputIdx, int32 InSourceIndex, int32 InInputChannel, int32 InOutputIdx, int32 InOutputChannel, 
	EDMMaterialPropertyType InStageProperty)
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InSourceIndex != FDMMaterialStageConnectorChannel::PREVIOUS_STAGE || InStageProperty != EDMMaterialPropertyType::None);
	check(InSourceIndex >= 0 && InSourceIndex <= Inputs.Num());

	const bool bMadeChanges = VerifyAllInputMaps();
	check(InputConnectionMap.IsValidIndex(InInputIdx));

	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));

	// Check the validity of the incoming data. It must be valid at set time.
	// There are no guarantees it'll be valid at a later time.
	if (InSourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
	{
		UDMMaterialLayerObject* Layer = GetLayer();
		check(Layer);

		UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(InStageProperty, EDMMaterialLayerStage::Base);
		UDMMaterialStageSource* PreviousSource = nullptr;

		if (PreviousLayer)
		{
			if (UDMMaterialStage* Stage = PreviousLayer->GetLastEnabledStage(EDMMaterialLayerStage::All))
			{
				PreviousSource = Stage->GetSource();
			}
		}

		if (PreviousSource)
		{
			const TArray<FDMMaterialStageConnector>& PreviousStageOutputs = PreviousSource->GetOutputConnectors();
			check(PreviousStageOutputs.IsValidIndex(InOutputIdx));
			check(Throughput->CanInputConnectTo(InInputIdx, PreviousStageOutputs[InOutputIdx], InOutputChannel));
		}
		else
		{
			check(InOutputIdx == 0);
			check(Throughput->GetInputConnectors().IsEmpty() == false);
			check(Throughput->GetInputConnectors()[0].IsCompatibleWith(EDMValueType::VT_Float3_RGB));
		}
	}
	else
	{
		const int32 StageInputIdx = InSourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
		check(Inputs.IsValidIndex(StageInputIdx));

		const TArray<FDMMaterialStageConnector>& StageInputOutputConnectors = Inputs[StageInputIdx]->GetOutputConnectors();
		check(StageInputOutputConnectors.IsValidIndex(InOutputIdx));
		check(Throughput->CanInputConnectTo(InInputIdx, StageInputOutputConnectors[InOutputIdx], InOutputChannel));
	}

	TArray<FDMMaterialStageConnectorChannel>& Channels = InputConnectionMap[InInputIdx].Channels;
	const int32 ChannelIndex = UE::DynamicMaterialEditor::Private::ChannelBitToChannelIndex(InInputChannel);
	const FDMMaterialStageConnectorChannel NewChannel = {InSourceIndex, InStageProperty, InOutputIdx, InOutputChannel};

	// Remove all mapping for this input and replace them with a whole channel mapping
	if (ChannelIndex == 0)
	{
		if (!bMadeChanges && Channels.Num() == 1 && Channels[0] == NewChannel)
		{
			return;
		}

		Channels = {NewChannel};
	}
	// Add/replace the new channel-specific mapping
	else
	{
		if (!bMadeChanges && Channels.IsValidIndex(ChannelIndex) && Channels[ChannelIndex] == NewChannel)
		{
			return;
		}

		if (!Channels.IsValidIndex(ChannelIndex))
		{
			const FDMMaterialStageConnectorChannel BlankChannel = {FDMMaterialStageConnectorChannel::NO_SOURCE, EDMMaterialPropertyType::None, 0, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL};

			for (int32 NewIndex = Channels.Num(); NewIndex <= ChannelIndex; ++NewIndex)
			{
				Channels.Add(BlankChannel);
			}
		}

		Channels[ChannelIndex] = NewChannel;
	}

	RemoveUnusedInputs();

	Source->Update(this, EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}

int32 UDMMaterialStage::FindIndex() const
{
	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return INDEX_NONE;
	}

	const TArray<TObjectPtr<UDMMaterialStage>>& Stages = Layer->GetAllStages();

	for (int32 StageIndex = 0; StageIndex < Stages.Num(); ++StageIndex)
	{
		if (Stages[StageIndex] == this)
		{
			return StageIndex;
		}
	}

	return INDEX_NONE;
}

UDMMaterialStage* UDMMaterialStage::GetPreviousStage() const
{
	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return nullptr;
	}

	TArray<UDMMaterialStage*> Stages = Layer->GetStages(EDMMaterialLayerStage::All);
	int32 StageIndex = Stages.IndexOfByKey(this);

	if (StageIndex == INDEX_NONE)
	{
		return nullptr;
	}

	--StageIndex;

	if (Stages.IsValidIndex(StageIndex))
	{
		return Stages[StageIndex];
	}

	return nullptr;
}

UDMMaterialStage* UDMMaterialStage::GetNextStage() const
{
	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return nullptr;
	}

	TArray<UDMMaterialStage*> Stages = Layer->GetStages(EDMMaterialLayerStage::All);
	int32 StageIndex = Stages.IndexOfByKey(this);

	if (StageIndex == INDEX_NONE)
	{
		return nullptr;
	}

	++StageIndex;

	if (Stages.IsValidIndex(StageIndex))
	{
		return Stages[StageIndex];
	}

	return nullptr;
}

bool UDMMaterialStage::VerifyAllInputMaps()
{
	if (!IsComponentValid())
	{
		return false;
	}

	bool bVerified = true;

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);

	if (!Throughput)
	{
		if (InputConnectionMap.Num() > 1)
		{
			InputConnectionMap.Empty();
			bVerified = false;
		}

		Inputs.Empty(); // If we have no inputs connectors, we don't need any inputs...

		return bVerified;
	}

	const TArray<FDMMaterialStageConnector>& ExpressionInputs = Throughput->GetInputConnectors();

	if (ExpressionInputs.IsEmpty())
	{
		if (InputConnectionMap.Num() > 1)
		{
			InputConnectionMap.Empty();
			bVerified = false;
		}

		Inputs.Empty(); // If we have no inputs connectors, we don't need any inputs...

		return bVerified;
	}

	if (InputConnectionMap.Num() != ExpressionInputs.Num())
	{
		InputConnectionMap.SetNum(ExpressionInputs.Num());
		bVerified = false;
	}

	for (int32 InputIdx = 0; InputIdx < InputConnectionMap.Num(); ++InputIdx)
	{
		bVerified = bVerified && VerifyInputMap(InputIdx);
	}

	return bVerified;
}

bool UDMMaterialStage::VerifyInputMap(int32 InInputIdx)
{
	if (!IsComponentValid())
	{
		return false;
	}

	bool bVerified = true;

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);

	if (!Throughput)
	{
		if (!InputConnectionMap.IsEmpty())
		{
			InputConnectionMap.Empty();
			bVerified = false;
		}

		Inputs.Empty(); // If we have no inputs connectors, we don't need any inputs...

		return bVerified;
	}

	const TArray<FDMMaterialStageConnector>& ExpressionInputs = Throughput->GetInputConnectors();

	if (ExpressionInputs.IsEmpty())
	{
		if (!InputConnectionMap.IsEmpty())
		{
			InputConnectionMap.Empty();
			bVerified = false;
		}

		Inputs.Empty(); // If we have no inputs connectors, we don't need any inputs...

		return bVerified;
	}

	if (!InputConnectionMap.IsValidIndex(InInputIdx))
	{
		return false;
	}

	if (InputConnectionMap[InInputIdx].Channels.IsEmpty())
	{
		return true;
	}

	UDMMaterialLayerObject* Layer = GetLayer();
	check(Layer);

	EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
	check(StageProperty != EDMMaterialPropertyType::None);

	for (FDMMaterialStageConnectorChannel& Channel : InputConnectionMap[InInputIdx].Channels)
	{
		bool bValidConnectionMap = false;

		if (Channel.SourceIndex == FDMMaterialStageConnectorChannel::NO_SOURCE)
		{
			continue;
		}

		// Check previous stage
		if (Channel.SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
		{
			if (UDMMaterialLayerObject* PreviousLayerAndStage = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base))
			{
				check(PreviousLayerAndStage->GetStage(EDMMaterialLayerStage::Mask));

				if (UDMMaterialStageSource* PreviousStageSource = PreviousLayerAndStage->GetStage(EDMMaterialLayerStage::Mask)->GetSource())
				{
					const TArray<FDMMaterialStageConnector>& PreviousStageOutputConnectors = PreviousStageSource->GetOutputConnectors();

					if (PreviousStageOutputConnectors.IsValidIndex(Channel.OutputIndex)
						&& Throughput->CanInputConnectTo(InInputIdx, PreviousStageOutputConnectors[Channel.OutputIndex], Channel.OutputChannel))
					{
						// This is valid
						bValidConnectionMap = true;
					}
				}
			}
		}
		// Check inputs
		else
		{
			const int32 StageInputIdx = Channel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			if (Inputs.IsValidIndex(StageInputIdx))
			{
				const TArray<FDMMaterialStageConnector>& StageInputOutputConnectors = Inputs[StageInputIdx]->GetOutputConnectors();

				if (StageInputOutputConnectors.IsValidIndex(Channel.OutputIndex)
					&& Throughput->CanInputConnectTo(InInputIdx, StageInputOutputConnectors[Channel.OutputIndex], Channel.OutputChannel))
				{
					bValidConnectionMap = true;
				}
			}
		}

		// Invalid source channel
		if (!bValidConnectionMap)
		{
			Channel.SourceIndex = FDMMaterialStageConnectorChannel::NO_SOURCE;
			bVerified = false;
		}
	}

	bool bFoundSource = false;

	for (const FDMMaterialStageConnectorChannel& Channel : InputConnectionMap[InInputIdx].Channels)
	{
		if (Channel.SourceIndex != FDMMaterialStageConnectorChannel::NO_SOURCE)
		{
			bFoundSource = true;
			break;
		}
	}

	// This is just clean up and does not affect verification.
	if (!bFoundSource)
	{
		InputConnectionMap[InInputIdx].Channels.Empty();
	}

	return bVerified;
}

UDMMaterialStageSource* UDMMaterialStage::ChangeSource(TSubclassOf<UDMMaterialStageSource> InSourceClass, FSourceInitFunctionPtr InPreInit)
{
	if (!bCanChangeSource)
	{
		return nullptr;
	}

	check(InSourceClass);
	check(!InSourceClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	UDMMaterialStageSource* NewSource = NewObject<UDMMaterialStageSource>(this, InSourceClass, NAME_None, RF_Transactional);
	check(NewSource);

	if (InPreInit)
	{
		InPreInit(this, NewSource);
	}

	SetSource(NewSource);

	return NewSource;
}

bool UDMMaterialStage::IsCompatibleWithPreviousStage(const UDMMaterialStage* InPreviousStage) const
{
	return true;
}

bool UDMMaterialStage::IsCompatibleWithNextStage(const UDMMaterialStage* InNextStage) const
{
	if (!InNextStage)
	{
		return true;
	}

	return InNextStage->IsCompatibleWithPreviousStage(this);
}

void UDMMaterialStage::AddInput(UDMMaterialStageInput* InNewInput)
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InNewInput);
	check(InNewInput->GetStage() == this);

	Inputs.Add(InNewInput);
	InputConnectionMap.Add(FDMMaterialStageConnection());

	if (HasComponentBeenAdded())
	{
		if (GUndo)
		{
			InNewInput->Modify();
		}

		InNewInput->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialStage::RemoveInput(UDMMaterialStageInput* InInput)
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InInput);
	check(InInput->GetStage() == this);

	bool bFound = false;

	for (int32 InputIdx = 0; InputIdx < Inputs.Num(); ++InputIdx)
	{
		if (Inputs[InputIdx] == InInput)
		{
			Inputs.RemoveAt(InputIdx);
			InputConnectionMap.RemoveAt(InputIdx);
			bFound = true;
			break;
		}
	}

	check(bFound);

	if (GUndo)
	{
		InInput->Modify();
	}

	InInput->SetComponentState(EDMComponentLifetimeState::Removed);

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStage::RemoveAllInputs()
{
	if (Inputs.IsEmpty())
	{
		return;
	}

	for (UDMMaterialStageInput* Input : Inputs)
	{
		if (GUndo)
		{
			Input->Modify();
		}

		Input->SetComponentState(EDMComponentLifetimeState::Removed);
	}

	Inputs.Empty();

	Update(this, EDMUpdateType::Structure);
}

const FDMMaterialStageConnectorChannel* UDMMaterialStage::FindInputChannel(UDMMaterialStageInput* InStageInput)
{
	check(InStageInput);

	int32 InputIdx = Inputs.Find(InStageInput);
	
	if (InputIdx == INDEX_NONE)
	{
		return nullptr;
	}

	InputIdx += FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

	for (const FDMMaterialStageConnection& Connection : InputConnectionMap)
	{
		for (const FDMMaterialStageConnectorChannel& Channel : Connection.Channels)
		{
			if (Channel.SourceIndex == InputIdx)
			{
				return &Channel;
			}
		}
	}

	return nullptr;
}

void UDMMaterialStage::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		SetComponentState(EDMComponentLifetimeState::Removed);
		return;
	}

	MarkComponentDirty();

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStage::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (!Source)
	{
		return;
	}

	UE_LOGF(LogDynamicMaterialEditor, Display, "Building Material Designer Stage Preview (%ls)...", *GetName());

	UDMMaterialLayerObject* Layer = GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	InPreviewMaterial->GetEditorOnlyData()->EmissiveColor.Expression = nullptr;
	InPreviewMaterial->GetEditorOnlyData()->EmissiveColor.OutputIndex = 0;

	const bool bGenerateSuccess = Source->GenerateStagePreviewMaterial(
		this,
		InPreviewMaterial,
		InPreviewMaterial->GetEditorOnlyData()->EmissiveColor.Expression,
		InPreviewMaterial->GetEditorOnlyData()->EmissiveColor.OutputIndex
	);
}

#undef LOCTEXT_NAMESPACE
