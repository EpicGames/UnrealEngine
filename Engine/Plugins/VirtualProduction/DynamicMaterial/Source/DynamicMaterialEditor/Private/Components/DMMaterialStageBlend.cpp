// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/DMMaterialStageBlend.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageInput.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "DMEDefs.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionMax.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMInputNodeBuilder.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageBlend)
 
#define LOCTEXT_NAMESPACE "DMMaterialProperty"

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageBlend::Blends = {};
 
UDMMaterialStageBlend::UDMMaterialStageBlend()
	: UDMMaterialStageBlend(FText::GetEmpty(), FText::GetEmpty())
{
}
 
UDMMaterialStageBlend::UDMMaterialStageBlend(const FText& InName, const FText& InDescription)
	: UDMMaterialStageThroughput(InName)
	, BlendDescription(InDescription)
{
	bInputRequired = true;
	bAllowNestedInputs = true;

	BaseChannelOverride = EAvaColorChannel::None;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageBlend, BaseChannelOverride));
 
	InputConnectors.Add({InputAlpha, LOCTEXT("Opacity", "Opacity"), EDMValueType::VT_Float1});
	InputConnectors.Add({InputA, LOCTEXT("PreviousStage", "Previous Stage"), EDMValueType::VT_Float3_RGB});
	InputConnectors.Add({InputB, LOCTEXT("Base", "Base"), EDMValueType::VT_Float3_RGB});
 
	OutputConnectors.Add({0, LOCTEXT("Blend", "Blend"), EDMValueType::VT_Float3_RGB});
}

bool UDMMaterialStageBlend::CanUseBaseChannelOverride() const
{
	return GetDefaultBaseChannelOverrideOutputIndex() != INDEX_NONE;
}

EAvaColorChannel UDMMaterialStageBlend::GetBaseChannelOverride() const
{
	if (CanUseBaseChannelOverride())
	{
		PullBaseChannelOverride();
		return BaseChannelOverride;
	}

	return EAvaColorChannel::None;
}

void UDMMaterialStageBlend::SetBaseChannelOverride(EAvaColorChannel InMaskChannel)
{
	if (!CanUseBaseChannelOverride())
	{
		return;
	}

	if (GetBaseChannelOverride() == InMaskChannel)
	{
		return;
	}

	BaseChannelOverride = InMaskChannel;
	PushBaseChannelOverride();

	Update(this, EDMUpdateType::Structure);
}

const FText& UDMMaterialStageBlend::GetBlendDescription() const
{
	return BlendDescription;
}

void UDMMaterialStageBlend::BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
	UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
	TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	CreateBlendOpacityLayer<UMaterialExpressionMax>(InBuildState, InBaseLayerOpacityExpression, InBaseOutputIndex, InBaseOutputChannel,
		InMyLayerOpacityExpression, InMyOutputIndex, InMyOutputChannel, OutAddedExpressions, OutOutputIndex, OutOutputChannel);
}

UDMMaterialStageInputValue* UDMMaterialStageBlend::GetOpacityValue(UDMMaterialStage* InStage) const
{
	if (!InStage)
	{
		return nullptr;
	}

	const TArray<UDMMaterialStageInput*>& StageInputs = InStage->GetInputs();
	const TArray<FDMMaterialStageConnection>& InputConnectionMap = InStage->GetInputConnectionMap();

	constexpr int32 OpacityIndex = 0;

	if (InputConnectionMap.IsValidIndex(OpacityIndex) && !InputConnectionMap[OpacityIndex].Channels.IsEmpty())
	{
		if (InputConnectionMap[OpacityIndex].Channels[0].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
		{
			const int32 StageInputIndex = InputConnectionMap[OpacityIndex].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			return Cast<UDMMaterialStageInputValue>(StageInputs[StageInputIndex]);
		}
	}

	return nullptr;
}

int32 UDMMaterialStageBlend::GetDefaultBaseChannelOverrideOutputIndex() const
{
	UDMMaterialStageInput* StageInputB = GetInputB();

	if (!StageInputB)
	{
		return INDEX_NONE;
	}

	const TArray<FDMMaterialStageConnector>& MaskInputOutputConnectors = StageInputB->GetOutputConnectors();

	for (int32 Index = 0; Index < MaskInputOutputConnectors.Num(); ++Index)
	{
		if (UDMValueDefinitionLibrary::GetValueDefinition(MaskInputOutputConnectors[Index].Type).GetFloatCount() > 1)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

bool UDMMaterialStageBlend::IsValidBaseChannelOverrideOutputIndex(int32 InIndex) const
{
	UDMMaterialStageInput* StageInputB = GetInputB();

	if (!StageInputB)
	{
		return false;
	}

	const TArray<FDMMaterialStageConnector>& MaskInputOutputConnectors = StageInputB->GetOutputConnectors();

	if (!MaskInputOutputConnectors.IsValidIndex(InIndex))
	{
		return false;
	}

	return UDMValueDefinitionLibrary::GetValueDefinition(MaskInputOutputConnectors[InIndex].Type).GetFloatCount() > 1;
}

void UDMMaterialStageBlend::PullBaseChannelOverride() const
{
	BaseChannelOverride = EAvaColorChannel::None;

	if (!CanUseBaseChannelOverride())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

	if (!InputMap.IsValidIndex(UDMMaterialStageBlend::InputB)
		|| !InputMap[UDMMaterialStageBlend::InputB].Channels.IsValidIndex(0))
	{
		return;
	}

	TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
	const FDMMaterialStageConnectorChannel& MaskConnectorChannel = InputMap[UDMMaterialStageBlend::InputB].Channels[0];

	switch (MaskConnectorChannel.OutputChannel)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			BaseChannelOverride = EAvaColorChannel::Red;
			break;

		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			BaseChannelOverride = EAvaColorChannel::Green;
			break;

		case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
			BaseChannelOverride = EAvaColorChannel::Blue;
			break;

		case FDMMaterialStageConnectorChannel::FOURTH_CHANNEL:
			BaseChannelOverride = EAvaColorChannel::Alpha;
			break;

		default:
			// Do nothing
			break;
	}
}

void UDMMaterialStageBlend::PushBaseChannelOverride()
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	if (!CanUseBaseChannelOverride())
	{
		return;
	}

	const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

	if (!InputMap.IsValidIndex(UDMMaterialStageBlend::InputB)
		|| !InputMap[UDMMaterialStageBlend::InputB].Channels.IsValidIndex(0))
	{
		return;
	}

	TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
	const FDMMaterialStageConnectorChannel& MaskConnectorChannel = InputMap[UDMMaterialStageBlend::InputB].Channels[0];
	const int32 MaskInputIdx = MaskConnectorChannel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

	if (!Inputs.IsValidIndex(MaskInputIdx))
	{
		return;
	}

	const int32 OutputIndex = IsValidBaseChannelOverrideOutputIndex(MaskConnectorChannel.OutputIndex)
		? MaskConnectorChannel.OutputIndex
		: GetDefaultBaseChannelOverrideOutputIndex();

	int32 OutputChannel;

	switch (BaseChannelOverride)
	{
		case EAvaColorChannel::Red:
			OutputChannel = FDMMaterialStageConnectorChannel::FIRST_CHANNEL;
			break;

		case EAvaColorChannel::Green:
			OutputChannel = FDMMaterialStageConnectorChannel::SECOND_CHANNEL;
			break;

		case EAvaColorChannel::Blue:
			OutputChannel = FDMMaterialStageConnectorChannel::THIRD_CHANNEL;
			break;

		case EAvaColorChannel::Alpha:
			OutputChannel = FDMMaterialStageConnectorChannel::FOURTH_CHANNEL;
			break;

		default:
			OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
			break;
	}

	Stage->UpdateInputMap(UDMMaterialStageBlend::InputB, MaskConnectorChannel.SourceIndex, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		OutputIndex, OutputChannel, MaskConnectorChannel.MaterialProperty);
}

UDMMaterialStage* UDMMaterialStageBlend::CreateStage(TSubclassOf<UDMMaterialStageBlend> InMaterialStageBlendClass, UDMMaterialLayerObject* InLayer)
{
	check(InMaterialStageBlendClass);
 
	GetAvailableBlends();
	check(Blends.Contains(TStrongObjectPtr<UClass>(InMaterialStageBlendClass.Get())));
 
	const FDMUpdateGuard Guard;
 
	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageBlend* SourceBlend = NewObject<UDMMaterialStageBlend>(
		NewStage, 
		InMaterialStageBlendClass.Get(), 
		NAME_None, 
		RF_Transactional
	);

	check(SourceBlend);
 
	NewStage->SetSource(SourceBlend);
 
	return NewStage;
}
 
const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageBlend::GetAvailableBlends()
{
	if (Blends.IsEmpty())
	{
		GenerateBlendList();
	}
 
	return Blends;
}
 
void UDMMaterialStageBlend::GenerateBlendList()
{
	Blends.Empty();
 
	const TArray<TStrongObjectPtr<UClass>>& SourceList = UDMMaterialStageSource::GetAvailableSourceClasses();
 
	for (const TStrongObjectPtr<UClass>& SourceClass : SourceList)
	{
		UDMMaterialStageBlend* StageBlendCDO = Cast<UDMMaterialStageBlend>(SourceClass->GetDefaultObject(true));
 
		if (!StageBlendCDO)
		{
			continue;
		}
 
		Blends.Add(SourceClass);
	}
}

bool UDMMaterialStageBlend::CanInputAcceptType(int32 InputIndex, EDMValueType ValueType) const
{
	check(InputConnectors.IsValidIndex(InputIndex));
 
	if (InputIndex == InputAlpha)
	{
		return InputConnectors[InputIndex].IsCompatibleWith(ValueType);
	}
 
	if (!UDMValueDefinitionLibrary::GetValueDefinition(ValueType).IsFloatType())
	{
		return false;
	}
 
	return InputConnectors[InputIndex].IsCompatibleWith(ValueType);
}
 
void UDMMaterialStageBlend::GetMaskAlphaBlendNode(const TSharedRef<FDMMaterialBuildState>& InBuildState, UMaterialExpression*& OutExpression, 
	int32& OutOutputIndex, int32& OutOutputChannel) const
{
	FDMMaterialStageConnectorChannel Channel;
	TArray<UMaterialExpression*> Expressions;
 
	OutOutputIndex = ResolveInput(InBuildState, 0, Channel, Expressions);
	OutOutputChannel = Channel.OutputChannel;
	OutExpression = Expressions.Last();
}

bool UDMMaterialStageBlend::GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
	UMaterialExpression*& OutMaterialExpression, int32& OutputIndex)
{
	check(InStage);
	check(InPreviewMaterial);

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	const TArray<FDMMaterialStageConnection>& InputConnectionMap = InStage->GetInputConnectionMap();

	if (!InputConnectionMap.IsValidIndex(InputB)
		|| InputConnectionMap[InputB].Channels.Num() != 1)
	{
		return false;
	}

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(InStage);

	UDMMaterialStageSource* PreviewSource = GetInputB();

	PreviewSource->GenerateExpressions(BuildState);
	const TArray<UMaterialExpression*>& SourceExpressions = BuildState->GetStageSourceExpressions(PreviewSource);

	if (SourceExpressions.IsEmpty())
	{
		return false;
	}

	UMaterialExpression* LastExpression = SourceExpressions.Last();
	OutputIndex = InputConnectionMap[InputB].Channels[0].OutputIndex;

	{
		const UDMMaterialStageSource* Source = nullptr;

		for (const TPair<const UDMMaterialStageSource*, TArray<UMaterialExpression*>>& Pair : BuildState->GetStageSourceMap())
		{
			if (Pair.Value.IsEmpty())
			{
				continue;
			}

			if (Pair.Value.Last() == LastExpression)
			{
				Source = Pair.Key;
				break;
			}
		}

		if (Source && Source->GetOutputConnectors().IsValidIndex(OutputIndex))
		{
			OutputIndex = Source->GetOutputConnectors()[OutputIndex].Index;
		}
	}

	if (InputConnectionMap[InputB].Channels[0].OutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		const int32 MaskedOutput = BuildState->GetBuildUtils().FindOutputForBitmask(
			LastExpression,
			InputConnectionMap[InputB].Channels[0].OutputChannel
		);

		if (MaskedOutput == INDEX_NONE)
		{
			LastExpression = BuildState->GetBuildUtils().CreateExpressionBitMask(
				LastExpression,
				OutputIndex,
				InputConnectionMap[InputB].Channels[0].OutputChannel
			);

			OutputIndex = 0;
		}
		else
		{
			OutputIndex = MaskedOutput;
		}
	}

	OutMaterialExpression = LastExpression;

	return true;
}

FSlateIcon UDMMaterialStageBlend::GetComponentIcon() const
{
	if (UDMMaterialStageInput* InputBValue = GetInputB())
	{
		return InputBValue->GetComponentIcon();
	}

	return Super::GetComponentIcon();
}

void UDMMaterialStageBlend::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	Super::Update(InSource, InUpdateType);

	PullBaseChannelOverride();
}

void UDMMaterialStageBlend::PostEditUndo()
{
	Super::PostEditUndo();

	PullBaseChannelOverride();
}

void UDMMaterialStageBlend::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName BaseChannelOverrideName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageBlend, BaseChannelOverride);

	if (InPropertyChangedEvent.GetPropertyName() == BaseChannelOverrideName)
	{
		PushBaseChannelOverride();
	}
}

void UDMMaterialStageBlend::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	if (!IsComponentValid())
	{
		return;
	}

	static const FName MaskChannelName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageBlend, BaseChannelOverride);

	if (InPropertyChangedEvent.GetPropertyName() == MaskChannelName)
	{
		PushBaseChannelOverride();
	}
	else
	{
		PullBaseChannelOverride();
	}

	Super::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
}

void UDMMaterialStageBlend::AddDefaultInput(int32 InInputIndex) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	switch (InInputIndex)
	{
		case InputAlpha:
		{
			UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
				Stage, 
				InInputIndex, 
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
				EDMValueType::VT_Float1, 
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);

			check(InputValue);
 
			UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
			check(Float1Value);
 
			Float1Value->SetDefaultValue(1.f);
			Float1Value->ApplyDefaultValue();
			Float1Value->SetValueRange(FFloatInterval(0, 1));
			break;
		}
 
		case InputA:
		{
			UDMMaterialLayerObject* Layer = Stage->GetLayer();
			check(Layer);
 
			EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
			check(StageProperty != EDMMaterialPropertyType::None);
 
			UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base);
 
			if (PreviousLayer)
			{
				Stage->ChangeInput_PreviousStage(
					InInputIndex, 
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
					StageProperty,
					0, 
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
				);
			}
			else
			{
				EDMMaterialPropertyType DefaultProperty = StageProperty;

				if (DefaultProperty == EDMMaterialPropertyType::None)
				{
					if (UDMMaterialSlot* Slot = Layer->GetSlot())
					{
						if (UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData())
						{
							if (ModelEditorOnlyData->GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::BaseColor))
							{
								DefaultProperty = EDMMaterialPropertyType::BaseColor;
							}
							else if (ModelEditorOnlyData->GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::EmissiveColor))
							{
								DefaultProperty = EDMMaterialPropertyType::EmissiveColor;
							}
						}
					}
				}

				Stage->ChangeInput_PreviousStage(
					InInputIndex,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
					DefaultProperty,
					0,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
				);

			}

			break;
		}
 
		case InputB:
			if (CanInputAcceptType(InInputIndex, EDMValueType::VT_Float3_RGB))
			{
				UDMMaterialStageInputExpression::ChangeStageInput_Expression(
					Stage,
					UDMMaterialStageExpressionTextureSample::StaticClass(), 
					InInputIndex, 
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
					0,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
				);
			}
			break;
 
		default:
			checkNoEntry();
			break;
	}
}

bool UDMMaterialStageBlend::CanChangeInput(int32 InputIndex) const
{
	return (InputIndex == InputB);
}
 
bool UDMMaterialStageBlend::CanChangeInputType(int32 InputIndex) const
{
	return false;
}

bool UDMMaterialStageBlend::IsInputVisible(int32 InputIndex) const
{
	if (InputIndex == InputA)
	{
		return false;
	}

	return Super::IsInputVisible(InputIndex);
}

int32 UDMMaterialStageBlend::ResolveInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InputIndex, FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const
{
	int32 NodeOutputIndex = Super::ResolveInput(InBuildState, InputIndex, OutChannel, OutExpressions);

	if (InputIndex == InputB)
	{
		if (UDMMaterialStage* Stage = GetStage())
		{
			if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
			{
				Layer->ApplyEffects(
					InBuildState, 
					Stage, 
					OutExpressions, 
					OutChannel.OutputChannel, 
					NodeOutputIndex
				);
			}
		}
	}

	return NodeOutputIndex;
}

void UDMMaterialStageBlend::OnPostInputAdded(int32 InInputIdx)
{
	Super::OnPostInputAdded(InInputIdx);

	if (BaseChannelOverride != EAvaColorChannel::None)
	{
		PushBaseChannelOverride();
	}
}

FText UDMMaterialStageBlend::GetStageDescription() const
{
	if (UDMMaterialStageInput* StageInputB = GetInputB())
	{
		return StageInputB->GetComponentDescription();
	}

	return Super::GetStageDescription();
}

bool UDMMaterialStageBlend::SupportsLayerMaskTextureUVLink() const
{
	UDMMaterialStageInput* StageInputB = GetInputB();

	if (!StageInputB)
	{
		return false;
	}

	UDMMaterialStageInputThroughput* InputThroughput = Cast<UDMMaterialStageInputThroughput>(StageInputB);

	if (!InputThroughput)
	{
		return false;
	}

	UDMMaterialStageThroughput* Throughput = InputThroughput->GetMaterialStageThroughput();

	if (!Throughput)
	{
		return false;
	}

	return Throughput->SupportsLayerMaskTextureUVLink();
}

FDMExpressionInput UDMMaterialStageBlend::GetLayerMaskLinkTextureUVInputExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	FDMExpressionInput ExpressionInput = {};

	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return ExpressionInput;
	}

	UDMMaterialStageInput* StageInputB = GetInputB();

	if (!StageInputB)
	{
		return ExpressionInput;
	}

	UDMMaterialStageInputThroughput* InputThroughput = Cast<UDMMaterialStageInputThroughput>(StageInputB);

	if (!InputThroughput)
	{
		return ExpressionInput;
	}

	UDMMaterialStageThroughput* Throughput = InputThroughput->GetMaterialStageThroughput();

	if (!Throughput)
	{
		return ExpressionInput;
	}

	const TArray<FDMMaterialStageConnection>& InputConnections = Stage->GetInputConnectionMap();

	if (!InputConnections.IsValidIndex(InputB) || InputConnections[InputB].Channels.Num() != 1)
	{
		return ExpressionInput;
	}

	FDMMaterialStageConnectorChannel Channel = InputConnections[InputB].Channels[0];

	ExpressionInput.OutputIndex = ResolveLayerMaskTextureUVLinkInputImpl(
		InBuildState,
		StageInputB,
		Channel,
		ExpressionInput.OutputExpressions
	);

	ExpressionInput.OutputChannel = Channel.OutputChannel;

	return ExpressionInput;
}
 
void UDMMaterialStageBlend::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}
 
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);
 
	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);
 
	const TArray<FDMMaterialStageConnection>& InputConnectionMap = Stage->GetInputConnectionMap();
	const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();
	TArray<UE::DynamicMaterialEditor::Private::FDMInputInputs> Inputs; // There can be multiple inputs per input
	bool bHasStageInput = false;
 
	for (int32 InputIdx = 0; InputIdx < InputConnectors.Num(); ++InputIdx)
	{
		TArray<UDMMaterialStageInput*> ChannelInputs;
 
		if (!InputConnectionMap.IsValidIndex(InputIdx))
		{
			continue;
		}
 
		ChannelInputs.SetNum(InputConnectionMap[InputIdx].Channels.Num());
		bool bNonStageInput = false;
 
		for (int32 ChannelIdx = 0; ChannelIdx < InputConnectionMap[InputIdx].Channels.Num(); ++ChannelIdx)
		{
			if (InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
			{
				bHasStageInput = true;
				ChannelInputs[ChannelIdx] = nullptr;
				continue;
			}
 
			if (InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
			{
				int32 StageInputIdx = InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
				ChannelInputs[ChannelIdx] = StageInputs[StageInputIdx];
				bNonStageInput = true;
				continue;
			}
		}
 
		if (ChannelInputs.IsEmpty())
		{
			continue;
		}
 
		if (bNonStageInput)
		{
			Inputs.Add({InputIdx, ChannelInputs});
		}
	}
 
	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);
 
	if (!bHasStageInput || Inputs.IsEmpty())
	{
		Stage->GenerateExpressions(BuildState);
		UMaterialExpression* StageExpression = BuildState->GetLastStageExpression(Stage);
 
		BuildState->GetBuildUtils().UpdatePreviewMaterial(
			StageExpression, 
			0, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			32
		);
	}
	else
	{
		UE::DynamicMaterialEditor::Private::BuildExpressionInputs(
			BuildState, 
			InputConnectionMap, 
			Inputs
		);
	}
}

UDMMaterialValueFloat1* UDMMaterialStageBlend::GetInputAlpha() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

		if (InputMap.IsValidIndex(InputAlpha)
			&& InputMap[InputAlpha].Channels.IsValidIndex(0))
		{
			TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
			int32 StageInputIdx = InputMap[InputAlpha].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			if (Stage->GetInputs().IsValidIndex(StageInputIdx))
			{
				if (UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs()[StageInputIdx]))
				{
					return Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
				}
			}
		}
	}

	return nullptr;
}

UDMMaterialStageInput* UDMMaterialStageBlend::GetInputB() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

		if (InputMap.IsValidIndex(InputB)
			&& InputMap[InputB].Channels.IsValidIndex(0))
		{
			TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
			int32 StageInputIdx = InputMap[InputB].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			if (Stage->GetInputs().IsValidIndex(StageInputIdx))
			{
				return Stage->GetInputs()[StageInputIdx];
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
