// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageBlendFunction.h"
#include "DMDefs.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageBlendFunction)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendFunction"

UDMMaterialStageBlendFunction::UDMMaterialStageBlendFunction()
	: UDMMaterialStageBlendFunction(LOCTEXT("BlendFunction", "Blend Function"), FText::GetEmpty(), nullptr)
{
}

UDMMaterialStageBlendFunction::UDMMaterialStageBlendFunction(const FText& InName, const FText& InDescription, UMaterialFunctionInterface* InMaterialFunction)
	: UDMMaterialStageBlend(InName, InDescription)
	, MaterialFunction(InMaterialFunction)
{
}

UDMMaterialStageBlendFunction::UDMMaterialStageBlendFunction(const FText& InName, const FText& InDescription, const FName& InFunctionName, const FString& InFunctionPath)
	: UDMMaterialStageBlend(InName, InDescription)
	, MaterialFunction(FDMMaterialFunctionLibrary::Get().GetFunction(InFunctionName, InFunctionPath))
{
	check(MaterialFunction);
}

void UDMMaterialStageBlendFunction::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	if (!MaterialFunction)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMaterialFunctionCall>(UE_DM_NodeComment_Default);;
	FunctionCall->SetMaterialFunction(MaterialFunction);
	FunctionCall->UpdateFromFunctionResource();

	InBuildState->AddStageSourceExpressions(this, {FunctionCall});
}

void UDMMaterialStageBlendFunction::ConnectOutputToInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InInputIdx, int32 InExpressionInputIndex,
	UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel)
{
	check(InSourceExpression);
	check(InSourceExpression->GetOutputs().IsValidIndex(InSourceOutputIndex));
	check(InExpressionInputIndex >= 0 && InExpressionInputIndex <= 2);

	const TArray<UMaterialExpression*>& StageSourceExpressions = InBuildState->GetStageSourceExpressions(this);
	check(!StageSourceExpressions.IsEmpty());

	if (!MaterialFunction)
	{
		return;
	}

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	MaterialFunction->GetInputsAndOutputs(Inputs, Outputs);

	int32 InputAlphaIndex = INDEX_NONE;
	int32 InputAIndex = INDEX_NONE;
	int32 InputBIndex = INDEX_NONE;

	// Could go by name here, but that is more bug prone.
	for (int32 InputIdx = 0; InputIdx < Inputs.Num(); ++InputIdx)
	{
		if (Inputs[InputIdx].ExpressionInput->InputType == FunctionInput_Vector3)
		{
			if (InputAIndex == INDEX_NONE)
			{
				InputAIndex = InputIdx;
			}
			else if (InputBIndex == INDEX_NONE)
			{
				InputBIndex = InputIdx;
			}
		}
		else if (Inputs[InputIdx].ExpressionInput->InputType == FunctionInput_Scalar)
		{
			if (InputAlphaIndex == INDEX_NONE)
			{
				InputAlphaIndex = InputIdx;
			}
		}
	}

	switch (InExpressionInputIndex)
	{
		case InputA:
			ConnectOutputToInput_Internal(InBuildState, StageSourceExpressions[0] /* FunctionCall */, InputAIndex, InSourceExpression, InSourceOutputIndex, InSourceOutputChannel);
			break;

		case InputB:
			ConnectOutputToInput_Internal(InBuildState, StageSourceExpressions[0] /* FunctionCall */, InputBIndex, InSourceExpression, InSourceOutputIndex, InSourceOutputChannel);
			break;

		case InputAlpha:
			ConnectOutputToInput_Internal(InBuildState, StageSourceExpressions[0] /* FunctionCall */, InputAlphaIndex, InSourceExpression, InSourceOutputIndex, InSourceOutputChannel);
			break;

		default:
			checkNoEntry();
			break;
	}
}

#undef LOCTEXT_NAMESPACE
