// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageExpression.h"

#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "DMDefs.h"
#include "EditorClassUtils.h"
#include "Materials/MaterialExpression.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Utils/DMUtils.h"

// Expressions to exclude
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionFontSample.h"
#include "Materials/MaterialExpressionGenericConstant.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionRerouteBase.h"
#include "Materials/MaterialExpressionSubstrate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageExpression)

#define LOCTEXT_NAMESPACE "DMMaterialProperty"

TSet<FName> UDMMaterialStageExpression::BlockedMaterialExpressionClasses = {
	UMaterialExpressionComment::StaticClass()->GetFName(),
	UMaterialExpressionCustom::StaticClass()->GetFName(),
	UMaterialExpressionCustomOutput::StaticClass()->GetFName(),
	UMaterialExpressionComposite::StaticClass()->GetFName(),
	UMaterialExpressionGenericConstant::StaticClass()->GetFName(),
	UMaterialExpressionFontSample::StaticClass()->GetFName(),
	UMaterialExpressionPinBase::StaticClass()->GetFName(),
	UMaterialExpressionReroute::StaticClass()->GetFName(),
	UMaterialExpressionRerouteBase::StaticClass()->GetFName(),
	UMaterialExpressionNamedRerouteBase::StaticClass()->GetFName(),
	UMaterialExpressionNamedRerouteDeclaration::StaticClass()->GetFName(),
	UMaterialExpressionNamedRerouteUsage::StaticClass()->GetFName(),
	UMaterialExpressionSubstrateBSDF::StaticClass()->GetFName()
};

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageExpression::SourceExpressions = TArray<TStrongObjectPtr<UClass>>();

UDMMaterialStage* UDMMaterialStageExpression::CreateStage(TSubclassOf<UDMMaterialStageExpression> InMaterialStageExpressionClass, UDMMaterialLayerObject* InLayer)
{
	check(InMaterialStageExpressionClass);

	GetAvailableSourceExpressions();
	check(SourceExpressions.Contains(TStrongObjectPtr<UClass>(InMaterialStageExpressionClass.Get())));

	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageExpression* SourceExpression = NewObject<UDMMaterialStageExpression>(
		NewStage, 
		InMaterialStageExpressionClass.Get(), 
		NAME_None, 
		RF_Transactional
	);
	
	check(SourceExpression);

	NewStage->SetSource(SourceExpression);

	return NewStage;
}

TSubclassOf<UMaterialExpression> UDMMaterialStageExpression::FindClass(FString InClassName)
{
	return FEditorClassUtils::GetClassFromString(InClassName);
}

const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageExpression::GetAvailableSourceExpressions()
{
	if (SourceExpressions.IsEmpty())
	{
		GenerateExpressionList();
	}

	return SourceExpressions;
}

UDMMaterialStageExpression* UDMMaterialStageExpression::ChangeStageSource_Expression(UDMMaterialStage* InStage, 
	TSubclassOf<UDMMaterialStageExpression> InExpressionClass)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	check(InExpressionClass);
	check(!InExpressionClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	return InStage->ChangeSource<UDMMaterialStageExpression>(InExpressionClass);
}

void UDMMaterialStageExpression::GenerateExpressionList()
{
	SourceExpressions.Empty();

	const TArray<TStrongObjectPtr<UClass>>& SourceList = UDMMaterialStageSource::GetAvailableSourceClasses();

	for (const TStrongObjectPtr<UClass>& SourceClass : SourceList)
	{
		UDMMaterialStageExpression* StageExpressionCDO = Cast<UDMMaterialStageExpression>(SourceClass->GetDefaultObject(true));

		if (!StageExpressionCDO)
		{
			continue;
		}

		TSubclassOf<UMaterialExpression> ExpressionClass = StageExpressionCDO->GetMaterialExpressionClass();

		if (!ExpressionClass.Get())
		{
			continue;
		}

		if (ExpressionClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags))
		{
			continue;
		}

		if (BlockedMaterialExpressionClasses.Contains(ExpressionClass->GetFName()) || ExpressionClass->GetName().EndsWith("Parameter"))
		{
			continue;
		}

		UMaterialExpression* ExpressionCDO = CastChecked<UMaterialExpression>(ExpressionClass->GetDefaultObject(true));
		const TArray<FExpressionOutput> ExpressionOutputs = ExpressionCDO->GetOutputs();
		bool bValidOutputs = true;

		for (int32 OutputIdx = 0; OutputIdx < ExpressionOutputs.Num(); ++OutputIdx)
		{
			if (ExpressionCDO->GetOutputValueType(OutputIdx) == MCT_Unknown)
			{
				bValidOutputs = false;
				break;
			}
		}

		if (!bValidOutputs)
		{
			continue;
		}

		SourceExpressions.Add(SourceClass);
	}
}

UDMMaterialStageExpression::UDMMaterialStageExpression()
	: UDMMaterialStageExpression(
		LOCTEXT("UDMMaterialStageExpression", "UDMMaterialStageExpression"),
		UMaterialExpression::StaticClass()
	)
{
}

UDMMaterialStageExpression::UDMMaterialStageExpression(const FText& InName, TSubclassOf<UMaterialExpression> InClass)
	: UDMMaterialStageThroughput(InName)
	, MaterialExpressionClass(InClass)
{
	bInputRequired = false;
	bAllowNestedInputs = false;

	ensureAlwaysMsgf(!InName.IsEmpty(), TEXT("MSE Class with invalid Name: %s"),             *StaticClass()->GetName());
	ensureAlwaysMsgf(InClass,           TEXT("MSE Class with invalid Expression Class: %s"), *StaticClass()->GetName());
}

void UDMMaterialStageExpression::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	check(MaterialExpressionClass.Get());

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	UMaterialExpression* NewExpression = InBuildState->GetBuildUtils().CreateExpression(MaterialExpressionClass.Get(), UE_DM_NodeComment_Default);
	AddExpressionProperties({NewExpression});

	InBuildState->AddStageSourceExpressions(this, {NewExpression});
}

#undef LOCTEXT_NAMESPACE
