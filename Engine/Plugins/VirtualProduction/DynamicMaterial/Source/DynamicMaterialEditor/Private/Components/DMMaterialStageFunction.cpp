// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageFunction.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "DynamicMaterialEditorSettings.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"
#include "MaterialValueType.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialFunctionFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageFunction)

#define LOCTEXT_NAMESPACE "DMMaterialStageFunction"

TSoftObjectPtr<UMaterialFunctionInterface> UDMMaterialStageFunction::NoOp = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(TEXT(
	"/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_NoOp.MF_DM_NoOp'"
)));

UDMMaterialStage* UDMMaterialStageFunction::CreateStage(UDMMaterialLayerObject* InLayer)
{
	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageFunction* SourceFunction = NewObject<UDMMaterialStageFunction>(
		NewStage, 
		StaticClass(), 
		NAME_None, 
		RF_Transactional
	);

	check(SourceFunction);

	NewStage->SetSource(SourceFunction);

	return NewStage;
}

UDMMaterialStageFunction* UDMMaterialStageFunction::ChangeStageSource_Function(UDMMaterialStage* InStage, UMaterialFunctionInterface* InMaterialFunction)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	check(InMaterialFunction);

	UDMMaterialStageFunction* NewFunction = InStage->ChangeSource<UDMMaterialStageFunction>(
		[InMaterialFunction](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageFunction>(InNewSource)->SetMaterialFunction(InMaterialFunction);
		});

	return NewFunction;
}

UMaterialFunctionInterface* UDMMaterialStageFunction::GetNoOpFunction()
{
	return NoOp.LoadSynchronous();
}

void UDMMaterialStageFunction::SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction)
{
	if (MaterialFunction == InMaterialFunction)
	{
		return;
	}

	MaterialFunction = InMaterialFunction;

	OnMaterialFunctionChanged();
}

UDMMaterialValue* UDMMaterialStageFunction::GetInputValue(int32 InIndex) const
{
	TArray<UDMMaterialValue*> Values = GetInputValues();

	if (Values.IsValidIndex(InIndex))
	{
		return Values[InIndex];
	}

	return nullptr;
}

TArray<UDMMaterialValue*> UDMMaterialStageFunction::GetInputValues() const
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return {};
	}

	TArray<UDMMaterialValue*> Values;

	for (UDMMaterialStageInput* Input : Stage->GetInputs())
	{
		if (UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Input))
		{
			if (UDMMaterialValue* Value = InputValue->GetValue())
			{
				Values.Add(Value);
			}
		}
	}

	return Values;
}

void UDMMaterialStageFunction::AddDefaultInput(int32 InInputIndex) const
{
	check(InInputIndex == 0);

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
	check(StageProperty != EDMMaterialPropertyType::None);

	UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base);

	if (PreviousLayer)
	{
		Stage->ChangeInput_PreviousStage(
			InInputIndex, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			StageProperty,
			0, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		EDMMaterialPropertyType DefaultProperty = StageProperty;

		if (DefaultProperty == EDMMaterialPropertyType::None)
		{
			if (UDMMaterialSlot* Slot = Layer->GetSlot())
			{
				if (UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData())
				{
					if (ModelEditorOnlyData->GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::BaseColor))
					{
						DefaultProperty = EDMMaterialPropertyType::BaseColor;
					}
					else if (ModelEditorOnlyData->GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::EmissiveColor))
					{
						DefaultProperty = EDMMaterialPropertyType::EmissiveColor;
					}
				}
			}
		}

		Stage->ChangeInput_PreviousStage(
			InInputIndex,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			DefaultProperty,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
}

bool UDMMaterialStageFunction::CanChangeInput(int32 InputIndex) const
{
	return true;
}

bool UDMMaterialStageFunction::CanChangeInputType(int32 InputIndex) const
{
	return false;
}

bool UDMMaterialStageFunction::IsInputVisible(int32 InputIndex) const
{
	return true;
}

void UDMMaterialStageFunction::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	UMaterialFunctionInterface* ActualMaterialFunction = MaterialFunction;

	if (!IsValid(ActualMaterialFunction))
	{
		ActualMaterialFunction = NoOp.LoadSynchronous();
	}

	if (!ActualMaterialFunction)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMaterialFunctionCall>(UE_DM_NodeComment_Default);
	FunctionCall->SetMaterialFunction(ActualMaterialFunction);
	FunctionCall->UpdateFromFunctionResource();

	InBuildState->AddStageSourceExpressions(this, {FunctionCall});
}

FText UDMMaterialStageFunction::GetComponentDescription() const
{
	if (UMaterialFunctionInterface* MaterialFunctionInterface = MaterialFunction.Get())
	{
		const FString Caption = MaterialFunctionInterface->GetUserExposedCaption();

		if (!Caption.IsEmpty())
		{
			return FText::FromString(Caption);
		}
	}

	return Super::GetComponentDescription();
}

void UDMMaterialStageFunction::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	MaterialFunction_PreEdit = MaterialFunction;
}

void UDMMaterialStageFunction::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (MaterialFunction != MaterialFunction_PreEdit)
	{
		OnMaterialFunctionChanged();
	}
}

void UDMMaterialStageFunction::PostLoad()
{
	Super::PostLoad();

	if (NeedsFunctionInit())
	{
		InitFunction();
	}
}

UDMMaterialStageFunction::UDMMaterialStageFunction()
	: UDMMaterialStageThroughput(LOCTEXT("MaterialFunction", "Material Function"))
{
	MaterialFunction = nullptr;

	bInputRequired = false;
	bAllowNestedInputs = true;

	InputConnectors.Add({InputPreviousStage, LOCTEXT("PreviousStage", "Previous Stage"), EDMValueType::VT_Float3_RGB});

	OutputConnectors.Add({0, LOCTEXT("Output", "Output"), EDMValueType::VT_Float3_RGB});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageFunction, MaterialFunction));
}

void UDMMaterialStageFunction::OnMaterialFunctionChanged()
{
	DeinitFunction();
	InitFunction();

	Update(this, EDMUpdateType::Structure);
}

bool UDMMaterialStageFunction::NeedsFunctionInit() const
{
	UMaterialFunctionInterface* MaterialFunctionInterface = MaterialFunction.Get();

	TArray<UDMMaterialValue*> InputValues = GetInputValues();

	if (!IsValid(MaterialFunctionInterface))
	{
		// If we have no function, but we do have inputs, they need to be refreshed (removed).
		return !InputValues.IsEmpty();
	}

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	MaterialFunctionInterface->GetInputsAndOutputs(Inputs, Outputs);

	if (Outputs.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Function must have at least one output."), true, this);
		return false;
	}

	const EMaterialValueType InputType = Inputs[0].ExpressionInput ? Inputs[0].ExpressionInput->GetInputValueType(0) : MCT_Unknown;
	const bool bInputTypeIsFloat = (InputType & MCT_Float) != 0;

	const EMaterialValueType OutputType = Outputs[0].ExpressionOutput ? Outputs[0].ExpressionOutput->GetOutputValueType(0) : MCT_Unknown;
	const bool bOutputTypeIsFloat = (InputType & MCT_Float) != 0;

	/**
	 * First input and output types must match.
	 * Previously MCT_Float (a combination of float 1, 2, 3 and 4) caused a equality check to fail. It is now more rigorous.
	 */
	bool bValidThroughput = true;

	if (InputType == MCT_Unknown || OutputType == MCT_Unknown)
	{
		bValidThroughput = false;
	}
	else if (bInputTypeIsFloat && bOutputTypeIsFloat)
	{
		// If they are both float types and either of them are MCT_Float then they are a match.
		if (InputType != MCT_Float && OutputType != MCT_Float)
		{
			bValidThroughput = InputType == OutputType;
		}
	}
	else if (InputType != OutputType)
	{
		bValidThroughput = false;
	}

	if (!bValidThroughput)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Function's first input must match its first output."), true, this);
		return false;
	}

	if (Inputs.Num() != InputValues.Num())
	{
		return true;
	}

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		UMaterialExpressionFunctionInput* FunctionInput = Inputs[InputIndex].ExpressionInput.Get();

		if (!IsValid(FunctionInput))
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Function has missing input object."), true, this);
			return false;
		}

		// First input must be a scalar or vector3.
		if (InputIndex == 0)
		{
			switch (FunctionInput->InputType)
			{
				case EFunctionInputType::FunctionInput_Scalar:
				case EFunctionInputType::FunctionInput_Vector3:
					break;

				default:
					UE::DynamicMaterialEditor::Private::LogError(TEXT("Function has invalid first input - must be a scalar or vector3."), true, this);
					return false;
			}

			continue;
		}

		const EDMValueType ValueType = UDMMaterialFunctionFunctionLibrary::GetInputValueType(FunctionInput);

		if (ValueType == EDMValueType::VT_None)
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Function has invalid input type - must be a scalar, vector or texture."), true, this);
			return false;
		}

		if (ValueType != InputValues[InputIndex]->GetType())
		{
			return true;
		}
	}

	return false;
}

void UDMMaterialStageFunction::InitFunction()
{
	if (!IsValid(MaterialFunction.Get()))
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		MaterialFunction = nullptr;
		return;
	}

	UDMMaterialSlot* Slot = GetTypedParent<UDMMaterialSlot>(/* bAllowSubclasses */ true);

	if (!Slot)
	{
		MaterialFunction = nullptr;
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

	if (!EditorOnlyData)
	{
		MaterialFunction = nullptr;
		return;
	}

	UDynamicMaterialModel* MaterialModel = EditorOnlyData->GetMaterialModel();

	if (!MaterialModel)
	{
		MaterialFunction = nullptr;
		return;
	}

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	MaterialFunction->GetInputsAndOutputs(Inputs, Outputs);

	if (Inputs.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Function must have at least one input."), true, this);
		MaterialFunction = nullptr;
		return;
	}

	if (Outputs.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Function must have at least one output."), true, this);
		MaterialFunction = nullptr;
		return;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	const EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();

	{
		FDMMaterialStageConnector PreviousStageConnector = InputConnectors[0];
		InputConnectors.SetNumZeroed(Inputs.Num());
		InputConnectors[0] = PreviousStageConnector;
	}

	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); ++InputIndex)
	{
		UMaterialExpressionFunctionInput* FunctionInput = Inputs[InputIndex].ExpressionInput.Get();

		if (!IsValid(FunctionInput))
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Function has missing input object."), true, this);
			InputConnectors.SetNum(1);
			MaterialFunction = nullptr;
			return;
		}

		// First input must be a scalar or vector3.
		if (InputIndex == InputPreviousStage)
		{
			switch (FunctionInput->InputType)
			{
				case EFunctionInputType::FunctionInput_Scalar:
				case EFunctionInputType::FunctionInput_Vector3:
					break;

				default:
					UE::DynamicMaterialEditor::Private::LogError(TEXT("Function has invalid first input - must be a scalar or vector."), true, this);
					InputConnectors.SetNum(1);
					MaterialFunction = nullptr;
					return;
			}

			Stage->ChangeInput_PreviousStage(
				InputPreviousStage, 
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
				StageProperty,
				0, 
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);

			continue;
		}

		const EDMValueType ValueType = UDMMaterialFunctionFunctionLibrary::GetInputValueType(FunctionInput);

		if (ValueType == EDMValueType::VT_None)
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Function has invalid input type - must be a scalar, vector or texture."), true, this);
			InputConnectors.SetNum(1);
			MaterialFunction = nullptr;
			return;
		}

		InputConnectors[InputIndex].Index = InputIndex;
		InputConnectors[InputIndex].Type = ValueType;

		if (Inputs[InputIndex].ExpressionInput->InputName.IsNone())
		{
			static const FText InputNameFormat = LOCTEXT("InputFormat", "Input {0}");
			InputConnectors[InputIndex].Name = FText::Format(InputNameFormat, FText::AsNumber(InputIndex + 1));
		}
		else
		{
			InputConnectors[InputIndex].Name = FText::FromName(Inputs[InputIndex].ExpressionInput->InputName);
		}

		UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			Stage,
			InputIndex, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			ValueType,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
		
		check(InputValue);

		UDMMaterialValue* Value = InputValue->GetValue();
		check(Value);

		UDMMaterialFunctionFunctionLibrary::SetInputDefault(FunctionInput, Value);
	}
}

void UDMMaterialStageFunction::DeinitFunction()
{
	InputConnectors.SetNum(1);

	if (UDMMaterialStage* Stage = GetStage())
	{
		if (GUndo)
		{
			Stage->Modify();
		}

		Stage->RemoveAllInputs();
	}
}

void UDMMaterialStageFunction::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	if (NeedsFunctionInit())
	{
		InitFunction();
	}

	Super::OnComponentAdded();
}

void UDMMaterialStageFunction::OnComponentRemoved()
{
	DeinitFunction();

	Super::OnComponentRemoved();
}

#undef LOCTEXT_NAMESPACE
