// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageGradient.h"

#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageFunction.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Model/DMMaterialBuildState.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageGradient)

#define LOCTEXT_NAMESPACE "DMMaterialProperty"

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageGradient::Gradients = {};

UDMMaterialStageGradient::UDMMaterialStageGradient()
	: UDMMaterialStageGradient(FText::GetEmpty())
{
}

UDMMaterialStageGradient::UDMMaterialStageGradient(const FText& InName)
	: UDMMaterialStageThroughput(InName)
{
	bAllowNestedInputs = true;

	InputConnectors.Add({InputUV, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
	InputConnectors.Add({InputStart, LOCTEXT("Start", "Start"), EDMValueType::VT_Float3_RGB});
	InputConnectors.Add({InputEnd, LOCTEXT("End", "End"), EDMValueType::VT_Float3_RGB});

	OutputConnectors.Add({0, LOCTEXT("Value", "Value"), EDMValueType::VT_Float3_RGB});
}

bool UDMMaterialStageGradient::SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction)
{
	if (!IsComponentValid())
	{
		return false;
	}

	if (InMaterialFunction == MaterialFunction)
	{
		return false;
	}

	MaterialFunction = InMaterialFunction;

	Update(this, EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);

	return true;
}

UDMMaterialStage* UDMMaterialStageGradient::CreateStage(TSubclassOf<UDMMaterialStageGradient> InMaterialStageGradientClass, UDMMaterialLayerObject* InLayer)
{
	check(InMaterialStageGradientClass);

	GetAvailableGradients();
	check(Gradients.Contains(TStrongObjectPtr<UClass>(InMaterialStageGradientClass.Get())));

	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageGradient* SourceGradient = NewObject<UDMMaterialStageGradient>(
		NewStage, 
		InMaterialStageGradientClass.Get(), 
		NAME_None, 
		RF_Transactional
	);
	
	check(SourceGradient);

	NewStage->SetSource(SourceGradient);

	return NewStage;
}

const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageGradient::GetAvailableGradients()
{
	if (Gradients.IsEmpty())
	{
		GenerateGradientList();
	}

	return Gradients;
}

UDMMaterialStageGradient* UDMMaterialStageGradient::ChangeStageSource_Gradient(UDMMaterialStage* InStage, 
	TSubclassOf<UDMMaterialStageGradient> InGradientClass)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	check(InGradientClass);
	check(!InGradientClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	return InStage->ChangeSource<UDMMaterialStageGradient>(InGradientClass);
}

bool UDMMaterialStageGradient::CanChangeInputType(int32 InputIndex) const
{
	return false;
}

void UDMMaterialStageGradient::AddDefaultInput(int32 InInputIndex) const
{
	check(InputConnectors.IsValidIndex(InInputIndex));

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	if (InInputIndex != InputStart && InInputIndex != InputEnd)
	{
		Super::AddDefaultInput(InInputIndex);
		return;
	}

	UDMMaterialStageInputValue * InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		Stage,
		InInputIndex,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		InputConnectors[InInputIndex].Type,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);
	check(InputValue);

	UDMMaterialValueFloat3RGB* Value = Cast<UDMMaterialValueFloat3RGB>(InputValue->GetValue());

	switch (InInputIndex)
	{
		case InputStart:
			Value->SetDefaultValue(FLinearColor::Black);
			break;

		case InputEnd:
			Value->SetDefaultValue(FLinearColor::White);
			break;
	}

	Value->ApplyDefaultValue();
}

void UDMMaterialStageGradient::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	UMaterialFunctionInterface* ActualMaterialFunction = MaterialFunction;

	if (!IsValid(ActualMaterialFunction))
	{
		ActualMaterialFunction = UDMMaterialStageFunction::NoOp.LoadSynchronous();
	}

	if (!ActualMaterialFunction)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* FunctionCall = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMaterialFunctionCall>(UE_DM_NodeComment_Default);
	FunctionCall->SetMaterialFunction(ActualMaterialFunction);
	FunctionCall->UpdateFromFunctionResource();

	InBuildState->AddStageSourceExpressions(this, {FunctionCall});
}

void UDMMaterialStageGradient::GenerateGradientList()
{
	Gradients.Empty();

	const TArray<TStrongObjectPtr<UClass>>& SourceList = UDMMaterialStageSource::GetAvailableSourceClasses();

	for (const TStrongObjectPtr<UClass>& SourceClass : SourceList)
	{
		UDMMaterialStageGradient* StageGradientCDO = Cast<UDMMaterialStageGradient>(SourceClass->GetDefaultObject(true));

		if (!StageGradientCDO)
		{
			continue;
		}

		Gradients.Add(SourceClass);
	}
}

#undef LOCTEXT_NAMESPACE
