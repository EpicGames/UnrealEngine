// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageInput.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageThroughput.h"
#include "DMComponentPath.h"
#include "Components/DMMaterialLayer.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageInput)

const FString UDMMaterialStageInput::StageInputPrefixStr = FString(TEXT("DMMaterialStageInput"));

void UDMMaterialStageInput::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		MarkComponentDirty();
	}
	
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	if (UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Stage->GetSource()))
	{
		Throughput->Update(InSource, InUpdateType);
		Stage->InputUpdated(this, InUpdateType);
	}
	else
	{
		Stage->Update(InSource, InUpdateType);
	}

	Super::Update(InSource, InUpdateType);
}

void UDMMaterialStageInput::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);

	GenerateExpressions(BuildState);
	UMaterialExpression* StageSourceExpression = BuildState->GetLastStageSourceExpression(this);

	const FDMMaterialStageConnectorChannel* InputChannel = Stage->FindInputChannel(this);

	if (InputChannel
		&& (InputChannel->OutputIndex != 0
			|| InputChannel->OutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL))
	{
		check(OutputConnectors.IsValidIndex(InputChannel->OutputIndex));
		int32 NodeOutputIndex = OutputConnectors[InputChannel->OutputIndex].Index;

		BuildState->GetBuildUtils().UpdatePreviewMaterial(StageSourceExpression, NodeOutputIndex, InputChannel->OutputChannel, 32);
	}
	else
	{
		BuildState->GetBuildUtils().UpdatePreviewMaterial(StageSourceExpression, 0, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 32);
	}
}

FString UDMMaterialStageInput::GetComponentPathComponent() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<UDMMaterialStageInput*>& Inputs = Stage->GetInputs();
		int32 Index = INDEX_NONE;

		for (int32 InputIdx = 0; InputIdx < Inputs.Num(); ++InputIdx)
		{
			if (Inputs[InputIdx] == this)
			{
				Index = InputIdx;
				break;
			}
		}

		return FString::Printf(
			TEXT("%s%c%i%c"),
			*UDMMaterialStage::InputsPathToken,
			FDMComponentPath::ParameterOpen,
			Index,
			FDMComponentPath::ParameterClose
		);
	}

	return Super::GetComponentPathComponent();
}

void UDMMaterialStageInput::GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const
{
	// Strip off the type index
	if (OutChildComponentPathComponents.IsEmpty() == false)
	{
		int32 IndexOfUnderscore = INDEX_NONE;

		if (OutChildComponentPathComponents.Last().FindChar('_', IndexOfUnderscore))
		{
			OutChildComponentPathComponents.Last().LeftInline(IndexOfUnderscore);
		}
	}

	Super::GetComponentPathInternal(OutChildComponentPathComponents);
}
