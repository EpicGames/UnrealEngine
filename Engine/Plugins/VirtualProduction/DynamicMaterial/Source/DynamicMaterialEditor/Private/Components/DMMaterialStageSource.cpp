// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageSource.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "DynamicMaterialEditorModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "MaterialValueType.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageSource)

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageSource::SourceClasses = TArray<TStrongObjectPtr<UClass>>();

UDMMaterialStage* UDMMaterialStageSource::GetStage() const
{
	return Cast<UDMMaterialStage>(GetOuterSafe());
}

void UDMMaterialStageSource::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		MarkComponentDirty();
	}

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	Stage->Update(InSource, InUpdateType);

	Super::Update(InSource, InUpdateType);
}

void UDMMaterialStageSource::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	Super::OnComponentAdded();

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageSource::GetMaskAlphaBlendNode(const TSharedRef<FDMMaterialBuildState>& InBuildState, UMaterialExpression*& OutExpression, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	OutExpression = nullptr;
}

const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageSource::GetAvailableSourceClasses()
{
	if (SourceClasses.IsEmpty())
	{
		GenerateClassList();
	}

	return SourceClasses;
}

void UDMMaterialStageSource::GenerateClassList()
{
	SourceClasses.Empty();

	for (TObjectIterator<UClass> ClassIT; ClassIT; ++ClassIT)
	{
		TSubclassOf<UDMMaterialStageSource> MSEClass = *ClassIT;

		if (!MSEClass.Get())
		{
			continue;
		}

		if (MSEClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags))
		{
			continue;
		}

		SourceClasses.Add(TStrongObjectPtr<UClass>(MSEClass));
	}
}

void UDMMaterialStageSource::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}

	UE_LOGF(LogDynamicMaterialEditor, Display, "Building Material Designer Source Preview (%ls)...", *GetName());

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);

	GenerateExpressions(BuildState);
	UMaterialExpression* StageSourceExpression = BuildState->GetLastStageSourceExpression(this);

	BuildState->GetBuildUtils().UpdatePreviewMaterial(StageSourceExpression, 0, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 32);
}

int32 UDMMaterialStageSource::GetInnateMaskOutput(int32 OutputIndex, int32 OutputChannels) const
{
	return INDEX_NONE;
}

bool UDMMaterialStageSource::GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
	UMaterialExpression*& OutMaterialExpression, int32& OutputIndex)
{
	check(InStage);
	check(InPreviewMaterial);

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(InStage);

	UDMMaterialStageSource* PreviewSource = InStage->GetSource();

	if (!PreviewSource)
	{
		return false;
	}

	PreviewSource->GenerateExpressions(BuildState);
	const TArray<UMaterialExpression*>& SourceExpressions = BuildState->GetStageSourceExpressions(PreviewSource);

	if (SourceExpressions.IsEmpty())
	{
		return false;
	}

	UMaterialExpression* LastExpression = SourceExpressions.Last();

	bool bIsMaskStage = false;

	if (Layer->GetStageType(InStage) == EDMMaterialLayerStage::Mask)
	{
		bIsMaskStage = true;
	}

	int32 BestMatch = INDEX_NONE;
	int32 OutputCount = 0;
	const int32 FloatsForPropertyType = bIsMaskStage ? 1 : 3;

	for (int32 OutputIdx = 0; OutputIdx < LastExpression->GetOutputs().Num(); ++OutputIdx)
	{
		EMaterialValueType CurrentOutputType = LastExpression->GetOutputValueType(OutputIdx);
		int32 CurrentOutputCount = 0;

		switch (CurrentOutputType)
		{
			case MCT_Float:
			case MCT_Float1:
				CurrentOutputCount = 1;
				break;

			case MCT_Float2:
				CurrentOutputCount = 2;
				break;

			case MCT_Float3:
				CurrentOutputCount = 3;
				break;

			case MCT_Float4:
				CurrentOutputCount = 4;
				break;

			default:
				continue; // For loop
		}

		if (CurrentOutputCount > OutputCount)
		{
			BestMatch = OutputIdx;
			OutputCount = CurrentOutputCount;

			if (CurrentOutputCount >= FloatsForPropertyType)
			{
				break;
			}
		}
	}

	if (BestMatch != INDEX_NONE)
	{
		OutMaterialExpression = LastExpression;
		OutputIndex = BestMatch;
		return true;
	}

	return false;
}

void UDMMaterialStageSource::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, class FEditPropertyChain* InPropertyThatChanged)
{
}

void UDMMaterialStageSource::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsComponentValid())
	{
		return;
	}

	MarkComponentDirty();
	Update(this, EDMUpdateType::Structure);
}

UDMMaterialComponent* UDMMaterialStageSource::GetParentComponent() const
{
	return GetStage();
}
