// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialStageThroughput.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSITextureUV.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Containers/ArrayView.h"
#include "DMDefs.h"
#include "DynamicMaterialEditorModule.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMInputNodeBuilder.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageThroughput)

#define LOCTEXT_NAMESPACE "DMMaterialStageThroughput"

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageThroughput::Throughputs = {};

const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageThroughput::GetAvailableThroughputs()
{
	if (Throughputs.IsEmpty())
	{
		GenerateThroughputList();
	}

	return Throughputs;
}

bool UDMMaterialStageThroughput::CanInputAcceptType(int32 InThroughputInputIndex, EDMValueType InValueType) const
{
	check(InputConnectors.IsValidIndex(InThroughputInputIndex));

	return InputConnectors[InThroughputInputIndex].IsCompatibleWith(InValueType);
}

bool UDMMaterialStageThroughput::CanInputConnectTo(int32 InThroughputInputIndex, const FDMMaterialStageConnector& InOutputConnector, 
	int32 InOutputChannel, bool bInCheckSingleFloat)
{
	if (InOutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		if (CanInputAcceptType(InThroughputInputIndex, InOutputConnector.Type))
		{
			return true;
		}

		if (UDMValueDefinitionLibrary::GetValueDefinition(InOutputConnector.Type).IsFloatType() && bInCheckSingleFloat)
		{
			return CanInputAcceptType(InThroughputInputIndex, EDMValueType::VT_Float1);
		}

		return false;
	}

	if (UDMValueDefinitionLibrary::GetValueDefinition(InOutputConnector.Type).IsFloatType())
	{
		return CanInputAcceptType(InThroughputInputIndex, EDMValueType::VT_Float1);
	}

	// Only float types can have non-whole channels.
	checkNoEntry();
	return false;
}

bool UDMMaterialStageThroughput::CanChangeInput(int32 InThroughputInputIndex) const
{
	return true;
}

bool UDMMaterialStageThroughput::CanChangeInputType(int32 InThroughputInputIndex) const
{
	check(InputConnectors.IsValidIndex(InThroughputInputIndex));

	return InputConnectors[InThroughputInputIndex].Type != EDMValueType::VT_Texture;
}

bool UDMMaterialStageThroughput::IsInputVisible(int32 InThroughputInputIndex) const
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(Stage))
	{
		Stage = SubStage->GetParentMostStage();
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	bool bSupportsUVLink = false;

	if (SupportsLayerMaskTextureUVLink())
	{
		if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base))
		{
			if (BaseStage->IsEnabled())
			{
				if (UDMMaterialStageThroughput* BaseThroughput = Cast<UDMMaterialStageThroughput>(BaseStage->GetSource()))
				{
					bSupportsUVLink = BaseThroughput->SupportsLayerMaskTextureUVLink();
				}
			}
		}
	}

	if (bSupportsUVLink && InThroughputInputIndex == GetLayerMaskTextureUVLinkInputIndex())
	{
		return !Layer->IsTextureUVLinkEnabled() || Layer->GetStageType(Stage) != EDMMaterialLayerStage::Mask;
	}

	return true;
}

bool UDMMaterialStageThroughput::ShouldKeepInput(int32 InThroughputInputIndex)
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	if (!CanChangeInput(InThroughputInputIndex) || !Stage->IsInputMapped(InThroughputInputIndex))
	{
		return false;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	const EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
	const TArray<FDMMaterialStageConnection>& InputConnectionMap = Stage->GetInputConnectionMap();

	if (!InputConnectionMap.IsValidIndex(InThroughputInputIndex))
	{
		return false;
	}

	if (InputConnectors[InThroughputInputIndex].Type == EDMValueType::VT_Float_Any || InputConnectors[InThroughputInputIndex].Type == EDMValueType::VT_Texture)
	{
		return false;
	}

	const int32 RequiredInputCount = UDMValueDefinitionLibrary::GetValueDefinition(InputConnectors[InThroughputInputIndex].Type).GetFloatCount();

	int32 ActualInputCount = 0;

	for (const FDMMaterialStageConnectorChannel& Channel : InputConnectionMap[InThroughputInputIndex].Channels)
	{
		int32 ThisInputCount = 0;

		switch (Channel.SourceIndex)
		{
			case FDMMaterialStageConnectorChannel::NO_SOURCE:
				return false; // Some sort of error or badly mapped input.

			case FDMMaterialStageConnectorChannel::PREVIOUS_STAGE:
			{
				if (UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base))
				{
					if (UDMMaterialStage* MaskStage = PreviousLayer->GetStage(EDMMaterialLayerStage::Mask))
					{
						check(MaskStage->GetSource());

						const TArray<FDMMaterialStageConnector>& PreviousStageOutputConnectors = MaskStage->GetSource()->GetOutputConnectors();
						check(PreviousStageOutputConnectors.IsValidIndex(Channel.OutputIndex));

						if (UDMValueDefinitionLibrary::GetValueDefinition(PreviousStageOutputConnectors[Channel.OutputIndex].Type).IsFloatType() == false)
						{
							check(InputConnectionMap[InThroughputInputIndex].Channels.Num() == 1);

							return InputConnectors[InThroughputInputIndex].IsCompatibleWith(PreviousStageOutputConnectors[Channel.OutputIndex]);
						}
						else
						{
							check(RequiredInputCount > 0);
						}

						if (Channel.OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
						{
							ThisInputCount = UDMValueDefinitionLibrary::GetValueDefinition(PreviousStageOutputConnectors[Channel.OutputIndex].Type).GetFloatCount();
						}
						else
						{
							ThisInputCount =
								!!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL)
								+ !!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL)
								+ !!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL)
								+ !!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);

							ThisInputCount = FMath::Min(
								ThisInputCount,
								static_cast<int32>(UDMValueDefinitionLibrary::GetValueDefinition(PreviousStageOutputConnectors[Channel.OutputIndex].Type).GetFloatCount())
							);
						}
					}
				}

				break;
			}

			// Input
			default:
			{
				const int32 InputObjectIdx = Channel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
				const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();
				check(StageInputs.IsValidIndex(InputObjectIdx));

				const TArray<FDMMaterialStageConnector>& InputOutputConnectors = StageInputs[InputObjectIdx]->GetOutputConnectors();
				check(InputOutputConnectors.IsValidIndex(Channel.OutputIndex));

				if (UDMValueDefinitionLibrary::GetValueDefinition(InputOutputConnectors[Channel.OutputIndex].Type).IsFloatType() == false)
				{
					check(InputConnectionMap[InThroughputInputIndex].Channels.Num() == 1);

					return InputConnectors[InThroughputInputIndex].IsCompatibleWith(InputOutputConnectors[Channel.OutputIndex]);
				}
				else
				{
					check(RequiredInputCount > 0);
				}

				if (Channel.OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
				{
					ThisInputCount = UDMValueDefinitionLibrary::GetValueDefinition(InputOutputConnectors[Channel.OutputIndex].Type).GetFloatCount();
				}
				else
				{
					ThisInputCount =
						!!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL)
						+ !!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL)
						+ !!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL)
						+ !!(Channel.OutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);

					ThisInputCount = FMath::Min(
						ThisInputCount,
						static_cast<int32>(UDMValueDefinitionLibrary::GetValueDefinition(InputOutputConnectors[Channel.OutputIndex].Type).GetFloatCount())
					);
				}

				break;
			}
		}

		ActualInputCount += ThisInputCount;
	}

	// An input size of 1 float will always work.
	return (ActualInputCount == 1) || (ActualInputCount == RequiredInputCount);
}

void UDMMaterialStageThroughput::ConnectOutputToInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex,
	int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel)
{
	check(InSourceExpression);
	check(InSourceExpression->GetOutputs().IsValidIndex(InSourceOutputIndex));

	const TArray<UMaterialExpression*>& TargetExpressions = InBuildState->GetStageSourceExpressions(this);
	check(!TargetExpressions.IsEmpty());

	UMaterialExpression* TargetExpression = GetExpressionForInput(TargetExpressions, InThroughputInputIndex, InExpressionInputIndex);
	check(TargetExpression->GetInput(InExpressionInputIndex));

	ConnectOutputToInput_Internal(
		InBuildState, 
		TargetExpression, 
		InExpressionInputIndex, 
		InSourceExpression, 
		InSourceOutputIndex, 
		InSourceOutputChannel
	);
}

void UDMMaterialStageThroughput::ConnectOutputToInput_Internal(const TSharedRef<FDMMaterialBuildState>& InBuildState, UMaterialExpression* InTargetExpression,
	int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel) const
{
	check(InTargetExpression != InSourceExpression);

	FExpressionInput* ExpressionInput = InTargetExpression->GetInput(InExpressionInputIndex);
	
	if (InSourceOutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		InSourceExpression->ConnectExpression(ExpressionInput, InSourceOutputIndex);
	}
	else if (InSourceExpression->IsA<UMaterialExpressionTextureBase>()
		&& (InSourceOutputChannel == FDMMaterialStageConnectorChannel::FIRST_CHANNEL
			|| InSourceOutputChannel == FDMMaterialStageConnectorChannel::SECOND_CHANNEL
			|| InSourceOutputChannel == FDMMaterialStageConnectorChannel::THIRD_CHANNEL
			|| InSourceOutputChannel == FDMMaterialStageConnectorChannel::FOURTH_CHANNEL))
	{
		switch (InSourceOutputChannel)
		{
			case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
				InSourceExpression->ConnectExpression(ExpressionInput, 1);
				break;

			case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
				InSourceExpression->ConnectExpression(ExpressionInput, 2);
				break;

			case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
				InSourceExpression->ConnectExpression(ExpressionInput, 3);
				break;

			case FDMMaterialStageConnectorChannel::FOURTH_CHANNEL:
				InSourceExpression->ConnectExpression(ExpressionInput, 4);
				break;

			default:
				checkNoEntry();
		}
	}
	else
	{
		const int32 MaskedOutput = InBuildState->GetBuildUtils().FindOutputForBitmask(
			InSourceExpression,
			InSourceOutputChannel
		);

		if (MaskedOutput == INDEX_NONE)
		{
			UMaterialExpression* MaskExpression = InBuildState->GetBuildUtils().CreateExpressionBitMask(
				InSourceExpression,
				InSourceOutputIndex,
				InSourceOutputChannel
			);

			MaskExpression->ConnectExpression(ExpressionInput, 0);
		}
		else
		{
			InSourceExpression->ConnectExpression(ExpressionInput, MaskedOutput);
		}
	}
}

int32 UDMMaterialStageThroughput::GetLayerMaskTextureUVLinkInputIndex() const
{
	return INDEX_NONE;
}

FDMExpressionInput UDMMaterialStageThroughput::GetLayerMaskLinkTextureUVInputExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	check(SupportsLayerMaskTextureUVLink());

	int32 TextureUVInputIdx = GetLayerMaskTextureUVLinkInputIndex();
	check(TextureUVInputIdx != INDEX_NONE);

	FDMMaterialStageConnectorChannel Channel;
	FDMExpressionInput ExpressionInput;

	ExpressionInput.OutputIndex = ResolveInput(
		InBuildState, 
		TextureUVInputIdx, 
		Channel, 
		ExpressionInput.OutputExpressions
	);

	ExpressionInput.OutputChannel = Channel.OutputChannel;

	return ExpressionInput;
}

UMaterialExpression* UDMMaterialStageThroughput::GetExpressionForInput(const TArray<UMaterialExpression*>& InStageSourceExpressions, 
	int32 InThroughputInputIndex, int32 InExpressionInputIndex)
{
	check(!InStageSourceExpressions.IsEmpty());

	return InStageSourceExpressions[0];
}

void UDMMaterialStageThroughput::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	Super::OnComponentAdded();

	if (IsComponentValid())
	{
		const FDMUpdateGuard Guard;

		for (int32 InputIdx = 0; InputIdx < InputConnectors.Num(); ++InputIdx)
		{
			if (ShouldKeepInput(InputIdx))
			{
				continue;
			}

			AddDefaultInput(InputIdx);
		}
	}
}

void UDMMaterialStageThroughput::AddDefaultInput(int32 InInputIndex) const
{
	if (!IsComponentValid())
	{
		return;
	}

	static const FText UVName = LOCTEXT("UV", "UV");
	
	check(InputConnectors.IsValidIndex(InInputIndex));

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	switch (InputConnectors[InInputIndex].Type)
	{
		default:
			checkNoEntry();
			break;

		case EDMValueType::VT_None:
		case EDMValueType::VT_Float_Any:
			break;

		case EDMValueType::VT_Float2:
			if (InputConnectors[InInputIndex].Name.EqualTo(UVName))
			{
				UDMMaterialStageInputTextureUV::ChangeStageInput_UV(
					Stage, 
					InInputIndex,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
				);
			}
			else
			{
				UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
					Stage, 
					InInputIndex,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
					EDMValueType::VT_Float2,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
				);
			}
			break;

		case EDMValueType::VT_Float3_RGB:
			UDMMaterialStageInputExpression::ChangeStageInput_Expression(
				Stage, 
				UDMMaterialStageExpressionTextureSample::StaticClass(),
				InInputIndex,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
				0,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);
			break;

		case EDMValueType::VT_Float4_RGBA:
			UDMMaterialStageInputExpression::ChangeStageInput_Expression(
				Stage,
				UDMMaterialStageExpressionTextureSample::StaticClass(), 
				InInputIndex,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
				5,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);
			break;

		case EDMValueType::VT_Float1:
		case EDMValueType::VT_Float3_RPY:
		case EDMValueType::VT_Float3_XYZ:
		case EDMValueType::VT_Texture:
			UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
				Stage, 
				InInputIndex,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
				InputConnectors[InInputIndex].Type,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);
			break;

		case EDMValueType::VT_ColorAtlas:
		{
			if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
			{
				if (Layer->GetStageType(Stage) == EDMMaterialLayerStage::Mask)
				{
					UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
						Stage, 
						InInputIndex,
						FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
						EDMValueType::VT_ColorAtlas,
						FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
					);
					break;
				}
			}

			UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
				Stage, 
				InInputIndex,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
				EDMValueType::VT_ColorAtlas,
				FDMMaterialStageConnectorChannel::THREE_CHANNELS
			);
			break;
		}
	}
}

int32 UDMMaterialStageThroughput::ResolveInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex, FDMMaterialStageConnectorChannel& OutChannel,
	TArray<UMaterialExpression*>& OutExpressions) const
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialStage* ParentMostStage = Stage;

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(Stage))
	{
		ParentMostStage = SubStage->GetParentMostStage();
	}

	UDMMaterialLayerObject* Layer = ParentMostStage->GetLayer();
	check(Layer);

	if (Layer->IsTextureUVLinkEnabled() 
		&& Layer->GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true)
		&& Layer->GetStageType(ParentMostStage) == EDMMaterialLayerStage::Mask
		&& SupportsLayerMaskTextureUVLink() 
		&& (GetLayerMaskTextureUVLinkInputIndex() == INDEX_NONE || GetLayerMaskTextureUVLinkInputIndex() == InThroughputInputIndex))
	{
		const int32 OutputIndex = ResolveLayerMaskTextureUVLinkInput(
			InBuildState,
			InThroughputInputIndex, 
			OutChannel, 
			OutExpressions
		);

		if (OutputIndex != INDEX_NONE)
		{
			return OutChannel.OutputIndex;
		}
	}

	const TArray<FDMMaterialStageConnection>& InputConnectionMap = Stage->GetInputConnectionMap();

	if (!InputConnectionMap.IsValidIndex(InThroughputInputIndex))
	{
		return INDEX_NONE;
	}

	if (InputConnectionMap[InThroughputInputIndex].Channels.IsEmpty())
	{
		return INDEX_NONE;
	}

	if (InputConnectionMap[InThroughputInputIndex].Channels.Num() == 1)
	{
		// Full copy in case it is changed by the channel resolve.
		OutChannel = InputConnectionMap[InThroughputInputIndex].Channels[0];

		return ResolveInputChannel(
			InBuildState, 
			InThroughputInputIndex, 
			0, 
			OutChannel, 
			OutExpressions
		);
	}

	// Only floats can have sub-channel mapping
	check(UDMValueDefinitionLibrary::GetValueDefinition(InputConnectors[InThroughputInputIndex].Type).IsFloatType());

	// Valid scalar/vector is from 1 to 4 floats.
	check(InputConnectionMap[InThroughputInputIndex].Channels.Num() <= 4);

	for (const FDMMaterialStageConnectorChannel& Channel : InputConnectionMap[InThroughputInputIndex].Channels)
	{
		check(Channel.OutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);
	}

	OutChannel.OutputIndex = 0;
	OutChannel.OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

	UMaterialExpression* MakeFloat = nullptr;

	switch (InputConnectionMap[InThroughputInputIndex].Channels.Num())
	{
		case 2:
			MakeFloat = FDMMaterialFunctionLibrary::Get().GetMakeFloat2(InBuildState->GetDynamicMaterial(), UE_DM_NodeComment_Default);
			break;

		case 3:
			MakeFloat = FDMMaterialFunctionLibrary::Get().GetMakeFloat3(InBuildState->GetDynamicMaterial(), UE_DM_NodeComment_Default);
			break;

		case 4:
			MakeFloat = FDMMaterialFunctionLibrary::Get().GetMakeFloat3(InBuildState->GetDynamicMaterial(), UE_DM_NodeComment_Default);
			break;

		default:
			checkNoEntry();
			return INDEX_NONE;
	}

	for (int32 ChannelIdx = 0; ChannelIdx < InputConnectionMap[InThroughputInputIndex].Channels.Num(); ++ChannelIdx)
	{
		FDMMaterialStageConnectorChannel ChannelTemp = InputConnectionMap[InThroughputInputIndex].Channels[ChannelIdx];
		TArray<UMaterialExpression*> ChannelExpressions;

		ResolveInputChannel(
			InBuildState, 
			InThroughputInputIndex, 
			ChannelIdx, 
			ChannelTemp, 
			ChannelExpressions
		);

		check(ChannelExpressions.IsEmpty() == false);

		OutExpressions.Append(ChannelExpressions);

		ChannelExpressions.Last()->ConnectExpression(MakeFloat->GetInput(ChannelIdx), ChannelTemp.OutputIndex);
	}

	OutExpressions.Add(MakeFloat);

	return 0;
}

int32 UDMMaterialStageThroughput::ResolveLayerMaskTextureUVLinkInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex, 
	FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);
	check(Layer->GetStage(EDMMaterialLayerStage::Base));

	return ResolveLayerMaskTextureUVLinkInputImpl(
		InBuildState, 
		Layer->GetStage(EDMMaterialLayerStage::Base)->GetSource(), 
		OutChannel, 
		OutExpressions
	);
}

int32 UDMMaterialStageThroughput::ResolveLayerMaskTextureUVLinkInputImpl(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
	const UDMMaterialStageSource* InStageSource, FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions)
{
	FDMExpressionInput ConnectedInputExpressions;

	if (const UDMMaterialStageInputTextureUV* InputTextureUV = Cast<const UDMMaterialStageInputTextureUV>(InStageSource))
	{
		InputTextureUV->GenerateExpressions(InBuildState);
		OutExpressions = InBuildState->GetStageSourceExpressions(InputTextureUV);
		OutChannel.OutputIndex = 0;
		OutChannel.OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

		return 0;
	}

	else if (const UDMMaterialStageThroughput* Throughput = Cast<const UDMMaterialStageThroughput>(InStageSource))
	{
		if (Throughput->SupportsLayerMaskTextureUVLink())
		{
			ConnectedInputExpressions = Throughput->GetLayerMaskLinkTextureUVInputExpressions(InBuildState);
		}
	}

	else if (const UDMMaterialStageInputThroughput* InputThroughput = Cast<const UDMMaterialStageInputThroughput>(InStageSource))
	{
		if (InputThroughput->GetMaterialStageThroughput())
		{
			UDMMaterialStageThroughput* InputThroughputActual = InputThroughput->GetMaterialStageThroughput();
			check(InputThroughputActual);

			if (InputThroughputActual->SupportsLayerMaskTextureUVLink())
			{
				ConnectedInputExpressions = InputThroughputActual->GetLayerMaskLinkTextureUVInputExpressions(InBuildState);
			}
		}
	}

	if (ConnectedInputExpressions.IsValid())
	{
		OutChannel.OutputIndex = ConnectedInputExpressions.OutputIndex;
		OutChannel.OutputChannel = ConnectedInputExpressions.OutputChannel;
		OutExpressions = ConnectedInputExpressions.OutputExpressions;

		return ConnectedInputExpressions.OutputIndex;
	}

	return INDEX_NONE;
}

int32 UDMMaterialStageThroughput::ResolveInputChannel(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex, int32 InChannelIndex,
	FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const
{
	if (OutChannel.SourceIndex == FDMMaterialStageConnectorChannel::NO_SOURCE)
	{
		return INDEX_NONE;
	}

	check(InputConnectors.IsValidIndex(InThroughputInputIndex));

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	const TArray<FDMMaterialStageConnector>* InputSourceOutputConnectors = nullptr;
	int32 InnateMaskOutput = INDEX_NONE;
	int32 NodeOutputIndex = INDEX_NONE;
	int32 OutputChannelOverride = INDEX_NONE;
	UMaterialExpression* LastExpression = nullptr;

	if (OutChannel.SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
	{
		UDMMaterialLayerObject* Layer = Stage->GetLayer();
		check(Layer);

		if (UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(OutChannel.MaterialProperty, EDMMaterialLayerStage::Base))
		{
			PreviousLayer->GenerateExpressions(InBuildState);

			const TArray<UMaterialExpression*>& PreviousLayerExpressions = InBuildState->GetLayerExpressions(PreviousLayer);
			check(!PreviousLayerExpressions.IsEmpty());
			LastExpression = PreviousLayerExpressions.Last();
			OutExpressions.Add(LastExpression);

			UDMMaterialStage* PreviousStage = PreviousLayer->GetLastValidStage(EDMMaterialLayerStage::All);
			check(PreviousStage);

			UDMMaterialStageSource* PreviousStageSource = PreviousStage->GetSource();
			check(PreviousStageSource);

			InputSourceOutputConnectors = &(PreviousStageSource->GetOutputConnectors());
			check(InputSourceOutputConnectors->IsValidIndex(OutChannel.OutputIndex));

			NodeOutputIndex = (*InputSourceOutputConnectors)[OutChannel.OutputIndex].Index;
			InnateMaskOutput = PreviousStageSource->GetInnateMaskOutput(NodeOutputIndex, OutChannel.OutputChannel);
		}
		else
		{
			UMaterialExpressionConstant3Vector* Black = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant3Vector>(UE_DM_NodeComment_Default);
			Black->Constant = FLinearColor(0, 0, 0, 1);

			LastExpression = Black;
			OutExpressions.Add(Black);

			NodeOutputIndex = 0;
			InnateMaskOutput = INDEX_NONE;
		}
	}
	else
	{
		const TArray<UDMMaterialStageInput*> StageInputs = Stage->GetInputs();

		const int32 StageInputIdx = OutChannel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
		check(StageInputs.IsValidIndex(StageInputIdx));

		UDMMaterialStageInput* InputValue = StageInputs[StageInputIdx];
		InputValue->GenerateExpressions(InBuildState);

		OutExpressions.Append(InBuildState->GetStageSourceExpressions(InputValue));
		
		if (OutExpressions.IsEmpty())
		{
			UE_LOGF(LogDynamicMaterialEditor, Error, "No expressions generated!");

			UMaterialExpressionScalarParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionScalarParameter>(
				UE_DM_NodeComment_Default
			);

			NewExpression->DefaultValue = 0.f;
			LastExpression = NewExpression;
		}
		else
		{
			LastExpression = OutExpressions.Last();
		}

		InputSourceOutputConnectors = &(InputValue->GetOutputConnectors());
		check(InputSourceOutputConnectors->IsValidIndex(OutChannel.OutputIndex));

		// Change from output index from Material Designer node to output index of the material expression
		NodeOutputIndex = (*InputSourceOutputConnectors)[OutChannel.OutputIndex].Index;
		InnateMaskOutput = InputValue->GetInnateMaskOutput(NodeOutputIndex, OutChannel.OutputChannel);
		OutputChannelOverride = InputValue->GetOutputChannelOverride(OutChannel.OutputIndex);
	}

	// If our "Previous Stage" is blank, these need default values.
	const int32 OutputFloatCount = InputSourceOutputConnectors ? UDMValueDefinitionLibrary::GetValueDefinition((*InputSourceOutputConnectors)[OutChannel.OutputIndex].Type).GetFloatCount() : 3;
	const bool bOutputIsFloatType = InputSourceOutputConnectors ? UDMValueDefinitionLibrary::GetValueDefinition((*InputSourceOutputConnectors)[OutChannel.OutputIndex].Type).IsFloatType() : true;
	const int32 InputFloatCount = UDMValueDefinitionLibrary::GetValueDefinition(InputConnectors[InThroughputInputIndex].Type).GetFloatCount();

	if (OutChannel.OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		&& bOutputIsFloatType && OutputFloatCount > 0 && InputFloatCount > 0)
	{
		if (OutputFloatCount > InputFloatCount)
		{
			int32 ComponentMask = 0;

			for (int32 FloatIdx = 1; FloatIdx <= InputFloatCount; ++FloatIdx)
			{
				ComponentMask += UE::DynamicMaterialEditor::Private::ChannelIndexToChannelBit(FloatIdx);
			}

			OutChannel.OutputChannel = ComponentMask;
		}
	}

	if (OutputChannelOverride != INDEX_NONE)
	{
		OutChannel.OutputChannel = OutputChannelOverride;
	}

	if (InnateMaskOutput != INDEX_NONE)
	{
		OutChannel.OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

		return InnateMaskOutput;
	}

	if (OutChannel.OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		&& bOutputIsFloatType && OutputFloatCount > 0 && InputFloatCount > 0 && OutputFloatCount < InputFloatCount)
	{
		UMaterialExpressionAppendVector* Append = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionAppendVector>(UE_DM_NodeComment_Default);
		Append->A.Expression = LastExpression;
		Append->A.OutputIndex = NodeOutputIndex;

		switch (InputFloatCount - OutputFloatCount)
		{
			case 1:
			{
				UMaterialExpressionConstant* Constant = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant>(UE_DM_NodeComment_Default);
				OutExpressions.Add(Constant);
				Constant->R = 0.f;
				Append->B.Expression = Constant;
				break;
			}

			case 2:
			{
				UMaterialExpressionConstant2Vector* Constant2 = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant2Vector>(UE_DM_NodeComment_Default);
				OutExpressions.Add(Constant2);
				Constant2->R = 0.f;
				Constant2->G = 0.f;
				Append->B.Expression = Constant2;
				break;
			}

			case 3:
			{
				UMaterialExpressionConstant3Vector* Constant3 = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant3Vector>(UE_DM_NodeComment_Default);
				OutExpressions.Add(Constant3);
				Constant3->Constant = FLinearColor::Black;
				Append->B.Expression = Constant3;
				break;
			}

			default:
				checkNoEntry();
				break;
		}

		Append->B.OutputIndex = 0;
		OutExpressions.Add(Append);

		OutChannel.OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

		return 0;
	}

	return NodeOutputIndex;
}

void UDMMaterialStageThroughput::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	const TArray<FDMMaterialStageConnection>& InputConnectionMap = Stage->GetInputConnectionMap();
	const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();
	TArray<UE::DynamicMaterialEditor::Private::FDMInputInputs> Inputs; // There can be multiple inputs per input
	bool bHasStageInput = false;

	for (int32 InputIdx = 0; InputIdx < InputConnectors.Num(); ++InputIdx)
	{
		TArray<UDMMaterialStageInput*> ChannelInputs;

		if (!InputConnectionMap.IsValidIndex(InputIdx))
		{
			continue;
		}

		ChannelInputs.SetNum(InputConnectionMap[InputIdx].Channels.Num());
		bool bNonStageInput = false;

		for (int32 ChannelIdx = 0; ChannelIdx < InputConnectionMap[InputIdx].Channels.Num(); ++ChannelIdx)
		{
			if (InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
			{
				bHasStageInput = true;
				ChannelInputs[ChannelIdx] = nullptr;
				continue;
			}

			if (InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
			{
				int32 StageInputIdx = InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
				ChannelInputs[ChannelIdx] = StageInputs[StageInputIdx];
				bNonStageInput = true;
				continue;
			}
		}

		if (ChannelInputs.IsEmpty())
		{
			continue;
		}

		if (bNonStageInput)
		{
			Inputs.Add({InputIdx, ChannelInputs});
		}
	}

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);

	if (!bHasStageInput || Inputs.IsEmpty())
	{
		Stage->GenerateExpressions(BuildState);
		UMaterialExpression* StageExpression = BuildState->GetLastStageExpression(Stage);

		BuildState->GetBuildUtils().UpdatePreviewMaterial(
			StageExpression, 
			0, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			32
		);
	}
	else
	{
		UE::DynamicMaterialEditor::Private::BuildExpressionInputs(
			BuildState,
			InputConnectionMap, 
			Inputs
		);
	}
}

void UDMMaterialStageThroughput::GenerateThroughputList()
{
	Throughputs.Empty();

	const TArray<TStrongObjectPtr<UClass>>& SourceList = UDMMaterialStageSource::GetAvailableSourceClasses();

	for (const TStrongObjectPtr<UClass>& SourceClass : SourceList)
	{
		UDMMaterialStageThroughput* StageThroughputCDO = Cast<UDMMaterialStageThroughput>(SourceClass->GetDefaultObject(true));

		if (!StageThroughputCDO)
		{
			continue;
		}

		Throughputs.Add(SourceClass);
	}
}

UDMMaterialStageThroughput::UDMMaterialStageThroughput()
	: UDMMaterialStageThroughput(FText::GetEmpty())
{
}

UDMMaterialStageThroughput::UDMMaterialStageThroughput(const FText& InName)
	: Name(InName)
{
}

#undef LOCTEXT_NAMESPACE
