// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/DMMaterialStageThroughputLayerBlend.h"
 
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DMEDefs.h"
#include "DynamicMaterialEditorSettings.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMInputNodeBuilder.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageThroughputLayerBlend)

#define LOCTEXT_NAMESPACE "DMMaterialStageThroughputLayerBlend"

namespace UE::DynamicMaterialEditor::Private
{
	UMaterialFunctionInterface* GetAlphaBlend()
	{
		static UMaterialFunctionInterface* AlphaBlend = FDMMaterialFunctionLibrary::Get().GetFunction(
			"DM_AlphaBlend",
			TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_AlphaBlend.MF_DM_AlphaBlend'")
		);

		return AlphaBlend;
	}
}
 
UDMMaterialStage* UDMMaterialStageThroughputLayerBlend::CreateStage(UDMMaterialLayerObject* InLayer)
{
		const FDMUpdateGuard Guard;
 
	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);
 
	UDMMaterialStageThroughputLayerBlend* NewLayerBlend = NewObject<UDMMaterialStageThroughputLayerBlend>(NewStage, NAME_None, RF_Transactional);
	check(NewLayerBlend);
 
	NewStage->SetSource(NewLayerBlend);
	NewStage->SetCanChangeSource(false);
 
	return NewStage;
}
 
UDMMaterialStageThroughputLayerBlend::UDMMaterialStageThroughputLayerBlend()
	: UDMMaterialStageThroughput(
		LOCTEXT("LayerBlend", "Layer Blend")
	)
{
	InputConnectors.Add({InputPreviousLayer, LOCTEXT("PreviousLayer", "Previous Layer"), EDMValueType::VT_Float_Any});
	InputConnectors.Add({InputBaseStage, LOCTEXT("BaseStage", "Base Stage"), EDMValueType::VT_Float_Any});
	InputConnectors.Add({InputMaskSource, LOCTEXT("MaskSource", "Mask Source"), EDMValueType::VT_Float1});
 
	OutputConnectors.Add({0, LOCTEXT("Blend", "Blend"), EDMValueType::VT_Float_Any});
 
	bInputRequired = true;
	bAllowNestedInputs = true;
	bPremultiplyAlpha = true;
	MaskChannelOverride = EAvaColorChannel::None;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageThroughputLayerBlend, MaskChannelOverride));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageThroughputLayerBlend, bPremultiplyAlpha));
 
	bBlockUpdate = false;
	bIsAlphaOnlyBlend = false;
}
 
void UDMMaterialStageThroughputLayerBlend::OnComponentAdded()
{
	Super::OnComponentAdded();
 
	if (!IsComponentValid())
	{
		return;
	}

	InitBlendStage();
 
	UpdateAlphaOnlyMaskStatus();
 
	if (bIsAlphaOnlyBlend)
	{
		UpdateAlphaOnlyMasks(EDMUpdateType::Structure);
	}

	PushMaskChannelOverride();
}
 
void UDMMaterialStageThroughputLayerBlend::AddDefaultInput(int32 InInputIndex) const
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);
 
	switch (InInputIndex)
	{
		case InputPreviousLayer:
		case InputBaseStage:
			// Resolved internally
			break;
 
		case InputMaskSource:
		{
			UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
				Stage,
				InInputIndex,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
				EDMValueType::VT_Float3_RGB,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);
			check(InputValue);

			UDMMaterialValueFloat3RGB* InputColor = Cast<UDMMaterialValueFloat3RGB>(InputValue->GetValue());
			check(InputColor);

			InputColor->SetDefaultValue(FLinearColor::White);
			InputColor->ApplyDefaultValue();
			break;
		}
 
		default:
			checkNoEntry();
			break;
	}
}
 
bool UDMMaterialStageThroughputLayerBlend::CanChangeInput(int32 InputIndex) const
{
	return (InputIndex == InputMaskSource);
}
 
bool UDMMaterialStageThroughputLayerBlend::IsInputVisible(int32 InputIndex) const
{
	if (!Super::IsInputVisible(InputIndex))
	{
		return false;
	}
 
	return (InputIndex == InputMaskSource);
}
 
void UDMMaterialStageThroughputLayerBlend::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (bBlockUpdate)
	{
		return;
	}

	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}
 
	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		UDMMaterialStage* Stage = GetStage();
		check(Stage);
 
		InputConnectors[1].Type = EDMValueType::VT_Float_Any;
		OutputConnectors[0].Type = EDMValueType::VT_Float_Any;
 
		if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
		{
			if (const UDMMaterialStageSource* LayerSource = Layer->GetStage(EDMMaterialLayerStage::Base)->GetSource())
			{
				const TArray<FDMMaterialStageConnector>& LayerSourceOutputConnectors = LayerSource->GetOutputConnectors();
				check(LayerSourceOutputConnectors.IsValidIndex(0));

				InputConnectors[InputBaseStage].Type = LayerSourceOutputConnectors[0].Type;
				OutputConnectors[0].Type = LayerSourceOutputConnectors[0].Type;
			}
		}
	}

	UpdateLinkedInputStage(InUpdateType);
 
	UpdateAlphaOnlyMasks(InUpdateType);

	Super::Update(InSource, InUpdateType);

	PullMaskChannelOverride();
}
 
int32 UDMMaterialStageThroughputLayerBlend::ResolveInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InputIndex, FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const
{
	switch (InputIndex)
	{
		case InputPreviousLayer:
		{
			const UDMMaterialStage* Stage = GetStage();
			check(Stage);

			UDMMaterialLayerObject* Layer = Stage->GetLayer();
			check(Layer);

			EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
			check(StageProperty != EDMMaterialPropertyType::None);

			if (UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(StageProperty, EDMMaterialLayerStage::Base))
			{
				PreviousLayer->GenerateExpressions(InBuildState);

				const TArray<UMaterialExpression*>& PreviousLayerExpressions = InBuildState->GetLayerExpressions(PreviousLayer);
				check(!PreviousLayerExpressions.IsEmpty());

				OutExpressions.Add(PreviousLayerExpressions.Last());
				return 0;
			}

			UMaterialExpressionConstant3Vector* Constant3Vector = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionConstant3Vector>(UE_DM_NodeComment_Default);
			check(Constant3Vector);

			Constant3Vector->Constant = FLinearColor::Black;

			OutExpressions.Add(Constant3Vector);

			return 0;
		}
 
		case InputBaseStage:
		{
			const UDMMaterialStage* Stage = GetStage();
			check(Stage);
 
			UDMMaterialLayerObject* Layer = Stage->GetLayer();
			check(Layer);

			UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true);
			check(BaseStage);
 
			BaseStage->GenerateExpressions(InBuildState);

			const TArray<UMaterialExpression*>& BaseStageExpressions = InBuildState->GetStageExpressions(BaseStage);
			OutExpressions.Add(BaseStageExpressions.Last());
 
			return OutChannel.OutputIndex;
		}
 
		case InputMaskSource:
			return ResolveMaskInput(InBuildState, InputIndex, OutChannel, OutExpressions);			
 
		default:
			checkNoEntry();
			return OutChannel.OutputIndex;
	}
}
 
void UDMMaterialStageThroughputLayerBlend::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	UObject* PreviewObject = InBuildState->GetPreviewObject();

	if (PreviewObject == this || PreviewObject == GetStage())
	{
		GeneratePreviewExpressions(InBuildState);
	}
	else
	{
		GenerateMainExpressions(InBuildState);
	}
}

void UDMMaterialStageThroughputLayerBlend::PostLoad()
{
	Super::PostLoad();

	InitBlendStage();

	PullMaskChannelOverride();
}

void UDMMaterialStageThroughputLayerBlend::PostEditImport()
{
	Super::PostEditImport();

	InitBlendStage();
}

void UDMMaterialStageThroughputLayerBlend::PostEditUndo()
{
	Super::PostEditUndo();

	PullMaskChannelOverride();
}

void UDMMaterialStageThroughputLayerBlend::InitBlendStage()
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	Stage->GetOnUpdate().AddUObject(this, &UDMMaterialStageThroughputLayerBlend::OnStageUpdated);
}

void UDMMaterialStageThroughputLayerBlend::GenerateMainExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	check(GetAlphaBlend());

	UMaterialExpressionMaterialFunctionCall* FunctionCall = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMaterialFunctionCall>(UE_DM_NodeComment_Default);
	FunctionCall->SetMaterialFunction(GetAlphaBlend());
	FunctionCall->UpdateFromFunctionResource();

	TArray<UMaterialExpression*> Expressions = {FunctionCall};

	bool bIsOpacitySlot = false;
	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = nullptr;
	UDMMaterialSlot* RGBSlot = nullptr;
	EDMMaterialPropertyType RGBProperty = EDMMaterialPropertyType::None;

	if (UDMMaterialStage* Stage = GetStage())
	{
		if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
		{
			switch (Layer->GetMaterialProperty())
			{
				case EDMMaterialPropertyType::Opacity:
				case EDMMaterialPropertyType::OpacityMask:
					bIsOpacitySlot = true;
					break;

				default:
					bIsOpacitySlot = false;
					break;
			}

			if (bIsOpacitySlot)
			{
				if (UDMMaterialSlot* Slot = Layer->GetSlot())
				{
					ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

					if (ModelEditorOnlyData)
					{
						if (UDMMaterialSlot* BaseColorSlot = ModelEditorOnlyData->GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::BaseColor))
						{
							RGBProperty = EDMMaterialPropertyType::BaseColor;
							RGBSlot = BaseColorSlot;
						}
						else if (UDMMaterialSlot* EmissiveColorSlot = ModelEditorOnlyData->GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::EmissiveColor))
						{
							RGBProperty = EDMMaterialPropertyType::EmissiveColor;
							RGBSlot = EmissiveColorSlot;
						}
					}
				}
			}
		}
	}

	if (bIsOpacitySlot && ModelEditorOnlyData && RGBSlot && RGBProperty != EDMMaterialPropertyType::None)
	{
		UMaterialExpression* LastExpression = nullptr;
		int32 OutputIndex;
		int32 OutputChannel;
		UDMMaterialProperty::GenerateOpacityExpressions(InBuildState, RGBSlot, RGBProperty, LastExpression, OutputIndex, OutputChannel);

		if (LastExpression)
		{
			Expressions.Add(LastExpression);

			UMaterialExpressionMultiply* Multiply = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMultiply>(UE_DM_NodeComment_Default);
			Multiply->A.Expression = LastExpression;
			Multiply->A.OutputIndex = OutputIndex;
			SetMask(Multiply->A, LastExpression->GetOutputs()[OutputIndex], OutputChannel);

			Multiply->B.Expression = FunctionCall;
			Multiply->B.OutputIndex = 0;
			SetMask(Multiply->B, FunctionCall->GetOutputs()[0], FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);

			Expressions.Add(Multiply);
		}
	}

	InBuildState->AddStageSourceExpressions(this, Expressions);
}

void UDMMaterialStageThroughputLayerBlend::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	InitBlendStage();
}

bool UDMMaterialStageThroughputLayerBlend::IsPropertyVisible(FName InProperty) const
{
	static const FName MaskChannelName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageThroughputLayerBlend, MaskChannelOverride);

	if (InProperty == MaskChannelName)
	{
		return CanUseMaskChannelOverride();
	}
		
	return Super::IsPropertyVisible(InProperty);
}

void UDMMaterialStageThroughputLayerBlend::GeneratePreviewExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();
	const TArray<FDMMaterialStageConnection>& StageConnections = Stage->GetInputConnectionMap();

	if (StageConnections.IsValidIndex(InputMaskSource) && StageConnections[InputMaskSource].Channels.IsEmpty() == false)
	{
		int32 StageInputIdx = StageConnections[InputMaskSource].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

		if (StageInputs.IsValidIndex(StageInputIdx))
		{
			StageInputs[StageInputIdx]->GenerateExpressions(InBuildState);
			TArray<UMaterialExpression*> MaskInputExpressions = InBuildState->GetStageSourceExpressions(StageInputs[StageInputIdx]);
			check(!MaskInputExpressions.IsEmpty());

			if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
			{
				int32 Channel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
				int32 OutputIndex = 0;
				Layer->ApplyEffects(InBuildState, Stage, MaskInputExpressions, Channel, OutputIndex);
			}

			InBuildState->AddStageSourceExpressions(this, MaskInputExpressions);
		}
	}
}

void UDMMaterialStageThroughputLayerBlend::UpdateLinkedInputStage(EDMUpdateType InUpdateType)
{
	UDMMaterialStage* MaskStage = GetTypedParent<UDMMaterialStage>(/* bInAllowSubclasses */ true);

	if (!MaskStage)
	{
		return;
	}

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(MaskStage))
	{
		MaskStage = SubStage->GetParentMostStage();
	}

	UDMMaterialLayerObject* Layer = MaskStage->GetLayer();

	if (!Layer)
	{
		return;
	}

	if (Layer->GetStageType(MaskStage) != EDMMaterialLayerStage::Mask)
	{
		return;
	}

	UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base);

	if (!BaseStage)
	{
		return;
	}

	bool bInputInCommon = false;

	for (UDMMaterialStageInput* MaskInput : MaskStage->GetInputs())
	{
		for (UDMMaterialStageInput* BaseInput : BaseStage->GetInputs())
		{
			if (MaskInput == BaseInput)
			{
				bInputInCommon = true;
				break;
			}

			if (UDMMaterialStageInputValue* MaskInputValue = Cast<UDMMaterialStageInputValue>(MaskInput))
			{
				if (UDMMaterialStageInputValue* BaseInputValue = Cast<UDMMaterialStageInputValue>(BaseInput))
				{
					if (MaskInputValue->GetValue() && MaskInputValue->GetValue() == BaseInputValue->GetValue())
					{
						bInputInCommon = true;
						break;
					}
				}
			}
		}

		if (bInputInCommon)
		{
			break;
		}
	}

	if (bInputInCommon)
	{
		bBlockUpdate = true;
		BaseStage->Update(this, InUpdateType);
		bBlockUpdate = false;
	}
}

FText UDMMaterialStageThroughputLayerBlend::GetStageDescription() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

		if (InputMap.IsValidIndex(UDMMaterialStageThroughputLayerBlend::InputMaskSource)
			&& InputMap[UDMMaterialStageThroughputLayerBlend::InputMaskSource].Channels.IsValidIndex(0))
		{
			TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
			const int32 StageInputIdx = InputMap[UDMMaterialStageThroughputLayerBlend::InputMaskSource].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			if (Inputs.IsValidIndex(StageInputIdx))
			{
				return Inputs[StageInputIdx]->GetComponentDescription();
			}
		}
	}

	return Super::GetComponentDescription();
}

bool UDMMaterialStageThroughputLayerBlend::GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
	UMaterialExpression*& OutMaterialExpression, int32& OutputIndex)
{
	check(InStage);
	check(InPreviewMaterial);

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialStage* StageActual = InStage;
	UDMMaterialStageSource* PreviewSource = nullptr;

	if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(GetInputMask()))
	{
		if (UDMMaterialStageExpressionTextureSample* TextureSample = Cast<UDMMaterialStageExpressionTextureSample>(InputExpression->GetMaterialStageExpression()))
		{
			if (TextureSample->GetUseBaseTexture())
			{
				if (const UDMMaterialStageExpressionTextureSample* BaseTextureSample = TextureSample->GetBaseTextureSample())
				{
					UDMMaterialStage* BaseStage = BaseTextureSample->GetStage();

					if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(BaseStage))
					{
						BaseStage = SubStage->GetParentMostStage();
					}

					if (BaseStage)
					{
						if (UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(BaseStage->GetSource()))
						{
							if (UDMMaterialStageInput* BaseInputExpression = Blend->GetInputB())
							{
								StageActual = BaseStage;
								PreviewSource = BaseInputExpression;
							}
						}
					}
				}
			}
		}
	}

	UDMMaterialLayerObject* LayerActual = StageActual->GetLayer();
	check(LayerActual);

	UDMMaterialSlot* Slot = LayerActual->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	const TArray<FDMMaterialStageConnection>& InputConnectionMap = InStage->GetInputConnectionMap();

	if (!InputConnectionMap.IsValidIndex(InputMaskSource)
		|| InputConnectionMap[InputMaskSource].Channels.Num() != 1)
	{
		return false;
	}

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(StageActual);

	if (!PreviewSource)
	{
		PreviewSource = GetInputMask();
	}

	PreviewSource->GenerateExpressions(BuildState);
	TArray<UMaterialExpression*> SourceExpressions = BuildState->GetStageSourceExpressions(PreviewSource);

	if (SourceExpressions.IsEmpty())
	{
		return false;
	}

	UMaterialExpression* ComponentMask = SourceExpressions.Last();
	OutputIndex = InputConnectionMap[InputMaskSource].Channels[0].OutputIndex;
	int32 Channel;
	bool bGenerateBitmask = true;

	{
		const UDMMaterialStageSource* Source = nullptr;

		for (const TPair<const UDMMaterialStageSource*, TArray<UMaterialExpression*>>& Pair : BuildState->GetStageSourceMap())
		{
			if (Pair.Value.IsEmpty())
			{
				continue;
			}

			if (Pair.Value.Last() == ComponentMask)
			{
				Source = Pair.Key;
				break;
			}
		}

		if (Source && Source->GetOutputConnectors().IsValidIndex(OutputIndex))
		{
			OutputIndex = Source->GetOutputConnectors()[OutputIndex].Index;

			const int32 InnateOutputIndex = Source->GetInnateMaskOutput(OutputIndex, InputConnectionMap[InputMaskSource].Channels[0].OutputChannel);

			if (InnateOutputIndex != INDEX_NONE)
			{
				OutputIndex = InnateOutputIndex;
				bGenerateBitmask = false;
			}
		}
	}

	if (bGenerateBitmask)
	{
		// Pick the first selected channel - this is always a single channel.
		if (InputConnectionMap[InputMaskSource].Channels[0].OutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL)
		{
			Channel = FDMMaterialStageConnectorChannel::FIRST_CHANNEL;
		}
		else if (InputConnectionMap[InputMaskSource].Channels[0].OutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL)
		{
			Channel = FDMMaterialStageConnectorChannel::SECOND_CHANNEL;
		}
		else if (InputConnectionMap[InputMaskSource].Channels[0].OutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL)
		{
			Channel = FDMMaterialStageConnectorChannel::THIRD_CHANNEL;
		}
		else if (InputConnectionMap[InputMaskSource].Channels[0].OutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL)
		{
			Channel = FDMMaterialStageConnectorChannel::FOURTH_CHANNEL;
		}
		else
		{
			Channel = FDMMaterialStageConnectorChannel::FIRST_CHANNEL;
		}

		const int32 MaskedOutput = BuildState->GetBuildUtils().FindOutputForBitmask(
			ComponentMask,
			Channel
		);

		if (MaskedOutput == INDEX_NONE)
		{
			ComponentMask = BuildState->GetBuildUtils().CreateExpressionBitMask(
				ComponentMask,
				OutputIndex,
				Channel
			);

			SourceExpressions.Add(ComponentMask);

			OutputIndex = 0;
			Channel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
		}
		else
		{
			OutputIndex = MaskedOutput;
		}
	}

	Layer->ApplyEffects(BuildState, InStage, SourceExpressions, Channel, OutputIndex);

	OutMaterialExpression = SourceExpressions.Last();

	return true;
}

void UDMMaterialStageThroughputLayerBlend::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	if (!IsComponentValid())
	{
		return;
	}

	static const FName MaskChannelName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageThroughputLayerBlend, MaskChannelOverride);

	if (InPropertyChangedEvent.GetPropertyName() == MaskChannelName)
	{
		PushMaskChannelOverride();
	}
	else
	{
		PullMaskChannelOverride();
	}

	Super::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
}

void UDMMaterialStageThroughputLayerBlend::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName MaskChannelOverrideName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageThroughputLayerBlend, MaskChannelOverride);

	if (InPropertyChangedEvent.GetPropertyName() == MaskChannelOverrideName)
	{
		PushMaskChannelOverride();
	}
}

void UDMMaterialStageThroughputLayerBlend::ConnectOutputToInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InInputIdx,
	int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel)
{
	check(InSourceExpression);
	check(InSourceExpression->GetOutputs().IsValidIndex(InSourceOutputIndex));
	check(InExpressionInputIndex >= InputPreviousLayer && InExpressionInputIndex <= InputMaskSource);
 
	const TArray<UMaterialExpression*>& StageSourceExpressions = InBuildState->GetStageSourceExpressions(this);
	check(!StageSourceExpressions.IsEmpty());

	using namespace UE::DynamicMaterialEditor::Private;

	check(GetAlphaBlend());

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	GetAlphaBlend()->GetInputsAndOutputs(Inputs, Outputs);

	int32 InputPreviousIndex = INDEX_NONE;
	int32 InputLayerIndex = INDEX_NONE;
	int32 InputMaskIndex = INDEX_NONE;

	// Could go by name here, but this is more bug prone.
	for (int32 InputIdx = 0; InputIdx < Inputs.Num(); ++InputIdx)
	{
		if (Inputs[InputIdx].ExpressionInput->InputType == FunctionInput_Vector3)
		{
			if (InputPreviousIndex == INDEX_NONE)
			{
				InputPreviousIndex = InputIdx;
			}
			else if (InputLayerIndex == INDEX_NONE)
			{
				InputLayerIndex = InputIdx;
			}
		}
		else if (Inputs[InputIdx].ExpressionInput->InputType == FunctionInput_Scalar)
		{
			if (InputMaskIndex == INDEX_NONE)
			{
				InputMaskIndex = InputIdx;
			}
		}
	}
 
	switch (InExpressionInputIndex)
	{
		case InputPreviousLayer:
			ConnectOutputToInput_Internal(
				InBuildState, 
				StageSourceExpressions[0] /* FunctionCall */, 
				InputPreviousIndex, 
				InSourceExpression, 
				InSourceOutputIndex, 
				InSourceOutputChannel
			);
			break;
 
		case InputBaseStage:
			ConnectOutputToInput_Internal(
				InBuildState, 
				StageSourceExpressions[0] /* FunctionCall */, 
				InputLayerIndex, 
				InSourceExpression, 
				InSourceOutputIndex, 
				InSourceOutputChannel
			);
			break;
 
		case InputMaskSource:
			ConnectOutputToInput_Internal(
				InBuildState, 
				StageSourceExpressions[0] /* FunctionCall */, 
				InputMaskIndex, 
				InSourceExpression, 
				InSourceOutputIndex, 
				InSourceOutputChannel
			);
			break;
 
		default:
			checkNoEntry();
			break;
	}
}

void UDMMaterialStageThroughputLayerBlend::OnPostInputAdded(int32 InInputIdx)
{
	Super::OnPostInputAdded(InInputIdx);

	if (MaskChannelOverride != EAvaColorChannel::None)
	{
		PushMaskChannelOverride();
	}
}

void UDMMaterialStageThroughputLayerBlend::GetMaskOutput(const TSharedRef<FDMMaterialBuildState>& InBuildState, UMaterialExpression*& OutExpression,
	int32& OutOutputIndex, int32& OutOutputChannel) const
{
	FDMMaterialStageConnectorChannel Channel;
	TArray<UMaterialExpression*> Expressions;
 
	OutOutputIndex = ResolveInput(InBuildState, 2, Channel, Expressions);
	OutOutputChannel = Channel.OutputChannel;
	OutExpression = Expressions.Last();
}
 
void UDMMaterialStageThroughputLayerBlend::SetPremultiplyAlpha(bool bInValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (bPremultiplyAlpha == bInValue)
	{
		return;
	}
 
	bPremultiplyAlpha = bInValue;
 
	Update(this, EDMUpdateType::Structure);
}
 
int32 UDMMaterialStageThroughputLayerBlend::ResolveMaskInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InputIndex, 
	FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const
{
	bool bIsSharedBaseTextureSample = false;

	if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(GetInputMask()))
	{
		if (UDMMaterialStageExpressionTextureSample* TextureSample = Cast<UDMMaterialStageExpressionTextureSample>(InputExpression->GetMaterialStageExpression()))
		{
			if (TextureSample->GetUseBaseTexture())
			{
				if (const UDMMaterialStageExpressionTextureSample* BaseTextureSample = TextureSample->GetBaseTextureSample())
				{
					BaseTextureSample->GenerateExpressions(InBuildState);
					OutExpressions = InBuildState->GetStageSourceExpressions(BaseTextureSample);

					switch (MaskChannelOverride)
					{
						default:
						case EAvaColorChannel::Red:
							OutChannel.OutputIndex = 1;
							break;

						case EAvaColorChannel::Green:
							OutChannel.OutputIndex = 2;
							break;

						case EAvaColorChannel::Blue:
							OutChannel.OutputIndex = 3;
							break;

						case EAvaColorChannel::Alpha:
							OutChannel.OutputIndex = 4;
							break;
					}

					bIsSharedBaseTextureSample = true;
				}
			}
		}
	}

	if (!bIsSharedBaseTextureSample)
	{
		// Our baseline input. Could be a texture node or color value. Anything inputy.
		OutChannel.OutputIndex = Super::ResolveInput(InBuildState, InputIndex, OutChannel, OutExpressions);
	}

	UDMMaterialStage* ThisStage = GetStage();
	check(ThisStage);
 
	UDMMaterialLayerObject* ThisLayer = ThisStage->GetLayer();
	check(ThisLayer);
	check(ThisLayer->GetStage(EDMMaterialLayerStage::Mask));

	ThisLayer->ApplyEffects(
		InBuildState, 
		ThisStage, 
		OutExpressions, 
		OutChannel.OutputChannel, 
		OutChannel.OutputIndex
	);
		
	while (UDMMaterialLayerObject* NextLayer = ThisLayer->GetNextLayer(ThisLayer->GetMaterialProperty(), EDMMaterialLayerStage::Mask))
	{
		UDMMaterialStage* NextMaskStage = NextLayer->GetStage(EDMMaterialLayerStage::Mask, /* Enabled Only */ true);

		// If the next layer has a base enabled, we stop scanning.
		// If the next layer's mask is disabled, we stop scanning.
		if (!NextMaskStage || NextLayer->GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true))
		{
			break;
		}
 
		// The next stage is one which has its base disabled and its mask enabled, so we multiply (not max) the outputs.
		UDMMaterialStageThroughputLayerBlend* NextMaskBlend = Cast<UDMMaterialStageThroughputLayerBlend>(NextMaskStage->GetSource());
 
		if (!NextMaskBlend)
		{
			break;
		}
		
		// Resolve the input of the next blend layer mask channel
		FDMMaterialStageConnectorChannel NextOutChannel;
		TArray<UMaterialExpression*> NextOutExpressions;

		NextMaskBlend->ResolveInput(
			InBuildState, 
			InputIndex, 
			NextOutChannel, 
			NextOutExpressions
		);
 
		if (NextOutExpressions.IsEmpty())
		{
			break;
		}

		NextLayer->ApplyEffects(
			InBuildState, 
			NextMaskStage, 
			NextOutExpressions, 
			NextOutChannel.OutputChannel, 
			NextOutChannel.OutputIndex
		);

		UMaterialExpressionMultiply* AlphaMultiply = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMultiply>(UE_DM_NodeComment_Default);

		ConnectOutputToInput_Internal(
			InBuildState, 
			AlphaMultiply, 
			0, 
			OutExpressions.Last(), 
			OutChannel.OutputIndex, 
			OutChannel.OutputChannel
		);

		ConnectOutputToInput_Internal(
			InBuildState, 
			AlphaMultiply, 
			1, 
			NextOutExpressions.Last(), 
			NextOutChannel.OutputIndex, 
			NextOutChannel.OutputChannel
		);
 
		OutExpressions.Append(NextOutExpressions);
		OutExpressions.Add(AlphaMultiply);
 
		OutChannel.OutputIndex = 0;
		OutChannel.OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

		ThisLayer = NextLayer;
	}

	return OutChannel.OutputIndex;
}
 
void UDMMaterialStageThroughputLayerBlend::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	if (!IsComponentValid())
	{
		return;
	}
 
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);
 
	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	const TArray<FDMMaterialStageConnection>& InputConnectionMap = Stage->GetInputConnectionMap();
	const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();
 
	TArray<UDMMaterialStageInput*> ChannelInputs;
 
	if (!InputConnectionMap.IsValidIndex(2))
	{
		return;
	}
 
	ChannelInputs.SetNum(InputConnectionMap[2].Channels.Num());
 
	for (int32 ChannelIdx = 0; ChannelIdx < InputConnectionMap[2].Channels.Num(); ++ChannelIdx)
	{
		if (InputConnectionMap[2].Channels[ChannelIdx].SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
		{
			ChannelInputs[ChannelIdx] = nullptr;
			continue;
		}
 
		if (InputConnectionMap[2].Channels[ChannelIdx].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
		{
			int32 StageInputIdx = InputConnectionMap[2].Channels[ChannelIdx].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
			ChannelInputs[ChannelIdx] = StageInputs[StageInputIdx];
			continue;
		}
	}
 
	if (ChannelInputs.IsEmpty())
	{
		return;
	}
 
	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);

	if (ChannelInputs.IsEmpty())
	{
		Stage->GenerateExpressions(BuildState);
		UMaterialExpression* StageExpression = BuildState->GetLastStageExpression(Stage);
 
		BuildState->GetBuildUtils().UpdatePreviewMaterial(StageExpression, 0, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 32);
	}
	else
	{
		UE::DynamicMaterialEditor::Private::BuildExpressionInputs(BuildState, InputConnectionMap, {{2, ChannelInputs}});
	}
}
 
void UDMMaterialStageThroughputLayerBlend::UpdateAlphaOnlyMaskStatus()
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();
	
	if (!Stage)
	{
		return;
	}
 
	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	
	if (!Layer)
	{
		return;
	}
 
	bIsAlphaOnlyBlend = (Layer->IsStageEnabled(EDMMaterialLayerStage::Base) == false && Layer->IsStageEnabled(EDMMaterialLayerStage::Mask) == true);
}
 
void UDMMaterialStageThroughputLayerBlend::OnStageUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (bBlockUpdate)
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		const bool bIsAlphaOnlyBlendNow = bIsAlphaOnlyBlend;
		UpdateAlphaOnlyMaskStatus();
 
		if (bIsAlphaOnlyBlendNow || bIsAlphaOnlyBlend)
		{
			UpdateAlphaOnlyMasks(InUpdateType);
		}
	}
}
 
void UDMMaterialStageThroughputLayerBlend::UpdateAlphaOnlyMasks(EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();
	
	if (!Stage)
	{
		return;
	}
 
	UDMMaterialLayerObject* CurrentLayer = Stage->GetLayer();
	
	if (!CurrentLayer)
	{
		return;
	}

	UDMMaterialStage* MaskStage = CurrentLayer->GetStage(EDMMaterialLayerStage::Mask, /* Enabled Only */ true);
 
	if (CurrentLayer->GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true) || !MaskStage)
	{
		return;
	}
 
	UDMMaterialLayerObject* PreviousLayer = nullptr;
 
	while (true)
	{
		PreviousLayer = CurrentLayer->GetPreviousLayer(CurrentLayer->GetMaterialProperty(), EDMMaterialLayerStage::None);
 
		if (!PreviousLayer)
		{
			break;
		}

		MaskStage = PreviousLayer->GetStage(EDMMaterialLayerStage::Mask, /* Enabled Only */ true);

		// We're looking for a chain of alpha stages. If we find no alpha stage and an invalid
		// base stage, we're got an error.
		if (!PreviousLayer->GetStage(EDMMaterialLayerStage::Base) || !MaskStage)
		{
			break;
		}
 
		CurrentLayer = PreviousLayer;
 
		if (PreviousLayer->IsStageEnabled(EDMMaterialLayerStage::Base))
		{
			break;
		}
	}
 
	if (PreviousLayer)
	{
		// Calling this will inevitably recall this method when the chain updates.
		// Block this update.
		bBlockUpdate = true;
		PreviousLayer->GetStage(EDMMaterialLayerStage::Mask)->Update(this, InUpdateType);
		bBlockUpdate = false;
	}
}

UDMMaterialStageInput* UDMMaterialStageThroughputLayerBlend::GetInputMask() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

		if (InputMap.IsValidIndex(InputMaskSource)
			&& InputMap[InputMaskSource].Channels.IsValidIndex(0))
		{
			TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
			int32 StageInputIdx = InputMap[InputMaskSource].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

			if (Stage->GetInputs().IsValidIndex(StageInputIdx))
			{
				return Stage->GetInputs()[StageInputIdx];
			}
		}
	}

	return nullptr;
}

EAvaColorChannel UDMMaterialStageThroughputLayerBlend::GetMaskChannelOverride() const
{
	if (CanUseMaskChannelOverride())
	{
		PullMaskChannelOverride();
		return MaskChannelOverride;
	}

	return EAvaColorChannel::None;
}

void UDMMaterialStageThroughputLayerBlend::SetMaskChannelOverride(EAvaColorChannel InMaskChannel)
{
	if (!CanUseMaskChannelOverride())
	{
		return;
	}

	if (GetMaskChannelOverride() == InMaskChannel)
	{
		return;
	}

	MaskChannelOverride = InMaskChannel;
	PushMaskChannelOverride();

	Update(this, EDMUpdateType::Structure);
}

FSlateIcon UDMMaterialStageThroughputLayerBlend::GetComponentIcon() const
{
	if (UDMMaterialStageInput* InputMaskValue = GetInputMask())
	{
		return InputMaskValue->GetComponentIcon();
	}

	return Super::GetComponentIcon();
}

bool UDMMaterialStageThroughputLayerBlend::CanUseMaskChannelOverride() const
{
	return GetDefaultMaskChannelOverrideOutputIndex() != INDEX_NONE;
}

int32 UDMMaterialStageThroughputLayerBlend::GetDefaultMaskChannelOverrideOutputIndex() const
{
	UDMMaterialStageInput* MaskInput = GetInputMask();

	if (!MaskInput)
	{
		return INDEX_NONE;
	}

	const TArray<FDMMaterialStageConnector>& MaskInputOutputConnectors = MaskInput->GetOutputConnectors();

	for (int32 Index = 0; Index < MaskInputOutputConnectors.Num(); ++Index)
	{
		if (UDMValueDefinitionLibrary::GetValueDefinition(MaskInputOutputConnectors[Index].Type).GetFloatCount() > 1)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

bool UDMMaterialStageThroughputLayerBlend::IsValidMaskChannelOverrideOutputIndex(int32 InIndex) const
{
	UDMMaterialStageInput* MaskInput = GetInputMask();

	if (!MaskInput)
	{
		return false;
	}

	const TArray<FDMMaterialStageConnector>& MaskInputOutputConnectors = MaskInput->GetOutputConnectors();

	if (!MaskInputOutputConnectors.IsValidIndex(InIndex))
	{
		return false;
	}

	return UDMValueDefinitionLibrary::GetValueDefinition(MaskInputOutputConnectors[InIndex].Type).GetFloatCount() > 1;
}

void UDMMaterialStageThroughputLayerBlend::PullMaskChannelOverride() const
{
	MaskChannelOverride = EAvaColorChannel::None;

	if (!CanUseMaskChannelOverride())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

	if (!InputMap.IsValidIndex(UDMMaterialStageThroughputLayerBlend::InputMaskSource)
		|| !InputMap[UDMMaterialStageThroughputLayerBlend::InputMaskSource].Channels.IsValidIndex(0))
	{
		return;
	}

	TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
	const FDMMaterialStageConnectorChannel& MaskConnectorChannel = InputMap[UDMMaterialStageThroughputLayerBlend::InputMaskSource].Channels[0];

	switch (MaskConnectorChannel.OutputChannel)
	{
		case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
			MaskChannelOverride = EAvaColorChannel::Red;
			break;

		case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
			MaskChannelOverride = EAvaColorChannel::Green;
			break;

		case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
			MaskChannelOverride = EAvaColorChannel::Blue;
			break;

		case FDMMaterialStageConnectorChannel::FOURTH_CHANNEL:
			MaskChannelOverride = EAvaColorChannel::Alpha;
			break;

		default:
			// Do nothing
			break;
	}
}

void UDMMaterialStageThroughputLayerBlend::PushMaskChannelOverride()
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	if (!CanUseMaskChannelOverride())
	{
		return;
	}

	const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();

	if (!InputMap.IsValidIndex(UDMMaterialStageThroughputLayerBlend::InputMaskSource)
		|| !InputMap[UDMMaterialStageThroughputLayerBlend::InputMaskSource].Channels.IsValidIndex(0))
	{
		return;
	}

	TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();
	const FDMMaterialStageConnectorChannel& MaskConnectorChannel = InputMap[UDMMaterialStageThroughputLayerBlend::InputMaskSource].Channels[0];
	const int32 MaskInputIdx = MaskConnectorChannel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

	if (!Inputs.IsValidIndex(MaskInputIdx))
	{
		return;
	}

	const int32 OutputIndex = IsValidMaskChannelOverrideOutputIndex(MaskConnectorChannel.OutputIndex)
		? MaskConnectorChannel.OutputIndex
		: GetDefaultMaskChannelOverrideOutputIndex();

	int32 OutputChannel;

	switch (MaskChannelOverride)
	{
		case EAvaColorChannel::Red:
			OutputChannel = FDMMaterialStageConnectorChannel::FIRST_CHANNEL;
			break;

		case EAvaColorChannel::Green:
			OutputChannel = FDMMaterialStageConnectorChannel::SECOND_CHANNEL;
			break;

		case EAvaColorChannel::Blue:
			OutputChannel = FDMMaterialStageConnectorChannel::THIRD_CHANNEL;
			break;

		case EAvaColorChannel::Alpha:
			OutputChannel = FDMMaterialStageConnectorChannel::FOURTH_CHANNEL;
			break;

		default:
			OutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
			break;
	}

	Stage->UpdateInputMap(UDMMaterialStageThroughputLayerBlend::InputMaskSource, MaskConnectorChannel.SourceIndex,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, OutputIndex, OutputChannel, 
		MaskConnectorChannel.MaterialProperty);
}

#undef LOCTEXT_NAMESPACE
