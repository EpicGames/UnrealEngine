// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMMaterialSubStage.h"

#include "Components/DMMaterialLayer.h"
#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialSubStage)

UDMMaterialSubStage* UDMMaterialSubStage::CreateMaterialSubStage(UDMMaterialStage* InParentStage)
{
	UDMMaterialLayerObject* Layer = InParentStage->GetLayer();
	check(Layer);

	UDMMaterialSubStage* SubStage = NewObject<UDMMaterialSubStage>(Layer, NAME_None, RF_Transactional);
	check(SubStage);

	SubStage->ParentStage = InParentStage;

	return SubStage;
}

UDMMaterialStage* UDMMaterialSubStage::GetParentStage() const
{
	return ParentStage;
}

UDMMaterialStage* UDMMaterialSubStage::GetParentMostStage() const
{
	if (!ParentStage)
	{
		return nullptr;
	}

	if (UDMMaterialSubStage* ParentSubStage = Cast<UDMMaterialSubStage>(ParentStage))
	{
		return ParentSubStage->GetParentMostStage();
	}

	return ParentStage;
}

UDMMaterialComponent* UDMMaterialSubStage::GetParentComponent() const
{
	return ParentComponent.Get();
}

void UDMMaterialSubStage::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	if (UDMMaterialStageInputThroughput* InputThroughput = Cast<UDMMaterialStageInputThroughput>(InParent))
	{
		ParentStage = InputThroughput->GetStage();
		ParentComponent = InputThroughput;
		Super::PostEditorDuplicate(InMaterialModel, ParentStage->GetLayer());
		ParentComponent = InputThroughput; // This will be reset to the slot by the parent call.
	}
	else
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Wrong parent component passed to substage."), true, this);
		ParentStage = nullptr;
		ParentComponent = nullptr;
		Super::PostEditorDuplicate(InMaterialModel, InParent);
		ParentComponent = nullptr;
	}
}

void UDMMaterialSubStage::SetParentComponent(UDMMaterialComponent* InParentComponent)
{
	ParentComponent = InParentComponent;
}

bool UDMMaterialSubStage::IsCompatibleWithPreviousStage(const UDMMaterialStage* PreviousStage) const
{
	return false;
}

bool UDMMaterialSubStage::IsCompatibleWithNextStage(const UDMMaterialStage* NextStage) const
{
	return false;
}

void UDMMaterialSubStage::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (!IsComponentValid())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		MarkComponentDirty();
		VerifyAllInputMaps();
	}

	// Skip UDMMaterialStage* update because we don't want to update other stages or layers.
	Super::Super::Update(InSource, InUpdateType);

	if (IsValid(ParentComponent))
	{
		ParentComponent->Update(InSource, InUpdateType);
	}
	else if (IsValid(ParentStage))
	{
		ParentStage->Update(InSource, InUpdateType);
	}
}

FString UDMMaterialSubStage::GetComponentPathComponent() const
{
	// Skip stage renaming
	return Super::Super::GetComponentPathComponent();
}
