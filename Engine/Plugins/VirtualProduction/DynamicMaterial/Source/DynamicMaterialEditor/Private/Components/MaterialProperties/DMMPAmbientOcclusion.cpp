// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPAmbientOcclusion.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPAmbientOcclusion)

UDMMaterialPropertyAmbientOcclusion::UDMMaterialPropertyAmbientOcclusion()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::AmbientOcclusion),
		EDMValueType::VT_Float1)
{
}

UMaterialExpression* UDMMaterialPropertyAmbientOcclusion::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::ZeroVector);
}
