// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialProperty.h"
#include "DMMPAnisotropy.generated.h"

class UDynamicMaterialModelEditorOnlyData;

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialPropertyAnisotropy : public UDMMaterialProperty
{
	GENERATED_BODY()

public:
	UDMMaterialPropertyAnisotropy();

	//~ Begin UDMMaterialProperty
	virtual UMaterialExpression* GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialProperty
};
