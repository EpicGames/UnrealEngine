// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPBaseColor.h"
#include "Components/DMMaterialSlot.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPBaseColor)

UDMMaterialPropertyBaseColor::UDMMaterialPropertyBaseColor()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::BaseColor),
		EDMValueType::VT_Float3_RGB)
{
}

UMaterialExpression* UDMMaterialPropertyBaseColor::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::OneVector);
}

TEnumAsByte<EMaterialSamplerType> UDMMaterialPropertyBaseColor::GetTextureSamplerType() const
{
	return EMaterialSamplerType::SAMPLERTYPE_Color;
}
