// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPDisplacement.h"

#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPDisplacement)

namespace UE::DynamicMaterialEditor::Private
{
	UMaterialFunctionInterface* GetDisplacementAlpha()
	{
		static UMaterialFunctionInterface* DisplacementAlpha = FDMMaterialFunctionLibrary::Get().GetFunction(
			"DisplacementAlpha",
			TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_DisplacementAlpha.MF_DM_DisplacementAlpha'")
		);

		return DisplacementAlpha;
	}
}

UDMMaterialPropertyDisplacement::UDMMaterialPropertyDisplacement()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::Displacement),
		EDMValueType::VT_Float3_XYZ)
{
}

UMaterialExpression* UDMMaterialPropertyDisplacement::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::ZeroVector);
}

void UDMMaterialPropertyDisplacement::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	Super::GenerateExpressions(InBuildState);

	if (!InBuildState->GetPreviewObject())
	{
		if (UMaterial* GeneratedMaterial = InBuildState->GetDynamicMaterial())
		{
			GeneratedMaterial->SetUsageByFlag(MATUSAGE_Nanite, true);
		}
	}
}

void UDMMaterialPropertyDisplacement::AddAlphaMultiplier(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	UDMMaterialValueFloat1* AlphaValue = GetTypedComponent<UDMMaterialValueFloat1>(UDynamicMaterialModelEditorOnlyData::AlphaValueName);

	if (!AlphaValue)
	{
		return;
	}

	FExpressionInput* PropertyInputExpression = InBuildState->GetMaterialProperty(MaterialProperty);

	if (!PropertyInputExpression || !PropertyInputExpression->Expression)
	{
		return;
	}

	AlphaValue->GenerateExpression(InBuildState);

	UMaterialExpression* GlobalOpacityExpression = InBuildState->GetLastValueExpression(AlphaValue);

	if (!GlobalOpacityExpression)
	{
		return;
	}

	using namespace UE::DynamicMaterialEditor::Private;

	UMaterialFunctionInterface* DisplacementAlphaFunction = GetDisplacementAlpha();

	if (!DisplacementAlphaFunction)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* DisplacementAlphaExpression = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionMaterialFunctionCall>(UE_DM_NodeComment_Default);
	DisplacementAlphaExpression->SetMaterialFunction(DisplacementAlphaFunction);
	DisplacementAlphaExpression->UpdateFromFunctionResource();

	FExpressionInput& ValueInput = DisplacementAlphaExpression->FunctionInputs[0].Input;
	ValueInput.Expression = PropertyInputExpression->Expression;
	ValueInput.Mask = PropertyInputExpression->Mask;
	ValueInput.MaskR = PropertyInputExpression->MaskR;
	ValueInput.MaskG = PropertyInputExpression->MaskG;
	ValueInput.MaskB = PropertyInputExpression->MaskB;
	ValueInput.MaskA = PropertyInputExpression->MaskA;
	ValueInput.OutputIndex = PropertyInputExpression->OutputIndex;

	FExpressionInput& AlphaInput = DisplacementAlphaExpression->FunctionInputs[1].Input;
	AlphaInput.Expression = GlobalOpacityExpression;
	AlphaInput.SetMask(1, 1, 0, 0, 0);
	AlphaInput.OutputIndex = 0;

	PropertyInputExpression->Expression = DisplacementAlphaExpression;
}
