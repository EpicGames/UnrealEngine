// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialProperty.h"
#include "DMMPDisplacement.generated.h"

class UDynamicMaterialModelEditorOnlyData;

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialPropertyDisplacement : public UDMMaterialProperty
{
	GENERATED_BODY()

public:
	UDMMaterialPropertyDisplacement();

	//~ Begin UDMMaterialProperty
	virtual UMaterialExpression* GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	virtual void AddAlphaMultiplier(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialProperty
};
