// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPEmissiveColor.h"

#include "Components/MaterialProperties/DMMPBaseColor.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPEmissiveColor)

UDMMaterialPropertyEmissiveColor::UDMMaterialPropertyEmissiveColor()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::EmissiveColor),
		EDMValueType::VT_Float3_RGB)
{
}

UMaterialExpression* UDMMaterialPropertyEmissiveColor::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::OneVector);
}

TEnumAsByte<EMaterialSamplerType> UDMMaterialPropertyEmissiveColor::GetTextureSamplerType() const
{
	return EMaterialSamplerType::SAMPLERTYPE_Color;
}
