// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPMetallic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPMetallic)

UDMMaterialPropertyMetallic::UDMMaterialPropertyMetallic()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::Metallic),
		EDMValueType::VT_Float1)
{
}

UMaterialExpression* UDMMaterialPropertyMetallic::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, 0.f);
}
