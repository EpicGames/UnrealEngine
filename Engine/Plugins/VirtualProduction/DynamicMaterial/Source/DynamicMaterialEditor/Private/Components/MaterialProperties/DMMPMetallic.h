// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialProperty.h"
#include "DMMPMetallic.generated.h"

class UDynamicMaterialModelEditorOnlyData;

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialPropertyMetallic : public UDMMaterialProperty
{
	GENERATED_BODY()

public:
	UDMMaterialPropertyMetallic();

	//~ Begin UDMMaterialProperty
	virtual UMaterialExpression* GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialProperty
};
