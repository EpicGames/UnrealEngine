// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPNormal.h"

#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPNormal)

namespace UE::DynamicMaterialEditor::Private
{
	UMaterialFunctionInterface* GetSafeNormalize()
	{
		static UMaterialFunctionInterface* NormalizeBlend = FDMMaterialFunctionLibrary::Get().GetFunction(
			"SafeNormalize",
			TEXT("/Script/Engine.MaterialFunction'/Engine/Functions/Engine_MaterialFunctions02/SafeNormalize.SafeNormalize'")
		);

		return NormalizeBlend;
	}

	UMaterialFunctionInterface* GetNormalsStrength()
	{
		static UMaterialFunctionInterface* NormalsStrength = FDMMaterialFunctionLibrary::Get().GetFunction(
			"MF_DM_Normals_Strength",
			TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Effects/Normals/MF_DM_Normals_Strength.MF_DM_Normals_Strength'")
		);

		return NormalsStrength;
	}
}

UDMMaterialPropertyNormal::UDMMaterialPropertyNormal()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::Normal),
		EDMValueType::VT_Float3_XYZ)
{
}

UMaterialExpression* UDMMaterialPropertyNormal::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::OneVector);
}

TEnumAsByte<EMaterialSamplerType> UDMMaterialPropertyNormal::GetTextureSamplerType() const
{
	return EMaterialSamplerType::SAMPLERTYPE_Normal;
}

void UDMMaterialPropertyNormal::AddOutputProcessor(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	Super::AddOutputProcessor(InBuildState);

	FExpressionInput* MaterialPropertyPtr = InBuildState->GetMaterialProperty(MaterialProperty);

	if (!MaterialPropertyPtr)
	{
		return;
	}

	UMaterialExpression* LastPropertyExpression = MaterialPropertyPtr->Expression;

	if (!LastPropertyExpression)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall = FDMMaterialFunctionLibrary::Get().MakeExpression(
		InBuildState->GetDynamicMaterial(),
		UE::DynamicMaterialEditor::Private::GetSafeNormalize(),
		UE_DM_NodeComment_Default
	);

	FExpressionInput* FirstInput = MaterialFunctionCall->GetInput(0);
	if (!FirstInput)
	{
		return;
	}

	LastPropertyExpression->ConnectExpression(FirstInput, MaterialPropertyPtr->OutputIndex);
	MaterialFunctionCall->ConnectExpression(MaterialPropertyPtr, 0);

	MaterialPropertyPtr->OutputIndex = 0;
}

void UDMMaterialPropertyNormal::AddAlphaMultiplier(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	UDMMaterialValueFloat1* AlphaValue = GetTypedComponent<UDMMaterialValueFloat1>(UDynamicMaterialModelEditorOnlyData::AlphaValueName);

	if (!AlphaValue)
	{
		return;
	}

	FExpressionInput* MaterialPropertyPtr = InBuildState->GetMaterialProperty(MaterialProperty);

	if (!MaterialPropertyPtr)
	{
		return;
	}

	UMaterialExpression* LastPropertyExpression = MaterialPropertyPtr->Expression;

	if (!LastPropertyExpression)
	{
		return;
	}

	AlphaValue->GenerateExpression(InBuildState);

	UMaterialExpression* GlobalOpacityExpression = InBuildState->GetLastValueExpression(AlphaValue);

	if (!GlobalOpacityExpression)
	{
		return;
	}

	UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall = FDMMaterialFunctionLibrary::Get().MakeExpression(
		InBuildState->GetDynamicMaterial(),
		UE::DynamicMaterialEditor::Private::GetNormalsStrength(),
		UE_DM_NodeComment_Default
	);

	FExpressionInput* FirstInput = MaterialFunctionCall->GetInput(0);

	if (!FirstInput)
	{
		return;
	}

	FExpressionInput* SecondInput = MaterialFunctionCall->GetInput(1);

	if (!SecondInput)
	{
		return;
	}

	LastPropertyExpression->ConnectExpression(FirstInput, MaterialPropertyPtr->OutputIndex);
	GlobalOpacityExpression->ConnectExpression(SecondInput, 0);
	MaterialFunctionCall->ConnectExpression(MaterialPropertyPtr, 0);

	MaterialPropertyPtr->OutputIndex = 0;
}
