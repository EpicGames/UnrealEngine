// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialProperty.h"
#include "DMMPNormal.generated.h"

class UDynamicMaterialModelEditorOnlyData;

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialPropertyNormal : public UDMMaterialProperty
{
	GENERATED_BODY()

public:
	UDMMaterialPropertyNormal();

	//~ Begin UDMMaterialProperty
	virtual UMaterialExpression* GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	virtual TEnumAsByte<EMaterialSamplerType> GetTextureSamplerType() const override;

	/**
	 * Adds the SafeNormalize function as an extra output processor.
	 */
	virtual void AddOutputProcessor(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/**
	 * Replaces the regular multiply node with a custom one that doesn't affect the Z axis.
	 */
	virtual void AddAlphaMultiplier(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialProperty
};
