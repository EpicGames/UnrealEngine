// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPOpacityMask.h"

#include "Components/MaterialProperties/DMMPOpacity.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPOpacityMask)

UDMMaterialPropertyOpacityMask::UDMMaterialPropertyOpacityMask()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::OpacityMask),
		EDMValueType::VT_Float1)
{
}

UMaterialExpression* UDMMaterialPropertyOpacityMask::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, 1.f);
}

TEnumAsByte<EMaterialSamplerType> UDMMaterialPropertyOpacityMask::GetTextureSamplerType() const
{
	return EMaterialSamplerType::SAMPLERTYPE_Masks;
}
