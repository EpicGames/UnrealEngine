// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPPixelDepthOffset.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPPixelDepthOffset)

UDMMaterialPropertyPixelDepthOffset::UDMMaterialPropertyPixelDepthOffset()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::PixelDepthOffset),
		EDMValueType::VT_Float1)
{
}

UMaterialExpression* UDMMaterialPropertyPixelDepthOffset::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, 0.f);
}
