// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPRefraction.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPRefraction)

UDMMaterialPropertyRefraction::UDMMaterialPropertyRefraction()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::Refraction),
		EDMValueType::VT_Float1)
{
}

UMaterialExpression* UDMMaterialPropertyRefraction::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, 0.f);
}
