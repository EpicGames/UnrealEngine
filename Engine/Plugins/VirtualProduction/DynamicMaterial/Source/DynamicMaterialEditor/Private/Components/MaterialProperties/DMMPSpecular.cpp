// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPSpecular.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPSpecular)

UDMMaterialPropertySpecular::UDMMaterialPropertySpecular()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::Specular),
		EDMValueType::VT_Float3_RGB)
{
}

UMaterialExpression* UDMMaterialPropertySpecular::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::OneVector);
}
