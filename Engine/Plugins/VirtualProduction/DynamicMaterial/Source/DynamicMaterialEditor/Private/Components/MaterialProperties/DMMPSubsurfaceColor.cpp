// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPSubsurfaceColor.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPSubsurfaceColor)

UDMMaterialPropertySubsurfaceColor::UDMMaterialPropertySubsurfaceColor()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::SubsurfaceColor),
		EDMValueType::VT_Float3_RGB)
{
}

UMaterialExpression* UDMMaterialPropertySubsurfaceColor::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::OneVector);
}

TEnumAsByte<EMaterialSamplerType> UDMMaterialPropertySubsurfaceColor::GetTextureSamplerType() const
{
	return EMaterialSamplerType::SAMPLERTYPE_Color;
}
