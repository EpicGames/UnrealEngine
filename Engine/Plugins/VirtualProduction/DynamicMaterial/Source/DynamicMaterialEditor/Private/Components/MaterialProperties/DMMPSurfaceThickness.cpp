// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPSurfaceThickness.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPSurfaceThickness)

UDMMaterialPropertySurfaceThickness::UDMMaterialPropertySurfaceThickness()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::SurfaceThickness),
		EDMValueType::VT_Float1)
{
}

UMaterialExpression* UDMMaterialPropertySurfaceThickness::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, 0.f);
}
