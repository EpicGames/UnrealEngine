// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPTangent.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPTangent)

UDMMaterialPropertyTangent::UDMMaterialPropertyTangent()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::Tangent),
		EDMValueType::VT_Float3_XYZ)
{
}

UMaterialExpression* UDMMaterialPropertyTangent::GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::OneVector);
}
