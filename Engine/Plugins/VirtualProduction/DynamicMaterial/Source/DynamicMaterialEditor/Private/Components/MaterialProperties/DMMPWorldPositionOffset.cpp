// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialProperties/DMMPWorldPositionOffset.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMPWorldPositionOffset)

UDMMaterialPropertyWorldPositionOffset::UDMMaterialPropertyWorldPositionOffset()
	: UDMMaterialProperty(
		EDMMaterialPropertyType(EDMMaterialPropertyType::WorldPositionOffset),
		EDMValueType::VT_Float3_XYZ)
{
}

UMaterialExpression* UDMMaterialPropertyWorldPositionOffset::GetDefaultInput(
	const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	return CreateConstant(InBuildState, FVector::ZeroVector);
}
