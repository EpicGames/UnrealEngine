// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBAdd.h"

#include "Materials/MaterialExpressionAdd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBAdd)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendAdd"

UDMMaterialStageBlendAdd::UDMMaterialStageBlendAdd()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendAdd", "Add"),
		LOCTEXT("BlendAddDescription", "Add increases the brightness of the base layer based on the blend layer on a per channel basis."),
		"DM_Blend_Add", 
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Add.MF_DM_Blend_Add'")
	)
{
}

void UDMMaterialStageBlendAdd::BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
	UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
	TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	CreateBlendOpacityLayer<UMaterialExpressionAdd>(InBuildState, InBaseLayerOpacityExpression, InBaseOutputIndex, InBaseOutputChannel,
		InMyLayerOpacityExpression, InMyOutputIndex, InMyOutputChannel, OutAddedExpressions, OutOutputIndex, OutOutputChannel);
}

#undef LOCTEXT_NAMESPACE
