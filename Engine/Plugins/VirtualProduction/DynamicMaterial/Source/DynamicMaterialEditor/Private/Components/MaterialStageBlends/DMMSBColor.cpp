// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendColor"

UDMMaterialStageBlendColor::UDMMaterialStageBlendColor()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendColor", "Color"),
		LOCTEXT("BlendColorDescription", "Color combines the blend layer's hue and saturation with the base layer's luminosity."),
		"DM_Blend_Color", 
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Color.MF_DM_Blend_Color'")
	)
{
}

#undef LOCTEXT_NAMESPACE
