// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBColorBurn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBColorBurn)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendColorBurn"

UDMMaterialStageBlendColorBurn::UDMMaterialStageBlendColorBurn()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendColorBurn", "ColorBurn"),
		LOCTEXT("BlendColorBurnDescription", "Color Burn is similar to Multiply, but increases the contrast between the base and the blend layers, resulting in more highly saturated mid-tones and reduced highlights."),
		"DM_Blend_ColorBurn",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_ColorBurn.MF_DM_Blend_ColorBurn'")
	)
{
}

#undef LOCTEXT_NAMESPACE
