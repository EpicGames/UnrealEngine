// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBColorDodge.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBColorDodge)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendColorDodge"

UDMMaterialStageBlendColorDodge::UDMMaterialStageBlendColorDodge()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendColorDodge", "ColorDodge"),
		LOCTEXT("BlendColorDodgeDescription", "Color Dodge is similar to Screen, but decreases contrast between the base and blend layer, resulting in saturated mid-tones and blown highlights."),
		"DM_Blend_ColorDodge",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_ColorDodge.MF_DM_Blend_ColorDodge'")
	)
{
}

#undef LOCTEXT_NAMESPACE
