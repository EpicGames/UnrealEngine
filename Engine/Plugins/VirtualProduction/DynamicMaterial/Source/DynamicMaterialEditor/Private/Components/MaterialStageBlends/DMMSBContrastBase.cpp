// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBContrastBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBContrastBase)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendContrastBase"

UDMMaterialStageBlendContrastBase::UDMMaterialStageBlendContrastBase()
	: UDMMaterialStageBlendContrastBase(LOCTEXT("BlendContrastBase", "Contrast Base"), FText::GetEmpty())
{
}

UDMMaterialStageBlendContrastBase::UDMMaterialStageBlendContrastBase(const FText& InName, const FText& InDescription)
	: UDMMaterialStageBlend(InName, InDescription)
{
}

#undef LOCTEXT_NAMESPACE
