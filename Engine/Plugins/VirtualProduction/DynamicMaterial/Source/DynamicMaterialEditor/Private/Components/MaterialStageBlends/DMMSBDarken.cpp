// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBDarken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBDarken)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendDarken"

UDMMaterialStageBlendDarken::UDMMaterialStageBlendDarken()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendDarken", "Darken"),
		LOCTEXT("BlendDarkenDescription", "Darken chooses the color with the lowest luminance between the base and blend layer."),
		"DM_Blend_Darken",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Darken.MF_DM_Blend_Darken'")
	)
{
}

#undef LOCTEXT_NAMESPACE
