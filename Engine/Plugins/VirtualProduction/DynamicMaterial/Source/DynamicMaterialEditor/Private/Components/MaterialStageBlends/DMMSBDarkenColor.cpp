// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBDarkenColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBDarkenColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendDarkenColor"

UDMMaterialStageBlendDarkenColor::UDMMaterialStageBlendDarkenColor()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendDarkenColor", "DarkenColor"),
		LOCTEXT("BlendDarkenColorDescription", "Darken Color chooses the lowest value in each color channel between the base and blend layer."),
		"DM_Blend_DarkenColor",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_DarkenColor.MF_DM_Blend_DarkenColor'")
	)
{
}

#undef LOCTEXT_NAMESPACE
