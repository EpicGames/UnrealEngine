// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBDifference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBDifference)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendDifference"

UDMMaterialStageBlendDifference::UDMMaterialStageBlendDifference()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendDifference", "Difference"),
		LOCTEXT("BlendDifferenceDescription", "Difference outputs the difference in color between the base and blend layers."),
		"DM_Blend_Difference",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Difference.MF_DM_Blend_Difference'")
	)
{
}

#undef LOCTEXT_NAMESPACE
