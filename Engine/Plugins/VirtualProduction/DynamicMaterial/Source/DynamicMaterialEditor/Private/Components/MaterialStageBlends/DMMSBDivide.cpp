// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBDivide.h"

#include "Materials/MaterialExpressionDivide.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBDivide)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendDivide"

UDMMaterialStageBlendDivide::UDMMaterialStageBlendDivide()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendDivide", "Divide"),
		LOCTEXT("BlendDivideDescription", "Divide increases the brightness of the base layer as the blend layer gets darker. Black and white both produce no change."),
		"DM_Blend_Divide",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Divide.MF_DM_Blend_Divide'")
	)
{
}

void UDMMaterialStageBlendDivide::BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
	UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
	TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	CreateBlendOpacityLayer<UMaterialExpressionDivide>(InBuildState, InBaseLayerOpacityExpression, InBaseOutputIndex, InBaseOutputChannel,
		InMyLayerOpacityExpression, InMyOutputIndex, InMyOutputChannel, OutAddedExpressions, OutOutputIndex, OutOutputChannel);
}

#undef LOCTEXT_NAMESPACE
