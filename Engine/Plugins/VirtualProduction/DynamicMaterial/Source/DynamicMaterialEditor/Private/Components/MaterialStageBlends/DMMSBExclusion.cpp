// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBExclusion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBExclusion)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendExclusion"

UDMMaterialStageBlendExclusion::UDMMaterialStageBlendExclusion()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendExclusion", "Exclusion"),
		LOCTEXT("BlendExclusionDescription", "Exclusion inverts the base layer colors based on the strength of the blend layer."),
		"DM_Blend_Exclusion",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Exclusion.MF_DM_Blend_Exclusion'")
	)
{
}

#undef LOCTEXT_NAMESPACE
