// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBHardLight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBHardLight)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendHardLight"

UDMMaterialStageBlendHardLight::UDMMaterialStageBlendHardLight()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendHardLight", "HardLight"),
		LOCTEXT("BlendHardLightDescription", "Hard Light is a combination of Multiply and Screen using the brightness of the blend layer."),
		"DM_Blend_HardLight",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_HardLight.MF_DM_Blend_HardLight'")
	)
{
}

#undef LOCTEXT_NAMESPACE
