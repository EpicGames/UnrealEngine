// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBHardMix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBHardMix)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendHardMix"

UDMMaterialStageBlendHardMix::UDMMaterialStageBlendHardMix()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendHardMix", "HardMix"),
		LOCTEXT("BlendHardMixDescription", "Hard Mix adds the individual color channels together and rounds them, producing one of eight colors: black, red, green, blue, cyan, magenta, yellow, or white."),
		"DM_Blend_HardMix",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_HardMix.MF_DM_Blend_HardMix'")
	)
{
}

#undef LOCTEXT_NAMESPACE
