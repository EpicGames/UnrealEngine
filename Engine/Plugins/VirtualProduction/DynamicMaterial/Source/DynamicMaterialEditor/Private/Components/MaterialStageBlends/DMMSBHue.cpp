// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBHue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBHue)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendHue"

UDMMaterialStageBlendHue::UDMMaterialStageBlendHue()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendHue", "Hue"),
		LOCTEXT("BlendHueDescription", "Hue combines the blend layer's hue with the base layer's luminosity and saturation."),
		"DM_Blend_Hue", 
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Hue.MF_DM_Blend_Hue'")
	)
{
}

#undef LOCTEXT_NAMESPACE
