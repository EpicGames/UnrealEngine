// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBLighten.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBLighten)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendLighten"

UDMMaterialStageBlendLighten::UDMMaterialStageBlendLighten()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendLighten", "Lighten"),
		LOCTEXT("BlendLightenDescription", "Lighten chooses the color with the highest luminance between the base and blend layer."),
		"DM_Blend_Lighten",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Lighten.MF_DM_Blend_Lighten'")
	)
{
}

#undef LOCTEXT_NAMESPACE
