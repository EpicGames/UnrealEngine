// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBLightenColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBLightenColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendLightenColor"

UDMMaterialStageBlendLightenColor::UDMMaterialStageBlendLightenColor()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendLightenColor", "LightenColor"),
		LOCTEXT("BlendLightenColorDescription", "Lighten Color chooses the highest value in each color channel between the base and blend layer."),
		"DM_Blend_LightenColor",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_LightenColor.MF_DM_Blend_LightenColor'")
	)
{
}

#undef LOCTEXT_NAMESPACE
