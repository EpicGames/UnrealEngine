// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBLinearBurn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBLinearBurn)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendLinearBurn"

UDMMaterialStageBlendLinearBurn::UDMMaterialStageBlendLinearBurn()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendLinearBurn", "LinearBurn"),
		LOCTEXT("BlendLinearBurnDescription", "Linear Burn decreases the brightness of the base layer based on the value of the blend layer."),
		"DM_Blend_LinearBurn",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_LinearBurn.MF_DM_Blend_LinearBurn'")
	)
{
}

#undef LOCTEXT_NAMESPACE
