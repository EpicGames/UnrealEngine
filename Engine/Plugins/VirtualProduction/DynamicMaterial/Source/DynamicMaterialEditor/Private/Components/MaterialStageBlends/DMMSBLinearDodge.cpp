// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBLinearDodge.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBLinearDodge)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendLinearDodge"

UDMMaterialStageBlendLinearDodge::UDMMaterialStageBlendLinearDodge()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendLinearDodge", "LinearDodge"),
		LOCTEXT("BlendLinearDodgeDescription", "Linear Dodge increases the brightness of the base layer based on the blend layer on a per channel basis."),
		"DM_Blend_LinearDodge",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_LinearDodge.MF_DM_Blend_LinearDodge'")
	)
{
}

#undef LOCTEXT_NAMESPACE
