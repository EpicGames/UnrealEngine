// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBLinearLight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBLinearLight)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendLinearLight"

UDMMaterialStageBlendLinearLight::UDMMaterialStageBlendLinearLight()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendLinearLight", "LinearLight"),
		LOCTEXT("BlendLinearLightDescription", "Linear Light applies Linear Burn below 50% gray and Linear Dodge above 50% gray."),
		"DM_Blend_LinearLight",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_LinearLight.MF_DM_Blend_LinearLight'")
	)
{
}

#undef LOCTEXT_NAMESPACE
