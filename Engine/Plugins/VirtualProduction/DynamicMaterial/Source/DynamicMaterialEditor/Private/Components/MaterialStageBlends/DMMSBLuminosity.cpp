// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBLuminosity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBLuminosity)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendLuminosity"

UDMMaterialStageBlendLuminosity::UDMMaterialStageBlendLuminosity()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendLuminosity", "Luminosity"),
		LOCTEXT("BlendLuminosityDescription", "Luminosity combines the blend layer's luminosity with the base layer's hue and saturation."),
		"DM_Blend_Luminosity", 
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Luminosity.MF_DM_Blend_Luminosity'")
	)
{
}

#undef LOCTEXT_NAMESPACE
