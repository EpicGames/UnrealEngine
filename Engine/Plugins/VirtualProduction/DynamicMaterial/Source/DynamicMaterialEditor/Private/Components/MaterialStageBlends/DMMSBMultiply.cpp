// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBMultiply.h"

#include "Materials/MaterialExpressionMultiply.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBMultiply)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendMultiply"

UDMMaterialStageBlendMultiply::UDMMaterialStageBlendMultiply()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendMultiply", "Multiply"),
		LOCTEXT("BlendMultiplyDescription", "Multiply multiplies the luminosity of the base and blend layers."),
		"DM_Blend_Multiply",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Multiply.MF_DM_Blend_Multiply'")
	)
{
}

void UDMMaterialStageBlendMultiply::BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
	UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
	TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	CreateBlendOpacityLayer<UMaterialExpressionMultiply>(InBuildState, InBaseLayerOpacityExpression, InBaseOutputIndex, InBaseOutputChannel,
		InMyLayerOpacityExpression, InMyOutputIndex, InMyOutputChannel, OutAddedExpressions, OutOutputIndex, OutOutputChannel);
}

#undef LOCTEXT_NAMESPACE
