// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBNormal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBNormal)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendNormal"

UDMMaterialStageBlendNormal::UDMMaterialStageBlendNormal()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendNormal", "Normal"),
		LOCTEXT("BlendNormalDescription", "Normal blends replace the pixels below them without any blending unless the layer opacity is below 100%."),
		"DM_Blend_Normal",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Normal.MF_DM_Blend_Normal'")
	)
{
}

#undef LOCTEXT_NAMESPACE
