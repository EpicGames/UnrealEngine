// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBOverlay)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendOverlay"

UDMMaterialStageBlendOverlay::UDMMaterialStageBlendOverlay()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendOverlay", "Overlay"),
		LOCTEXT("BlendOverlayDescription", "Overlay blends the base and blend layer with Multiply (at half strength) below 50% gray and Screen (at half strength) above 50% gray."),
		"DM_Blend_Overlay",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Overlay.MF_DM_Blend_Overlay'")
	)
{
}

#undef LOCTEXT_NAMESPACE
