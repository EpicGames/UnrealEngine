// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBPinLight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBPinLight)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendPinLight"

UDMMaterialStageBlendPinLight::UDMMaterialStageBlendPinLight()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendPinLight", "PinLight"),
		LOCTEXT("BlendPinLightDescription", "Pin Light uses both Darken and Lighten simultaneously."),
		"DM_Blend_PinLight",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_PinLight.MF_DM_Blend_PinLight'")
	)
{
}

#undef LOCTEXT_NAMESPACE
