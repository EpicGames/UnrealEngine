// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBSaturation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBSaturation)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendSaturation"

UDMMaterialStageBlendSaturation::UDMMaterialStageBlendSaturation()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendSaturation", "Saturation"),
		LOCTEXT("BlendSaturationDescription", "Saturation combines the blend layer's saturation with the base layer's luminosity and hue."),
		"DM_Blend_Saturation", 
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Saturation.MF_DM_Blend_Saturation'")
	)
{
}

#undef LOCTEXT_NAMESPACE
