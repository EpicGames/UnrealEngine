// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBScreen.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBScreen)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendScreen"

UDMMaterialStageBlendScreen::UDMMaterialStageBlendScreen()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendScreen", "Screen"),
		LOCTEXT("BlendScreenDescription", "Screen increases the brightness of the base layer based on the blend layer."),
		"DM_Blend_Screen",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Screen.MF_DM_Blend_Screen'")
	)
{
}

#undef LOCTEXT_NAMESPACE
