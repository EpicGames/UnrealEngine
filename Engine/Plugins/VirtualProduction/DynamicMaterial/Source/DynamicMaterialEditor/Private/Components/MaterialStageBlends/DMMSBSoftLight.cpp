// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBSoftLight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBSoftLight)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendSoftLight"

UDMMaterialStageBlendSoftLight::UDMMaterialStageBlendSoftLight()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendSoftLight", "SoftLight"),
		LOCTEXT("BlendSoftLightDescription", "Soft Light is similar to Overlay, but much more subtle."),
		"DM_Blend_SoftLight",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_SoftLight.MF_DM_Blend_SoftLight'")
	)
{
}

#undef LOCTEXT_NAMESPACE
