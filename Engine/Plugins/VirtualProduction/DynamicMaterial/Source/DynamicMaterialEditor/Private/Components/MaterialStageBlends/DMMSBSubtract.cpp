// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBSubtract.h"

#include "Materials/MaterialExpressionSubtract.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBSubtract)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendSubtract"

UDMMaterialStageBlendSubtract::UDMMaterialStageBlendSubtract()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendSubtract", "Subtract"),
		LOCTEXT("BlendSubtractDescription", "Subtract subtracts the blend layer's color from the base layer. As the blend layer gets brighter, the base layer gets darker."),
		"DM_Blend_Subtract",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_Subtract.MF_DM_Blend_Subtract'")
	)
{
}

void UDMMaterialStageBlendSubtract::BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
	UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
	TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	CreateBlendOpacityLayer<UMaterialExpressionSubtract>(InBuildState, InBaseLayerOpacityExpression, InBaseOutputIndex, InBaseOutputChannel,
		InMyLayerOpacityExpression, InMyOutputIndex, InMyOutputChannel, OutAddedExpressions, OutOutputIndex, OutOutputChannel);
}

#undef LOCTEXT_NAMESPACE
