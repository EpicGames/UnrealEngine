// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageBlends/DMMSBVividLight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSBVividLight)

#define LOCTEXT_NAMESPACE "DMMaterialStageBlendVividLight"

UDMMaterialStageBlendVividLight::UDMMaterialStageBlendVividLight()
	: UDMMaterialStageBlendFunction(
		LOCTEXT("BlendVividLight", "VividLight"),
		LOCTEXT("BlendVividLightDescription", "Vivid Light darkens colors below 50% gray and lightens colors above 50% gray."),
		"DM_Blend_VividLight",
		TEXT("/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Blends/MF_DM_Blend_VividLight.MF_DM_Blend_VividLight'")
	)
{
}

#undef LOCTEXT_NAMESPACE
