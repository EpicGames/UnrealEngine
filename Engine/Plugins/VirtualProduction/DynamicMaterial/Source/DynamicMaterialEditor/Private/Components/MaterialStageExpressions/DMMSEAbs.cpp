// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEAbs.h"
#include "Materials/MaterialExpressionAbs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEAbs)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionAbs"

UDMMaterialStageExpressionAbs::UDMMaterialStageExpressionAbs()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Abs", "Abs"),
		UMaterialExpressionAbs::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
