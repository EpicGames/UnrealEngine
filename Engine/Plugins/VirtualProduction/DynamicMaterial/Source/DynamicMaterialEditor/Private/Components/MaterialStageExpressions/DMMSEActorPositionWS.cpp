// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEActorPositionWS.h"
#include "Materials/MaterialExpressionActorPositionWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEActorPositionWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionActorPositionWS"

UDMMaterialStageExpressionActorPositionWS::UDMMaterialStageExpressionActorPositionWS()
	: UDMMaterialStageExpression(
		LOCTEXT("ActorPositionWS", "Actor Position (WS)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionActorPositionWS"))
	)
{
	Menus.Add(EDMExpressionMenu::Object);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
