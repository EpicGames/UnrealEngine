// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEAdd.h"
#include "Materials/MaterialExpressionAdd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEAdd)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionAdd"

UDMMaterialStageExpressionAdd::UDMMaterialStageExpressionAdd()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Add", "Add"),
		UMaterialExpressionAdd::StaticClass()
	)
{
	SetupInputs(2);
}

#undef LOCTEXT_NAMESPACE
