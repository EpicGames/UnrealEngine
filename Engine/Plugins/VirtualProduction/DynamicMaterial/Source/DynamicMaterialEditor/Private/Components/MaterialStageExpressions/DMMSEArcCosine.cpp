// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArcCosine.h"

#include "Materials/MaterialExpressionArccosine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArcCosine)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArccosine"

UDMMaterialStageExpressionArccosine::UDMMaterialStageExpressionArccosine()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Arccosine", "Arccosine"),
		UMaterialExpressionArccosine::StaticClass()
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("A/H", "A/H");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
