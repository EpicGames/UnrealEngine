// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArcCosineFast.h"

#include "Materials/MaterialExpressionArccosineFast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArcCosineFast)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArccosineFast"

UDMMaterialStageExpressionArccosineFast::UDMMaterialStageExpressionArccosineFast()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("ArccosineFast", "Arccosine (Fast)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionArccosineFast"))
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("A/H", "A/H");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
