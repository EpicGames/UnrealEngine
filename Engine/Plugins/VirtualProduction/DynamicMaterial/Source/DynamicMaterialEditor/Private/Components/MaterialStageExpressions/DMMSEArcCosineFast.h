// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageExpressions/DMMSEMathBase.h"

#include "DMMSEArcCosineFast.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionArccosineFast : public UDMMaterialStageExpressionMathBase
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionArccosineFast();
};
