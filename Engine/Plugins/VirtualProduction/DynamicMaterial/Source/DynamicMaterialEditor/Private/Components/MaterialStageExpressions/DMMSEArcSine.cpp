// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArcSine.h"

#include "Materials/MaterialExpressionArcsine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArcSine)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArcsine"

UDMMaterialStageExpressionArcsine::UDMMaterialStageExpressionArcsine()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Arcsine", "Arcsine"),
		UMaterialExpressionArcsine::StaticClass()
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("O/H", "O/H");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
