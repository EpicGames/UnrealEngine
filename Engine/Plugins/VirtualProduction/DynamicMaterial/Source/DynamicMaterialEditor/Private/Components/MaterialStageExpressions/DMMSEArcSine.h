// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageExpressions/DMMSEMathBase.h"

#include "DMMSEArcSine.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionArcsine : public UDMMaterialStageExpressionMathBase
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionArcsine();
};
