// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArcSineFast.h"

#include "Materials/MaterialExpressionArcsineFast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArcSineFast)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArcsineFast"

UDMMaterialStageExpressionArcsineFast::UDMMaterialStageExpressionArcsineFast()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("ArcsineFast", "Arcsine (Fast)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionArcsineFast"))
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("O/H", "O/H");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
