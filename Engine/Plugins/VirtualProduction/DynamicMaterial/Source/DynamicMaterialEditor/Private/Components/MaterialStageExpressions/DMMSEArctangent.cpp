// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArctangent.h"
#include "Materials/MaterialExpressionArctangent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArctangent)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArctangent"

UDMMaterialStageExpressionArctangent::UDMMaterialStageExpressionArctangent()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Arctangent", "Arctangent"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionArctangent"))
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("O/A", "O/A");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
