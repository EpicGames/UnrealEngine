// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArctangent2.h"
#include "Materials/MaterialExpressionArctangent2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArctangent2)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArctangent2"

UDMMaterialStageExpressionArctangent2::UDMMaterialStageExpressionArctangent2()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Arctangent2", "Arctangent2"),
		UMaterialExpressionArctangent2::StaticClass()
	)
{
	SetupInputs(2);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("O", "O");
	InputConnectors[1].Name = LOCTEXT("H", "H");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
