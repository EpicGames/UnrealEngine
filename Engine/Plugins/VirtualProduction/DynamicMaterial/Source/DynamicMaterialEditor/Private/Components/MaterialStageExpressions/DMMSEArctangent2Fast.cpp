// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArctangent2Fast.h"
#include "Materials/MaterialExpressionArctangent2Fast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArctangent2Fast)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArctangent2Fast"

UDMMaterialStageExpressionArctangent2Fast::UDMMaterialStageExpressionArctangent2Fast()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Arctangent2Fast", "Arctangent2 (Fast)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionArctangent2Fast"))
	)
{
	SetupInputs(2);

	InputConnectors[0].Name = LOCTEXT("O", "O");
	InputConnectors[1].Name = LOCTEXT("H", "H");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
