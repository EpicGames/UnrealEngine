// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEArctangentFast.h"
#include "Materials/MaterialExpressionArctangentFast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEArctangentFast)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionArctangentFast"

UDMMaterialStageExpressionArctangentFast::UDMMaterialStageExpressionArctangentFast()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("ArctangentFast", "Arctangent (Fast)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionArctangentFast"))
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("O/A", "O/A");
	OutputConnectors[0].Name = LOCTEXT("Angle", "Angle");
}

#undef LOCTEXT_NAMESPACE
