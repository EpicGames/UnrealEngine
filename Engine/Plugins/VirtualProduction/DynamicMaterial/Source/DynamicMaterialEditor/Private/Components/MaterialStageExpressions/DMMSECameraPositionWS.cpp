// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSECameraPositionWS.h"
#include "Materials/MaterialExpressionCameraPositionWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSECameraPositionWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionCameraPositionWS"

UDMMaterialStageExpressionCameraPositionWS::UDMMaterialStageExpressionCameraPositionWS()
	: UDMMaterialStageExpression(
		LOCTEXT("CameraPositionWS", "Camera Position (WS)"),
		UMaterialExpressionCameraPositionWS::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Camera);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
