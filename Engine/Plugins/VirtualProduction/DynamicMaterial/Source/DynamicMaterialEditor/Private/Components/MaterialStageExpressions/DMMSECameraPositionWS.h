// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSECameraPositionWS.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionCameraPositionWS : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionCameraPositionWS();
};
