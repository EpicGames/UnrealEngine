// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSECameraVectorWS.h"
#include "Materials/MaterialExpressionCameraVectorWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSECameraVectorWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionCameraVectorWS"

UDMMaterialStageExpressionCameraVectorWS::UDMMaterialStageExpressionCameraVectorWS()
	: UDMMaterialStageExpression(
		LOCTEXT("CameraVectorWS", "Camera Vector (WS)"),
		UMaterialExpressionCameraVectorWS::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Camera);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Vector", "Vector"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
