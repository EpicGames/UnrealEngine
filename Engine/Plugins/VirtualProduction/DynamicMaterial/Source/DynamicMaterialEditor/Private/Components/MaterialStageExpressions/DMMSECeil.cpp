// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSECeil.h"
#include "Materials/MaterialExpressionCeil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSECeil)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionCeil"

UDMMaterialStageExpressionCeil::UDMMaterialStageExpressionCeil()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Ceil", "Ceil"),
		UMaterialExpressionCeil::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
