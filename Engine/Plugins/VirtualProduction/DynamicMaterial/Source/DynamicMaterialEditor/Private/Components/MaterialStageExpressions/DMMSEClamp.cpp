// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialStageExpressions/DMMSEClamp.h"
#include "Components/DMMaterialStage.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/MaterialExpressionClamp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEClamp)
 
#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionClamp"
 
UDMMaterialStageExpressionClamp::UDMMaterialStageExpressionClamp()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Clamp", "Clamp"),
		UMaterialExpressionClamp::StaticClass()
	)
{
	SetupInputs(1);
 
	InputConnectors[0].Name = LOCTEXT("Input", "Input");
 
	InputConnectors.Add({1, LOCTEXT("Min", "Min"), EDMValueType::VT_Float1});
	InputConnectors.Add({2, LOCTEXT("Max", "Max"), EDMValueType::VT_Float1});
}
 
void UDMMaterialStageExpressionClamp::AddDefaultInput(int32 InInputIndex) const
{
	UDMMaterialStageExpressionMathBase::AddDefaultInput(InInputIndex);
 
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	switch (InInputIndex)
	{
		case 1:
		{
			UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
			check(InputValue);
 
			UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
			check(Float1Value);
 
			Float1Value->SetValue(0.f);
			break;
		}
 
		case 2:
		{
			UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
			check(InputValue);
 
			UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
			check(Float1Value);
 
			Float1Value->SetDefaultValue(1.f);
			Float1Value->ApplyDefaultValue();
			break;
		}
 
		default:
			// Do nothing
			break;
	}
}
 
#undef LOCTEXT_NAMESPACE
