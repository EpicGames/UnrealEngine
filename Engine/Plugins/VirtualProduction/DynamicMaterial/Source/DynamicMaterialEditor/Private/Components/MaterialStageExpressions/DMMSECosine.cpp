// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSECosine.h"
#include "Materials/MaterialExpressionCosine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSECosine)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionCosine"

UDMMaterialStageExpressionCosine::UDMMaterialStageExpressionCosine()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Cosine", "Cosine"),
		UMaterialExpressionCosine::StaticClass()
	)
{
	SetupInputs(1);

	InputConnectors[0].Name = LOCTEXT("Angle", "Angle");
	OutputConnectors[0].Name = LOCTEXT("A/H", "A/H");
}

#undef LOCTEXT_NAMESPACE
