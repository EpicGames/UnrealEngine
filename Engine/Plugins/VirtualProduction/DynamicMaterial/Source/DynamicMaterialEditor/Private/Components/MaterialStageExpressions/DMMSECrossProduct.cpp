// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSECrossProduct.h"
#include "Materials/MaterialExpressionCrossProduct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSECrossProduct)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionCrossProduct"

UDMMaterialStageExpressionCrossProduct::UDMMaterialStageExpressionCrossProduct()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("CrossProduct", "Cross Product"),
		UMaterialExpressionCrossProduct::StaticClass()
	)
{
	SetupInputs(2);

	bAllowSingleFloatMatch = false;
}

#undef LOCTEXT_NAMESPACE
