// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEDeltaTime.h"
#include "Materials/MaterialExpressionDeltaTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEDeltaTime)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionDeltaTime"

UDMMaterialStageExpressionDeltaTime::UDMMaterialStageExpressionDeltaTime()
	: UDMMaterialStageExpression(
		LOCTEXT("DeltaTime", "DeltaTime"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionDeltaTime"))
	)
{
	Menus.Add(EDMExpressionMenu::Time);

	OutputConnectors.Add({0, LOCTEXT("DeltaTimeLabel", "Delta Time"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
