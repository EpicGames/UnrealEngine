// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialStageExpressions/DMMSEDivide.h"
#include "Components/DMMaterialStage.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/MaterialExpressionDivide.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEDivide)
 
#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionDivide"
 
UDMMaterialStageExpressionDivide::UDMMaterialStageExpressionDivide()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Divide", "Divide"),
		UMaterialExpressionDivide::StaticClass()
	)
{
	SetupInputs(2);
}
 
void UDMMaterialStageExpressionDivide::AddDefaultInput(int32 InInputIndex) const
{
	if (InInputIndex != 1)
	{
		UDMMaterialStageExpressionMathBase::AddDefaultInput(InInputIndex);
		return;
	}
 
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		Stage, 
		InInputIndex,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
		EDMValueType::VT_Float1,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);
 
	UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
	check(InputValue);
 
	UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
	check(Float1Value);
 
	Float1Value->SetDefaultValue(2.f);
	Float1Value->ApplyDefaultValue();
}
 
#undef LOCTEXT_NAMESPACE
