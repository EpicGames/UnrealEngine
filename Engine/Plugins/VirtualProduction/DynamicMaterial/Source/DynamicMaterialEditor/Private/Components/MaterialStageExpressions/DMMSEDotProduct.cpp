// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEDotProduct.h"
#include "Materials/MaterialExpressionDotProduct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEDotProduct)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionDotProduct"

UDMMaterialStageExpressionDotProduct::UDMMaterialStageExpressionDotProduct()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("DotProduct", "Dot Product"),
		UMaterialExpressionDotProduct::StaticClass()
	)
{
	SetupInputs(2);
}

#undef LOCTEXT_NAMESPACE
