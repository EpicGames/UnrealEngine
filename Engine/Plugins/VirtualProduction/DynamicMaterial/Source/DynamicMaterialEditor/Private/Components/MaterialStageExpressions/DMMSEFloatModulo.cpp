// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialStageExpressions/DMMSEFloatModulo.h"
#include "Components/DMMaterialStage.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/MaterialExpressionFmod.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEFloatModulo)
 
#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionFloatModulo"
 
UDMMaterialStageExpressionFloatModulo::UDMMaterialStageExpressionFloatModulo()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("FloatModulo", "Modulo (Float)"),
		UMaterialExpressionFmod::StaticClass()
	)
{
	SetupInputs(1);
 
	InputConnectors.Add({1, LOCTEXT("Divisor", "Divisor"), EDMValueType::VT_Float1});
}
 
void UDMMaterialStageExpressionFloatModulo::AddDefaultInput(int32 InInputIndex) const
{
	UDMMaterialStageExpressionMathBase::AddDefaultInput(InInputIndex);
 
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	switch (InInputIndex)
	{
		case 1:
		{
			UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
			check(InputValue);
 
			UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
			check(Float1Value);
 
			Float1Value->SetDefaultValue(1.f);
			Float1Value->ApplyDefaultValue();
			break;
		}
 
		default:
			// No nothing
			break;
	}
}
 
#undef LOCTEXT_NAMESPACE
