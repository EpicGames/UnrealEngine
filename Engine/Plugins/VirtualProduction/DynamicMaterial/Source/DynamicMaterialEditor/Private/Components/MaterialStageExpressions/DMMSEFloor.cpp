// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEFloor.h"
#include "Materials/MaterialExpressionFloor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEFloor)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionFloor"

UDMMaterialStageExpressionFloor::UDMMaterialStageExpressionFloor()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Floor", "Floor"),
		UMaterialExpressionFloor::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
