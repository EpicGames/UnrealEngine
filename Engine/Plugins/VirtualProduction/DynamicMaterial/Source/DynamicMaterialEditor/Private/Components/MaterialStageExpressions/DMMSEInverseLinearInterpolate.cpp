// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEInverseLinearInterpolate.h"
#include "Materials/MaterialExpressionInverseLinearInterpolate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEInverseLinearInterpolate)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionInverseLinearInterpolate"

UDMMaterialStageExpressionInverseLinearInterpolate::UDMMaterialStageExpressionInverseLinearInterpolate()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("InverseLinearInterpolate", "Inverse Linear Interpolate"),
		UMaterialExpressionInverseLinearInterpolate::StaticClass()
	)
{
	SetupInputs(2);

	bAllowSingleFloatMatch = false;

	InputConnectors[0].Name = LOCTEXT("Min", "Min");
	InputConnectors[1].Name = LOCTEXT("Max", "Max");

	InputConnectors.Add({2, LOCTEXT("Value", "Value"), EDMValueType::VT_Float1});

	OutputConnectors[0].Name = LOCTEXT("Alpha", "Alpha");
}

#undef LOCTEXT_NAMESPACE
