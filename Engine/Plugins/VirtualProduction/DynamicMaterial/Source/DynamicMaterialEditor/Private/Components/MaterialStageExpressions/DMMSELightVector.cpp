// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSELightVector.h"
#include "Materials/MaterialExpressionLightVector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSELightVector)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionLightVector"

UDMMaterialStageExpressionLightVector::UDMMaterialStageExpressionLightVector()
	: UDMMaterialStageExpression(
		LOCTEXT("LightVector", "Light Vector"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionLightVector"))
	)
{
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Vector", "Vector"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
