// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSELightVector.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionLightVector : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionLightVector();
};
