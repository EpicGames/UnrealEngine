// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSELightmapUVs.h"
#include "Materials/MaterialExpressionLightmapUVs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSELightmapUVs)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionLightmapUVs"

UDMMaterialStageExpressionLightmapUVs::UDMMaterialStageExpressionLightmapUVs()
	: UDMMaterialStageExpression(
		LOCTEXT("LightmapUVs", "Lightmap UVs"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionLightmapUVs"))
	)
{
	Menus.Add(EDMExpressionMenu::Texture);

	OutputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
}

#undef LOCTEXT_NAMESPACE
