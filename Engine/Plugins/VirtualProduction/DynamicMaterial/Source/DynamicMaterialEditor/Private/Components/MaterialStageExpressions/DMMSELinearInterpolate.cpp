// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Components/MaterialStageExpressions/DMMSELinearInterpolate.h"
#include "Components/DMMaterialStage.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSELinearInterpolate)
 
#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionLinearInterpolate"
 
UDMMaterialStageExpressionLinearInterpolate::UDMMaterialStageExpressionLinearInterpolate()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("LinearInterpolate", "Linear Interpolate"),
		UMaterialExpressionLinearInterpolate::StaticClass()
	)
{
	SetupInputs(2);
 
	bAllowSingleFloatMatch = false;
 
	InputConnectors[0].Name = LOCTEXT("Min", "Min");
	InputConnectors[1].Name = LOCTEXT("Max", "Max");
 
	InputConnectors.Add({2, LOCTEXT("Alpha", "Alpha"), EDMValueType::VT_Float1});
 
	OutputConnectors[0].Name = LOCTEXT("Value", "Value");
}
 
void UDMMaterialStageExpressionLinearInterpolate::AddDefaultInput(int32 InInputIndex) const
{
	if (InInputIndex != 2)
	{
		UDMMaterialStageExpressionMathBase::AddDefaultInput(InInputIndex);
		return;
	}
 
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
 
	UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(Stage, InInputIndex,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, EDMValueType::VT_Float1,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);
 
	UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
	check(InputValue);
 
	UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InputValue->GetValue());
	check(Float1Value);
 
	Float1Value->SetDefaultValue(0.5f);
	Float1Value->ApplyDefaultValue();
	Float1Value->SetValueRange(FFloatInterval(0, 1));
}
 
#undef LOCTEXT_NAMESPACE
