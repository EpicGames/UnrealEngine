// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSELogarithm10.h"
#include "Materials/MaterialExpressionLogarithm10.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSELogarithm10)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionLogarithm10"

UDMMaterialStageExpressionLogarithm10::UDMMaterialStageExpressionLogarithm10()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Logarithm10", "Log (Base 10)"),
		UMaterialExpressionLogarithm10::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
