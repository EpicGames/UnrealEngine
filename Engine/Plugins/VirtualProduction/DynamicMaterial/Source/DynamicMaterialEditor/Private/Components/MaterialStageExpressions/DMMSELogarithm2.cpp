// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSELogarithm2.h"
#include "Materials/MaterialExpressionLogarithm2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSELogarithm2)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionLogarithm2"

UDMMaterialStageExpressionLogarithm2::UDMMaterialStageExpressionLogarithm2()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Logarithm2", "Log (Base 2)"),
		UMaterialExpressionLogarithm2::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
