// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEMathBase.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageInput.h"
#include "Materials/MaterialExpression.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMInputNodeBuilder.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEMathBase)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionMathBase"

UDMMaterialStageExpressionMathBase::UDMMaterialStageExpressionMathBase()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("MathBase", "Math Base"),
		UMaterialExpression::StaticClass()
	)
{
}

UDMMaterialStageExpressionMathBase::UDMMaterialStageExpressionMathBase(const FText& InName, TSubclassOf<UMaterialExpression> InClass)
	: UDMMaterialStageExpression(InName, InClass)
	, bSingleChannelOnly(false)
	, VariadicInputCount(255)
	, bAllowSingleFloatMatch(true)
{
	bInputRequired = true;

	Menus.Add(EDMExpressionMenu::Math);

	OutputConnectors.Add({0, LOCTEXT("Output", "Output"), EDMValueType::VT_Float_Any});
}

bool UDMMaterialStageExpressionMathBase::CanInputAcceptType(int32 InInputIndex, EDMValueType InValueType) const
{
	check(InputConnectors.IsValidIndex(InInputIndex));

	if (!UDMValueDefinitionLibrary::GetValueDefinition(InValueType).IsFloatType())
	{
		return false;
	}

	if (InInputIndex == 0)
	{
		return true;
	}

	if (InInputIndex < VariadicInputCount)
	{
		UDMMaterialStage* Stage = GetStage();
		check(Stage);

		TArray<FDMMaterialStageConnection>& InputConnections = Stage->GetInputConnectionMap();

		// TODO Sub channel mapping
		EDMValueType InputType = EDMValueType::VT_None;

		for (const FDMMaterialStageConnectorChannel& Channel : InputConnections[0].Channels)
		{
			switch (Channel.SourceIndex)
			{
				case FDMMaterialStageConnectorChannel::NO_SOURCE:
					continue; // For loop

				case FDMMaterialStageConnectorChannel::PREVIOUS_STAGE:
				{
					UDMMaterialLayerObject* Layer = Stage->GetLayer();
					check(Layer);

					if (UDMMaterialLayerObject* PreviousLayer = Layer->GetPreviousLayer(Channel.MaterialProperty, EDMMaterialLayerStage::Base))
					{
						UDMMaterialStage* PreviousStage = PreviousLayer->GetLastEnabledStage(EDMMaterialLayerStage::All);
						check(PreviousStage);

						if (UDMMaterialStageSource* PreviousStageSource = PreviousStage->GetSource())
						{
							const TArray<FDMMaterialStageConnector>& PreviousStageOutputs = PreviousStageSource->GetOutputConnectors();

							if (Channel.OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
							{
								if (PreviousStageOutputs.IsValidIndex(Channel.OutputIndex))
								{
									InputType = PreviousStageOutputs[Channel.OutputIndex].Type;
								}

								goto FoundInputType;
							}
							else
							{
								switch (InputType)
								{
									case EDMValueType::VT_None:
										InputType = EDMValueType::VT_Float1;
										break;

									case EDMValueType::VT_Float1:
										InputType = EDMValueType::VT_Float2;
										break;

									case EDMValueType::VT_Float2:
										InputType = EDMValueType::VT_Float3_XYZ;
										break;

									case EDMValueType::VT_Float3_RGB:
									case EDMValueType::VT_Float3_RPY:
									case EDMValueType::VT_Float3_XYZ:
										InputType = EDMValueType::VT_Float4_RGBA;
										break;

									// Not valid
									default:
									case EDMValueType::VT_Float4_RGBA:
										return false;
								}
							}
						}
					}

					break; // Switch
				}

				// Inputs
				default:
				{
					const int32 StageInputIdx = Channel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
					const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();

					if (StageInputs.IsValidIndex(StageInputIdx))
					{
						const TArray<FDMMaterialStageConnector>& StageInputOutputs = StageInputs[StageInputIdx]->GetOutputConnectors();

						if (StageInputOutputs.IsValidIndex(Channel.OutputIndex))
						{
							if (Channel.OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
							{
								InputType = StageInputOutputs[Channel.OutputIndex].Type;
								goto FoundInputType;
							}
							else
							{
								switch (InputType)
								{
									case EDMValueType::VT_None:
										InputType = EDMValueType::VT_Float1;
										break;

									case EDMValueType::VT_Float1:
										InputType = EDMValueType::VT_Float2;
										break;

									case EDMValueType::VT_Float2:
										InputType = EDMValueType::VT_Float3_XYZ;
										break;

									case EDMValueType::VT_Float3_RGB:
									case EDMValueType::VT_Float3_RPY:
									case EDMValueType::VT_Float3_XYZ:
										InputType = EDMValueType::VT_Float4_RGBA;
										break;

									// Not valid
									default:
									case EDMValueType::VT_Float4_RGBA:
										return false;
								}
							}
						}
					}

					break; // Switch
				}
			}
		}

	FoundInputType:

		// We don't have a primary type set, so return false.
		if (InputType == EDMValueType::VT_None || InputType == EDMValueType::VT_Float_Any)
		{
			return false;
		}

		if (bAllowSingleFloatMatch)
		{
			if (InputType == EDMValueType::VT_Float1 || InValueType == EDMValueType::VT_Float1)
			{
				return true;
			}
		}

		if (UDMValueDefinitionLibrary::GetValueDefinition(InputType).IsFloat3Type() && UDMValueDefinitionLibrary::GetValueDefinition(InValueType).IsFloat3Type())
		{
			return true;
		}

		return (InputType == InValueType);
	}

	return true;
}

void UDMMaterialStageExpressionMathBase::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!FDMUpdateGuard::CanUpdate())
	{
		return;
	}

	if (HasComponentBeenRemoved())
	{
		return;
	}

	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		check(InputConnectors.IsEmpty() == false);
		check(OutputConnectors.Num() == 1);

		UDMMaterialStage* Stage = GetStage();
		check(Stage);

		for (int32 InputIdx = 0; InputIdx < InputConnectors.Num() && InputIdx < VariadicInputCount; ++InputIdx)
		{
			InputConnectors[InputIdx].Type = EDMValueType::VT_Float_Any;
		}

		OutputConnectors[0].Type = EDMValueType::VT_Float_Any;

		TArray<FDMMaterialStageConnection>& InputConnections = Stage->GetInputConnectionMap();
		int32 MaxFloatCount = 0;

		// Calculate what the types of our input connectors are.
		for (int32 InputIdx = 0; InputIdx < InputConnections.Num() && InputIdx < VariadicInputCount; ++InputIdx)
		{
			// If the map has too many entries, truncate it and break
			if (!InputConnectors.IsValidIndex(InputIdx))
			{
				InputConnections.SetNum(InputIdx);
				break;
			}

			// Nothing is connected to this input node.
			if (InputConnections[InputIdx].Channels.IsEmpty())
			{
				continue;
			}

			int FloatCount = 0;
			EDMValueType ConnectorType = EDMValueType::VT_None;

			for (const FDMMaterialStageConnectorChannel& Channel : InputConnections[InputIdx].Channels)
			{
				EDMValueType ChannelConnectorType = Stage->GetSourceType(Channel);

				// Will fail unless it is float1-4.
				// Indeterminate or non-float types cannot be added together.
				int32 ChannelFloatCount = UDMValueDefinitionLibrary::GetValueDefinition(ChannelConnectorType).GetFloatCount();
				check(ChannelFloatCount > 0);

				FloatCount += ChannelFloatCount;
			}

			if (FloatCount > 0)
			{
				ConnectorType = UDMValueDefinitionLibrary::GetTypeForFloatCount(FloatCount).GetType();
			}

			InputConnectors[InputIdx].Type = ConnectorType;

			MaxFloatCount = FMath::Max(MaxFloatCount, FloatCount);
		}

		if (MaxFloatCount > 0)
		{
			OutputConnectors[0].Type = UDMValueDefinitionLibrary::GetTypeForFloatCount(MaxFloatCount).GetType();
		}
	}

	Super::Update(InSource, InUpdateType);
}

void UDMMaterialStageExpressionMathBase::AddDefaultInput(int32 InInputIndex) const
{
	if (InInputIndex < VariadicInputCount)
	{
		return;
	}

	Super::AddDefaultInput(InInputIndex);
}

bool UDMMaterialStageExpressionMathBase::GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
	UMaterialExpression*& OutMaterialExpression, int32& OutputIndex)
{
	GeneratePreviewMaterial(InPreviewMaterial);
	return true;
}

void UDMMaterialStageExpressionMathBase::SetupInputs(int32 InCount)
{
	check(InCount <= 15);

	VariadicInputCount = InCount;

	static const TArray<FText> ConnectorNames = {
		LOCTEXT("A", "A"),
		LOCTEXT("B", "B"),
		LOCTEXT("C", "C"),
		LOCTEXT("D", "D"),
		LOCTEXT("E", "E"),
		LOCTEXT("F", "F"),
		LOCTEXT("G", "G"),
		LOCTEXT("H", "H"),
		LOCTEXT("I", "I"),
		LOCTEXT("J", "J"),
		LOCTEXT("K", "K"),
		LOCTEXT("L", "L"),
		LOCTEXT("M", "M"),
		LOCTEXT("N", "N"),
		LOCTEXT("O", "O")
	};

	InputConnectors.Empty();

	for (int32 Index = 0; Index < InCount; ++Index)
	{
		InputConnectors.Add({Index, ConnectorNames[Index], EDMValueType::VT_Float_Any});
	}
}

void UDMMaterialStageExpressionMathBase::GeneratePreviewMaterial(UMaterial* InPreviewMaterial)
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	const TArray<FDMMaterialStageConnection>& InputConnectionMap = Stage->GetInputConnectionMap();
	const TArray<UDMMaterialStageInput*>& StageInputs = Stage->GetInputs();
	TArray<UE::DynamicMaterialEditor::Private::FDMInputInputs> Inputs; // Each channel can have multiple inputs
	bool bHasStageInput = false;

	for (int32 InputIdx = 0; InputIdx < InputConnectors.Num(); ++InputIdx)
	{
		TArray<UDMMaterialStageInput*> ChannelInputs;

		if (!InputConnectionMap.IsValidIndex(InputIdx))
		{
			continue;
		}

		ChannelInputs.SetNum(InputConnectionMap[InputIdx].Channels.Num());
		bool bNonStageInput = false;

		for (int32 ChannelIdx = 0; ChannelIdx < InputConnectionMap[InputIdx].Channels.Num(); ++ChannelIdx)
		{
			if (InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex == FDMMaterialStageConnectorChannel::PREVIOUS_STAGE)
			{
				bHasStageInput = true;
				ChannelInputs[ChannelIdx] = nullptr;
				continue;
			}

			if (InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
			{
				int32 StageInputIdx = InputConnectionMap[InputIdx].Channels[ChannelIdx].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
				ChannelInputs[ChannelIdx] = StageInputs[StageInputIdx];
				bNonStageInput = true;
				continue;
			}
		}

		if (ChannelInputs.IsEmpty())
		{
			continue;
		}

		if (bNonStageInput)
		{
			Inputs.Add({InputIdx, ChannelInputs});
		}
	}

	TSharedRef<FDMMaterialBuildState> BuildState = ModelEditorOnlyData->CreateBuildState(InPreviewMaterial);
	BuildState->SetPreviewObject(this);

	if (!bHasStageInput || Inputs.IsEmpty())
	{
		Stage->GenerateExpressions(BuildState);
		UMaterialExpression* StageExpression = BuildState->GetLastStageExpression(Stage);

		BuildState->GetBuildUtils().UpdatePreviewMaterial(StageExpression, 0, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 32);
	}
	else
	{
		UE::DynamicMaterialEditor::Private::BuildExpressionInputs(BuildState, InputConnectionMap, Inputs);
	}
}

#undef LOCTEXT_NAMESPACE
