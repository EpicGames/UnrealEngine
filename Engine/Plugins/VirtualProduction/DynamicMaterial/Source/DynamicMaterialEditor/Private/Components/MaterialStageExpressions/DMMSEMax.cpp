// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEMax.h"
#include "Materials/MaterialExpressionMax.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEMax)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionMax"

UDMMaterialStageExpressionMax::UDMMaterialStageExpressionMax()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Max", "Max"),
		UMaterialExpressionMax::StaticClass()
	)
{
	SetupInputs(2);

	bAllowSingleFloatMatch = false;
}

#undef LOCTEXT_NAMESPACE
