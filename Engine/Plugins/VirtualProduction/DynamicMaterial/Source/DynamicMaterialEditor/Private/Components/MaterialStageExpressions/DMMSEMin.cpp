// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEMin.h"
#include "Materials/MaterialExpressionMin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEMin)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionMin"

UDMMaterialStageExpressionMin::UDMMaterialStageExpressionMin()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Min", "Min"),
		UMaterialExpressionMin::StaticClass()
	)
{
	SetupInputs(2);

	bAllowSingleFloatMatch = false;
}

#undef LOCTEXT_NAMESPACE
