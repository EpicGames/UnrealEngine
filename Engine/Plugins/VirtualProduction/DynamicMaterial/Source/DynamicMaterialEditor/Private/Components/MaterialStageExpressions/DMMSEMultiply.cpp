// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEMultiply.h"
#include "Materials/MaterialExpressionMultiply.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEMultiply)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionMultiply"

UDMMaterialStageExpressionMultiply::UDMMaterialStageExpressionMultiply()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Multiply", "Multiply"),
		UMaterialExpressionMultiply::StaticClass()
	)
{
	SetupInputs(2);
}

#undef LOCTEXT_NAMESPACE
