// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEObjectBounds.h"
#include "Materials/MaterialExpressionObjectBounds.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEObjectBounds)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionObjectBounds"

UDMMaterialStageExpressionObjectBounds::UDMMaterialStageExpressionObjectBounds()
	: UDMMaterialStageExpression(
		LOCTEXT("ObjectLocalBounds", "Object Local Bounds"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionObjectBounds"))
	)
{
	Menus.Add(EDMExpressionMenu::Object);

	OutputConnectors.Add({0, LOCTEXT("Min", "Min"), EDMValueType::VT_Float3_XYZ});
	OutputConnectors.Add({1, LOCTEXT("Max", "Max"), EDMValueType::VT_Float3_XYZ});
	OutputConnectors.Add({1, LOCTEXT("Size", "Size"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
