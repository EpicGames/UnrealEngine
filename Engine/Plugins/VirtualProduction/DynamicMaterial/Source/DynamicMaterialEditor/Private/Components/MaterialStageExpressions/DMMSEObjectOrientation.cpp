// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEObjectOrientation.h"
#include "Materials/MaterialExpressionObjectOrientation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEObjectOrientation)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionObjectOrientation"

UDMMaterialStageExpressionObjectOrientation::UDMMaterialStageExpressionObjectOrientation()
	: UDMMaterialStageExpression(
		LOCTEXT("ObjectOrientation", "Object Orientation"),
		UMaterialExpressionObjectOrientation::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Object);

	OutputConnectors.Add({0, LOCTEXT("Orientation", "Orientation"), EDMValueType::VT_Float3_RPY});
}

#undef LOCTEXT_NAMESPACE
