// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEObjectPositionWS.h"
#include "Materials/MaterialExpressionObjectPositionWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEObjectPositionWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionObjectPositionWS"

UDMMaterialStageExpressionObjectPositionWS::UDMMaterialStageExpressionObjectPositionWS()
	: UDMMaterialStageExpression(
		LOCTEXT("ObjectPositionWS", "Object Position (WS)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionObjectPositionWS"))
	)
{
	Menus.Add(EDMExpressionMenu::Object);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
