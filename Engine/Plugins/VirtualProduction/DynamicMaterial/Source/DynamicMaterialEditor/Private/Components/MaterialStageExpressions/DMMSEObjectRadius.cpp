// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEObjectRadius.h"
#include "Materials/MaterialExpressionObjectRadius.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEObjectRadius)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionObjectRadius"

UDMMaterialStageExpressionObjectRadius::UDMMaterialStageExpressionObjectRadius()
	: UDMMaterialStageExpression(
		LOCTEXT("ObjectRadius", "Object Radius"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionObjectRadius"))
	)
{
	Menus.Add(EDMExpressionMenu::Object);

	OutputConnectors.Add({0, LOCTEXT("Radius", "Radius"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
