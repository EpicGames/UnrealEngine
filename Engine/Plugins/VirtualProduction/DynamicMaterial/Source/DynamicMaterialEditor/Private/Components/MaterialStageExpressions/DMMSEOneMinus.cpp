// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEOneMinus.h"
#include "Materials/MaterialExpressionOneMinus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEOneMinus)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionOneMinus"

UDMMaterialStageExpressionOneMinus::UDMMaterialStageExpressionOneMinus()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("OneMinus", "One Minus"),
		UMaterialExpressionOneMinus::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
