// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEPanner.h"
#include "Materials/MaterialExpressionPanner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEPanner)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionPanner"

UDMMaterialStageExpressionPanner::UDMMaterialStageExpressionPanner()
	: UDMMaterialStageExpression(
		LOCTEXT("UV Panner", "UV Panner"),
		UMaterialExpressionPanner::StaticClass()
	)
{
	bInputRequired = true;

	InputConnectors.Add({0, LOCTEXT("UV Index", "UV Index"), EDMValueType::VT_Float1});
	InputConnectors.Add({1, LOCTEXT("Time", "Time"), EDMValueType::VT_Float1});
	InputConnectors.Add({2, LOCTEXT("Speed", "Speed"), EDMValueType::VT_Float2});

	Menus.Add(EDMExpressionMenu::Texture);
	Menus.Add(EDMExpressionMenu::Time);

	OutputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
}

void UDMMaterialStageExpressionPanner::AddDefaultInput(int32 InInputIndex) const
{
	return;
}

#undef LOCTEXT_NAMESPACE
