// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSEPanner.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionPanner : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionPanner();

	//~ Begin UDMMaterialStageThroughput
	virtual void AddDefaultInput(int32 InInputIndex) const override;
	//~ End UDMMaterialStageThroughput
};
