// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleColor.h"
#include "Materials/MaterialExpressionParticleColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleColor"

UDMMaterialStageExpressionParticleColor::UDMMaterialStageExpressionParticleColor()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleColor", "Particle Color"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleColor"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);

	OutputConnectors.Add({0, LOCTEXT("Color", "Color"), EDMValueType::VT_Float4_RGBA});
}

#undef LOCTEXT_NAMESPACE
