// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleDirection.h"
#include "Materials/MaterialExpressionParticleDirection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleDirection)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleDirection"

UDMMaterialStageExpressionParticleDirection::UDMMaterialStageExpressionParticleDirection()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleDirection", "Particle Direction"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleDirection"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Direction", "Direction"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
