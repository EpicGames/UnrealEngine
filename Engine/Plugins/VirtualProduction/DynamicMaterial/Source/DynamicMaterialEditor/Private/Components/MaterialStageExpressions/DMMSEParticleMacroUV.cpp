// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleMacroUV.h"
#include "Materials/MaterialExpressionParticleMacroUV.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleMacroUV)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleMacroUV"

UDMMaterialStageExpressionParticleMacroUV::UDMMaterialStageExpressionParticleMacroUV()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleMacroUV", "Particle Macro UV"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleMacroUV"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);
	Menus.Add(EDMExpressionMenu::Texture);

	OutputConnectors.Add({0, LOCTEXT("Macro UV", "Macro UV"), EDMValueType::VT_Float2});
}

#undef LOCTEXT_NAMESPACE
