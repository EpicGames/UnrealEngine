// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleMotionBlurFade.h"
#include "Materials/MaterialExpressionParticleMotionBlurFade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleMotionBlurFade)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleMotionBlurFade"

UDMMaterialStageExpressionParticleMotionBlurFade::UDMMaterialStageExpressionParticleMotionBlurFade()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleMotionBlurFade", "Particle Motion Blur Fade"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleMotionBlurFade"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);

	OutputConnectors.Add({0, LOCTEXT("Motion Blur Fade", "Motion Blur Fade"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
