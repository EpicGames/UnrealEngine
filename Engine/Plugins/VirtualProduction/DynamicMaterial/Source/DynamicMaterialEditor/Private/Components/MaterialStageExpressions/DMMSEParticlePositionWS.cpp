// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticlePositionWS.h"
#include "Materials/MaterialExpressionParticlePositionWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticlePositionWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticlePositionWS"

UDMMaterialStageExpressionParticlePositionWS::UDMMaterialStageExpressionParticlePositionWS()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticlePositionWS", "Particle Position (WS)"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticlePositionWS"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
