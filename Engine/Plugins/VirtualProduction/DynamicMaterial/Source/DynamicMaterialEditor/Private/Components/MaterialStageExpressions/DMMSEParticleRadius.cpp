// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleRadius.h"
#include "Materials/MaterialExpressionParticleRadius.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleRadius)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleRadius"

UDMMaterialStageExpressionParticleRadius::UDMMaterialStageExpressionParticleRadius()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleRadius", "Particle Radius"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleRadius"))
	)
{
	Menus.Add(EDMExpressionMenu::Geometry);
	Menus.Add(EDMExpressionMenu::Particle);

	OutputConnectors.Add({0, LOCTEXT("Radius", "Radius"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
