// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleRandom.h"
#include "Materials/MaterialExpressionParticleRandom.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleRandom)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleRandom"

UDMMaterialStageExpressionParticleRandom::UDMMaterialStageExpressionParticleRandom()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleRandomFloat", "Particle Random Float"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleRandom"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);

	OutputConnectors.Add({0, LOCTEXT("Random Float", "Random Float"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
