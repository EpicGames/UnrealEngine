// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleRelativeTime.h"
#include "Materials/MaterialExpressionParticleRelativeTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleRelativeTime)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleRelativeTime"

UDMMaterialStageExpressionParticleRelativeTime::UDMMaterialStageExpressionParticleRelativeTime()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleRelativeTime", "Particle Relative Time"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleRelativeTime"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);
	Menus.Add(EDMExpressionMenu::Time);

	OutputConnectors.Add({0, LOCTEXT("RelativeTime", "Relative Time"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
