// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleSize.h"
#include "Materials/MaterialExpressionParticleSize.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleSize)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleSize"

UDMMaterialStageExpressionParticleSize::UDMMaterialStageExpressionParticleSize()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleSize", "Particle Size"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleSize"))
	)
{
	Menus.Add(EDMExpressionMenu::Geometry);
	Menus.Add(EDMExpressionMenu::Particle);

	OutputConnectors.Add({0, LOCTEXT("Size", "Size"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
