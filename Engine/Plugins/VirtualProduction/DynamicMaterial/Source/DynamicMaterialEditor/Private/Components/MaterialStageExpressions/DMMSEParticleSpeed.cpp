// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleSpeed.h"
#include "Materials/MaterialExpressionParticleSpeed.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleSpeed)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleSpeed"

UDMMaterialStageExpressionParticleSpeed::UDMMaterialStageExpressionParticleSpeed()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleSpeed", "Particle Speed"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionParticleSpeed"))
	)
{
	Menus.Add(EDMExpressionMenu::Particle);
	Menus.Add(EDMExpressionMenu::Object);

	OutputConnectors.Add({0, LOCTEXT("Speed", "Speed"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
