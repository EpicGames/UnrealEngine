// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleSubUV.h"
#include "Materials/MaterialExpressionParticleSubUV.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleSubUV)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleSubUV"

UDMMaterialStageExpressionParticleSubUV::UDMMaterialStageExpressionParticleSubUV()
	: UDMMaterialStageExpressionTextureSampleBase(
		LOCTEXT("ParticleSubUV", "Particle Sub UV"),
		UMaterialExpressionParticleSubUV::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Particle);
	Menus.Add(EDMExpressionMenu::Texture);
}

#undef LOCTEXT_NAMESPACE
