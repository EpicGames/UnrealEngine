// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageExpressions/DMMSETextureSampleBase.h"
#include "DMMSEParticleSubUV.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionParticleSubUV : public UDMMaterialStageExpressionTextureSampleBase
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionParticleSubUV();
};
