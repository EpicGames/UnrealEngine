// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEParticleSubUVProperties.h"
#include "Materials/MaterialExpressionParticleSubUVProperties.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEParticleSubUVProperties)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionParticleSubUVProperties"

UDMMaterialStageExpressionParticleSubUVProperties::UDMMaterialStageExpressionParticleSubUVProperties()
	: UDMMaterialStageExpression(
		LOCTEXT("ParticleSubUVProperties", "Particle Sub UV Properties"),
		UMaterialExpressionParticleSubUVProperties::StaticClass()
	)
{
	bInputRequired = true;

	Menus.Add(EDMExpressionMenu::Particle);

	OutputConnectors.Add({0, LOCTEXT("UV1", "UV 1"), EDMValueType::VT_Float2});
	OutputConnectors.Add({1, LOCTEXT("UV2", "UV 2"), EDMValueType::VT_Float2});
	OutputConnectors.Add({2, LOCTEXT("Blend", "Blend"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
