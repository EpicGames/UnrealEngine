// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEPixelDepth.h"
#include "Materials/MaterialExpressionPixelDepth.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEPixelDepth)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionPixelDepth"

UDMMaterialStageExpressionPixelDepth::UDMMaterialStageExpressionPixelDepth()
	: UDMMaterialStageExpression(
		LOCTEXT("PixelDepth", "PixelDepth"),
		UMaterialExpressionPixelDepth::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Other);

	OutputConnectors.Add({0, LOCTEXT("Depth", "Depth"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
