// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEPixelNormalWS.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEPixelNormalWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionPixelNormalWS"

UDMMaterialStageExpressionPixelNormalWS::UDMMaterialStageExpressionPixelNormalWS()
	: UDMMaterialStageExpression(
		LOCTEXT("PixelNormalWS", "Pixel Normal (WS)"),
		UMaterialExpressionPixelNormalWS::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::WorldSpace);
	Menus.Add(EDMExpressionMenu::Geometry);

	OutputConnectors.Add({0, LOCTEXT("Normal", "Normal"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
