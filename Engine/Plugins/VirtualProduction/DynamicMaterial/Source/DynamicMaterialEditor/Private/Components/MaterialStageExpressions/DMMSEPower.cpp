// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEPower.h"
#include "Materials/MaterialExpressionPower.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEPower)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionPower"

UDMMaterialStageExpressionPower::UDMMaterialStageExpressionPower()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Power", "Power"),
		UMaterialExpressionPower::StaticClass()
	)
{
	SetupInputs(2);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("Mantissa", "Mantissa");
	InputConnectors[1].Name = LOCTEXT("Exponent", "Exponent");
}

#undef LOCTEXT_NAMESPACE
