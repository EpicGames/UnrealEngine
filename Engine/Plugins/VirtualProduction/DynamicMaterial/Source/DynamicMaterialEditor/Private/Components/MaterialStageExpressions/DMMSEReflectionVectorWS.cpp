// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEReflectionVectorWS.h"
#include "Materials/MaterialExpressionReflectionVectorWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEReflectionVectorWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionReflectionVectorWS"

UDMMaterialStageExpressionReflectionVectorWS::UDMMaterialStageExpressionReflectionVectorWS()
	: UDMMaterialStageExpression(
		LOCTEXT("ReflectionVectorWS", "Reflection Vector (WS)"),
		UMaterialExpressionReflectionVectorWS::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Geometry);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Vector", "Vector"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
