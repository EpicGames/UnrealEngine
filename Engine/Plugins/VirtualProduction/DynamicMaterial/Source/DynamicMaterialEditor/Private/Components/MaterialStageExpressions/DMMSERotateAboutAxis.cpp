// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSERotateAboutAxis.h"
#include "Materials/MaterialExpressionRotateAboutAxis.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSERotateAboutAxis)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionRotateAboutAxis"

UDMMaterialStageExpressionRotateAboutAxis::UDMMaterialStageExpressionRotateAboutAxis()
	: UDMMaterialStageExpression(
		LOCTEXT("RotateAboutAxis", "Rotate About Axis"),
		UMaterialExpressionRotateAboutAxis::StaticClass()
	)
{
	bInputRequired = true;

	InputConnectors.Add({0, LOCTEXT("Axis", "Axis"), EDMValueType::VT_Float3_XYZ});
	InputConnectors.Add({1, LOCTEXT("Angle", "Angle"), EDMValueType::VT_Float1});
	InputConnectors.Add({2, LOCTEXT("Pivot", "Pivot"), EDMValueType::VT_Float3_XYZ});
	InputConnectors.Add({3, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionRotateAboutAxis, Period));
	Period = 1.f;

	OutputConnectors.Add({0, LOCTEXT("RotatedPosition", "Rotated Position"), EDMValueType::VT_Float3_XYZ});
}

void UDMMaterialStageExpressionRotateAboutAxis::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionRotateAboutAxis* RotateExpression = Cast<UMaterialExpressionRotateAboutAxis>(InExpressions[0]);
	RotateExpression->Period = Period;
}

#undef LOCTEXT_NAMESPACE
