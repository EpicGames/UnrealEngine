// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSERound.h"
#include "Materials/MaterialExpressionRound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSERound)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionRound"

UDMMaterialStageExpressionRound::UDMMaterialStageExpressionRound()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Round", "Round"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionRound"))
	)
{
	SetupInputs(1);

	OutputConnectors[0].Name = LOCTEXT("Whole Number", "Whole Number");
}

#undef LOCTEXT_NAMESPACE
