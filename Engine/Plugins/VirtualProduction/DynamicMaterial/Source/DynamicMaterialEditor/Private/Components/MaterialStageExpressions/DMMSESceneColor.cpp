// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESceneColor.h"
#include "Materials/MaterialExpressionSceneColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESceneColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSceneColor"

UDMMaterialStageExpressionSceneColor::UDMMaterialStageExpressionSceneColor()
	: UDMMaterialStageExpression(
		LOCTEXT("SceneColor", "Scene Color"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionSceneColor"))
	)
{
	InputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionSceneColor, InputMode));
	InputMode = EMaterialSceneAttributeInputMode::Coordinates;

	Menus.Add(EDMExpressionMenu::Other);

	OutputConnectors.Add({0, LOCTEXT("Color", "Color"), EDMValueType::VT_Float3_RGB});
}

void UDMMaterialStageExpressionSceneColor::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionSceneColor* SceneColorExpression = static_cast<UMaterialExpressionSceneColor*>(InExpressions[0]); // Type is not exportde.
	SceneColorExpression->InputMode = InputMode;
}

#undef LOCTEXT_NAMESPACE
