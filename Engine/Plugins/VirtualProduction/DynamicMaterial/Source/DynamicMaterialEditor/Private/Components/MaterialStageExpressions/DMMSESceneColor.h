// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "Materials/MaterialExpressionSceneColor.h"
#include "DMMSESceneColor.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionSceneColor : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionSceneColor();

	//~ Begin UDMMaterialStageSource
	virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialStageThroughput
	virtual bool SupportsLayerMaskTextureUVLink() const override { return true; }
	virtual int32 GetLayerMaskTextureUVLinkInputIndex() const override { return 0; }
	//~ End UDMMaterialStageThroughput

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TEnumAsByte<enum EMaterialSceneAttributeInputMode::Type> InputMode;
};
