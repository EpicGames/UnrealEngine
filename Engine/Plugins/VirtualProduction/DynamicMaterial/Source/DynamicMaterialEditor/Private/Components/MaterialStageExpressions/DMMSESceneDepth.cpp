// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESceneDepth.h"
#include "Materials/MaterialExpressionSceneDepth.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESceneDepth)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSceneDepth"

UDMMaterialStageExpressionSceneDepth::UDMMaterialStageExpressionSceneDepth()
	: UDMMaterialStageExpression(
		LOCTEXT("SceneDepth", "Scene Depth"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionSceneDepth"))
	)
{
	InputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});

	Menus.Add(EDMExpressionMenu::Geometry);
	Menus.Add(EDMExpressionMenu::Other);

	OutputConnectors.Add({0, LOCTEXT("Depth", "Depth"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
