// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESceneTexture.h"
#include "Materials/MaterialExpressionSceneTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESceneTexture)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSceneTexture"

UDMMaterialStageExpressionSceneTexture::UDMMaterialStageExpressionSceneTexture()
	: UDMMaterialStageExpression(
		LOCTEXT("SceneTexture", "Post Process"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionSceneTexture"))
	)
{
	InputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionSceneTexture, SceneTextureId));
	SceneTextureId = ESceneTextureId::PPI_PostProcessInput0;

	Menus.Add(EDMExpressionMenu::Texture);

	OutputConnectors.Add({0, LOCTEXT("Scene", "Scene"), EDMValueType::VT_Float4_RGBA});
}

void UDMMaterialStageExpressionSceneTexture::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionSceneTexture* SceneTextureExpression = Cast<UMaterialExpressionSceneTexture>(InExpressions[0]);
	SceneTextureExpression->SceneTextureId = SceneTextureId;
}

#undef LOCTEXT_NAMESPACE
