// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "Materials/MaterialExpressionSceneTexture.h"
#include "DMMSESceneTexture.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionSceneTexture : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionSceneTexture();

	//~ Begin UDMMaterialStageSource
	virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialStageThroughput
	virtual bool SupportsLayerMaskTextureUVLink() const override { return true; }
	virtual int32 GetLayerMaskTextureUVLinkInputIndex() const override { return 0; }
	//~ End UDMMaterialStageThroughput

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TEnumAsByte<ESceneTextureId> SceneTextureId;
};
