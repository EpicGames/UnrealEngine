// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEScreenPosition.h"
#include "Materials/MaterialExpressionScreenPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEScreenPosition)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionScreenPosition"

UDMMaterialStageExpressionScreenPosition::UDMMaterialStageExpressionScreenPosition()
	: UDMMaterialStageExpression(
		LOCTEXT("ScreenPositionUV", "Screen Position UV"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionScreenPosition"))
	)
{
	Menus.Add(EDMExpressionMenu::Texture);

	OutputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
	OutputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float2});
}

#undef LOCTEXT_NAMESPACE
