// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSEScreenPosition.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionScreenPosition : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionScreenPosition();

	//~ Begin UDMMaterialStageThroughput
	virtual bool SupportsLayerMaskTextureUVLink() const override { return true; }
	virtual int32 GetLayerMaskTextureUVLinkInputIndex() const override { return 0; }
	//~ End UDMMaterialStageThroughput
};
