// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESign.h"
#include "Materials/MaterialExpressionSign.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESign)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSign"

UDMMaterialStageExpressionSign::UDMMaterialStageExpressionSign()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Sign", "Sign"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionSign"))
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;
}

#undef LOCTEXT_NAMESPACE
