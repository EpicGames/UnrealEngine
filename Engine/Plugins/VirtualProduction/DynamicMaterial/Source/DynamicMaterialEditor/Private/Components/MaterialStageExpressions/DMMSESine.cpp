// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESine.h"
#include "Materials/MaterialExpressionSine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESine)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSine"

UDMMaterialStageExpressionSine::UDMMaterialStageExpressionSine()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Sine", "Sine"),
		UMaterialExpressionSine::StaticClass()
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("Angle", "Angle");
	OutputConnectors[0].Name = LOCTEXT("O/H", "O/H");
}

#undef LOCTEXT_NAMESPACE
