// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESmoothStep.h"
#include "Materials/MaterialExpressionSmoothStep.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESmoothStep)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSmoothStep"

UDMMaterialStageExpressionSmoothStep::UDMMaterialStageExpressionSmoothStep()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("SmoothStep", "Smooth Step"),
		UMaterialExpressionSmoothStep::StaticClass()
	)
{
	SetupInputs(3);

	bAllowSingleFloatMatch = false;

	InputConnectors[0].Name = LOCTEXT("Min", "Min");
	InputConnectors[1].Name = LOCTEXT("Max", "Max");
	InputConnectors[2].Name = LOCTEXT("Value", "Value");
}

#undef LOCTEXT_NAMESPACE
