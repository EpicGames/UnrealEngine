// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESquareRoot.h"
#include "Materials/MaterialExpressionSquareRoot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESquareRoot)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSquareRoot"

UDMMaterialStageExpressionSquareRoot::UDMMaterialStageExpressionSquareRoot()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("SquareRoot", "Square Root"),
		UMaterialExpressionSquareRoot::StaticClass()
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
