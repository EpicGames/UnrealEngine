// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEStep.h"
#include "Materials/MaterialExpressionStep.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEStep)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionStep"

UDMMaterialStageExpressionStep::UDMMaterialStageExpressionStep()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Step", "Step"),
		UMaterialExpressionStep::StaticClass()
	)
{
	SetupInputs(2);

	bAllowSingleFloatMatch = false;

	InputConnectors[0].Name = LOCTEXT("Y", "Y");
	InputConnectors[1].Name = LOCTEXT("X", "X");
}

#undef LOCTEXT_NAMESPACE
