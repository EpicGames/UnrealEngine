// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSESubtract.h"
#include "Materials/MaterialExpressionSubtract.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSESubtract)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionSubtract"

UDMMaterialStageExpressionSubtract::UDMMaterialStageExpressionSubtract()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Subtract", "Subtract"),
		UMaterialExpressionSubtract::StaticClass()
	)
{
	SetupInputs(2);
}

#undef LOCTEXT_NAMESPACE
