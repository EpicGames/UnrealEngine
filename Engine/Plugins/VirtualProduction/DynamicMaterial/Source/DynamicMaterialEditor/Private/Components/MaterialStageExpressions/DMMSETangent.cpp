// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETangent.h"
#include "Materials/MaterialExpressionTangent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETangent)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTangent"

UDMMaterialStageExpressionTangent::UDMMaterialStageExpressionTangent()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Tangent", "Tangent"),
		UMaterialExpressionTangent::StaticClass()
	)
{
	SetupInputs(1);

	bSingleChannelOnly = true;

	InputConnectors[0].Name = LOCTEXT("Angle", "Angle");
	OutputConnectors[0].Name = LOCTEXT("A/O", "A/O");
}

#undef LOCTEXT_NAMESPACE
