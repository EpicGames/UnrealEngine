// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETextureCoordinate.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETextureCoordinate)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTextureCoordinate"

UDMMaterialStageExpressionTextureCoordinate::UDMMaterialStageExpressionTextureCoordinate()
	: UDMMaterialStageExpression(
		LOCTEXT("TextureCoordinates", "Texture Coordinates"),
		UMaterialExpressionTextureCoordinate::StaticClass()
	)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureCoordinate, CoordinateIndex));
	CoordinateIndex = 0;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureCoordinate, UTiling));
	UTiling = 1;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureCoordinate, VTiling));
	VTiling = 1;

	Menus.Add(EDMExpressionMenu::Texture);

	OutputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
}

void UDMMaterialStageExpressionTextureCoordinate::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionTextureCoordinate* TextureCoordinateExpression = Cast<UMaterialExpressionTextureCoordinate>(InExpressions[0]);
	TextureCoordinateExpression->CoordinateIndex = CoordinateIndex;
	TextureCoordinateExpression->UTiling = UTiling;
	TextureCoordinateExpression->VTiling = VTiling;
}

#undef LOCTEXT_NAMESPACE
