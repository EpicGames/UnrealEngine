// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETextureSample.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Model/DMMaterialBuildState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETextureSample)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTextureSample"

UDMMaterialStageExpressionTextureSample::UDMMaterialStageExpressionTextureSample()
	: UDMMaterialStageExpressionTextureSampleBase(
		LOCTEXT("Texture", "Texture"),
		UMaterialExpressionTextureSample::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Texture);
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSample, bUseBaseTexture));
}

void UDMMaterialStageExpressionTextureSample::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSample, bUseBaseTexture))
	{
		OnUseBaseTextureChanged();
	}
}

bool UDMMaterialStageExpressionTextureSample::IsPropertyVisible(FName InProperty) const
{
	const bool bIsUseBaseTexture = (InProperty == GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSample, bUseBaseTexture));
	const bool bIsClampTexture = (InProperty == GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSample, bClampTexture));

	if (bIsUseBaseTexture || bIsClampTexture)
	{
		const bool bCanUseBaseTexture = CanUseBaseTexture();

		if (bIsUseBaseTexture)
		{
			return bCanUseBaseTexture;
		}

		if (bIsClampTexture)
		{
			return !bUseBaseTexture || !bCanUseBaseTexture;
		}
	}

	return Super::IsPropertyVisible(InProperty);
}

bool UDMMaterialStageExpressionTextureSample::GetUseBaseTexture() const
{
	return bUseBaseTexture;
}

void UDMMaterialStageExpressionTextureSample::SetUseBaseTexture(bool bInUseBaseTexture)
{
	if (bUseBaseTexture == bInUseBaseTexture)
	{
		return;
	}

	bUseBaseTexture = bInUseBaseTexture;

	OnUseBaseTextureChanged();
}

bool UDMMaterialStageExpressionTextureSample::CanUseBaseTexture() const
{
	return !!GetBaseTextureSample();
}

const UDMMaterialStageExpressionTextureSample* UDMMaterialStageExpressionTextureSample::GetBaseTextureSample() const
{
	UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(GetStage());

	if (!SubStage)
	{
		return nullptr;
	}

	UDMMaterialStage* Stage = SubStage->GetParentStage();

	if (!Stage || Stage->IsA<UDMMaterialSubStage>())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();

	if (!Layer || !Layer->IsTextureUVLinkEnabled())
	{
		return nullptr;
	}

	UDMMaterialStageThroughputLayerBlend* LayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(Stage->GetSource());

	if (!LayerBlend)
	{
		return nullptr;
	}

	UDMMaterialStage* BaseStage = Layer->GetFirstEnabledStage(EDMMaterialLayerStage::Base);

	if (!BaseStage)
	{
		return nullptr;
	}

	UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(BaseStage->GetSource());

	if (!Blend)
	{
		return nullptr;
	}

	UDMMaterialStageInputThroughput* InputThroughput = Cast<UDMMaterialStageInputThroughput>(Blend->GetInputB());

	if (!InputThroughput)
	{
		return nullptr;
	}

	UDMMaterialSubStage* BaseSubStage = InputThroughput->GetSubStage();

	if (!BaseSubStage)
	{
		return nullptr;
	}

	return Cast<UDMMaterialStageExpressionTextureSample>(BaseSubStage->GetSource());
}

void UDMMaterialStageExpressionTextureSample::OnUseBaseTextureChanged()
{
	Update(this, EDMUpdateType::Structure | EDMUpdateType::AllowParentUpdate);
}

void UDMMaterialStageExpressionTextureSample::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	if (bUseBaseTexture)
	{
		if (const UDMMaterialStageExpressionTextureSample* BaseTextureSample = GetBaseTextureSample())
		{
			BaseTextureSample->GenerateExpressions(InBuildState);
			InBuildState->AddStageSourceExpressions(this, InBuildState->GetStageSourceExpressions(BaseTextureSample));
			return;
		}
	}

	Super::GenerateExpressions(InBuildState);
}

#undef LOCTEXT_NAMESPACE
