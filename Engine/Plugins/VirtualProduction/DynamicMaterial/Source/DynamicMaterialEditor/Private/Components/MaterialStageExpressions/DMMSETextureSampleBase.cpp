// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETextureSampleBase.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "CoreGlobals.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETextureSampleBase)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTextureSample"

UDMMaterialStageExpressionTextureSampleBase::UDMMaterialStageExpressionTextureSampleBase()
	: UDMMaterialStageExpression(
		LOCTEXT("UDMMaterialStageExpressionTextureSampleBase", "UDMMaterialStageExpressionTextureSampleBase"),
		UMaterialExpression::StaticClass()
	)
{
}

UDMMaterialStageExpressionTextureSampleBase::UDMMaterialStageExpressionTextureSampleBase(const FText& InName, TSubclassOf<UMaterialExpression> InClass)
	: UDMMaterialStageExpression(InName, InClass)
{
	bInputRequired = true;
	bAllowNestedInputs = true;

	InputConnectors.Add({1, LOCTEXT("Texture", "Texture"), EDMValueType::VT_Texture});
	InputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});

	bClampTexture = false;

	OutputConnectors.Add({0, LOCTEXT("ColorRGB", "Color (RGB)"), EDMValueType::VT_Float3_RGB});
	OutputConnectors.Add({4, LOCTEXT("Alpha", "Alpha"), EDMValueType::VT_Float1});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSampleBase, bClampTexture));
}

void UDMMaterialStageExpressionTextureSampleBase::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	check(MaterialExpressionClass.Get());

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	UMaterialExpression* NewExpression = InBuildState->GetBuildUtils().CreateExpression(MaterialExpressionClass.Get(), UE_DM_NodeComment_Default);
	AddExpressionProperties({NewExpression});

	InBuildState->AddStageSourceExpressions(this, {NewExpression});

	UMaterialExpressionTextureSample* TextureSample = Cast<UMaterialExpressionTextureSample>(NewExpression);

	if (!TextureSample)
	{
		return;
	}

	const UDMMaterialProperty* CurrentProperty = InBuildState->GetCurrentMaterialProperty();

	if (!CurrentProperty)
	{
		return;
	}

	TextureSample->SamplerType = CurrentProperty->GetTextureSamplerType();
}

void UDMMaterialStageExpressionTextureSampleBase::OnComponentAdded()
{
	Super::OnComponentAdded();

	UpdateMask();
}

bool UDMMaterialStageExpressionTextureSampleBase::IsPropertyVisible(FName Property) const
{
	if (Property == GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSampleBase, bClampTexture))
	{
		UDMMaterialStage* Stage = GetStage();
		check(Stage);

		UDMMaterialStage* ParentMostStage = Stage;

		if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(Stage))
		{
			ParentMostStage = SubStage->GetParentMostStage();
		}

		UDMMaterialLayerObject* Layer = ParentMostStage->GetLayer();
		check(Layer);

		if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true))
		{
			if (UDMMaterialStageThroughput* BaseThroughput = Cast<UDMMaterialStageThroughput>(BaseStage->GetSource()))
			{
				if (Layer->IsTextureUVLinkEnabled()
					&& Layer->GetStageType(ParentMostStage) == EDMMaterialLayerStage::Mask
					&& BaseThroughput->SupportsLayerMaskTextureUVLink()
					&& SupportsLayerMaskTextureUVLink())
				{
					if (UDMMaterialStageExpressionTextureSampleBase* BaseTextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(BaseStage->GetSource()))
					{
						return false;
					}

					if (UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(BaseStage->GetSource()))
					{
						if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(Blend->GetInputB()))
						{
							if (UDMMaterialStageExpressionTextureSampleBase* InputBaseTextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(InputExpression->GetMaterialStageExpression()))
							{
								return false;
							}
						}
					}
				}
			}
		}
	}

	return Super::IsPropertyVisible(Property);
}

void UDMMaterialStageExpressionTextureSampleBase::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialStage* ParentMostStage = Stage;

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(Stage))
	{
		ParentMostStage = SubStage->GetParentMostStage();
	}

	UDMMaterialLayerObject* Layer = ParentMostStage->GetLayer();
	check(Layer);

	bool bClampTextureActual = IsClampTextureEnabled();

	if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base, /* Enabled Only */ true))
	{
		if (Layer->IsTextureUVLinkEnabled() 
			&& Layer->GetStageType(ParentMostStage) == EDMMaterialLayerStage::Mask 
			&& SupportsLayerMaskTextureUVLink())
		{
			if (UDMMaterialStageExpressionTextureSampleBase* BaseTextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(BaseStage->GetSource()))
			{
				bClampTextureActual = BaseTextureSample->IsClampTextureEnabled();
			}
			else if (UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(BaseStage->GetSource()))
			{
				if (UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(Blend->GetInputB()))
				{
					if (UDMMaterialStageExpressionTextureSampleBase* InputBaseTextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(InputExpression->GetMaterialStageExpression()))
					{
						bClampTextureActual = InputBaseTextureSample->IsClampTextureEnabled();
					}

				}
			}
		}
	}

	UMaterialExpressionTextureSample* TextureSamplerExpression = Cast<UMaterialExpressionTextureSample>(InExpressions[0]);
	TextureSamplerExpression->SamplerSource = bClampTextureActual ? ESamplerSourceMode::SSM_Clamp_WorldGroupSettings : ESamplerSourceMode::SSM_FromTextureAsset;
}

int32 UDMMaterialStageExpressionTextureSampleBase::GetInnateMaskOutput(int32 InOutputIndex, int32 InOutputChannels) const
{
	if (InOutputIndex == 0)
	{
		switch (InOutputChannels)
		{
			case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
				return 1;

			case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
				return 2;

			case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
				return 3;

			case FDMMaterialStageConnectorChannel::FOURTH_CHANNEL:
				return 4;
		}
	}

	return Super::GetInnateMaskOutput(InOutputIndex, InOutputChannels);
}

int32 UDMMaterialStageExpressionTextureSampleBase::GetOutputChannelOverride(int32 InOutputIndex) const
{
	if (InOutputIndex == 1) // Alpha
	{
		return FDMMaterialStageConnectorChannel::FOURTH_CHANNEL;
	}

	return Super::GetOutputChannelOverride(InOutputIndex);
}

bool UDMMaterialStageExpressionTextureSampleBase::CanChangeInputType(int32 InInputIndex) const
{
	// Can't change UV input type
	if (InInputIndex == 1)
	{
		return false;
	}

	return Super::CanChangeInputType(InInputIndex);
}

void UDMMaterialStageExpressionTextureSampleBase::OnInputUpdated(int32 InInputIndex, EDMUpdateType InUpdateType)
{
	// If the texture changes, update the mask!
	if (InInputIndex == 0)
	{
		UpdateMask();
	}
}

FText UDMMaterialStageExpressionTextureSampleBase::GetComponentDescription() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<UDMMaterialStageInput*>& Inputs = Stage->GetInputs();

		for (UDMMaterialStageInput* StageInput : Inputs)
		{
			if (UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(StageInput))
			{
				if (UDMMaterialValue* Value = Cast<UDMMaterialValueTexture>(InputValue->GetValue()))
				{
					return Value->GetComponentDescription();
				}
			}
		}
	}

	return Super::GetComponentDescription();
}

FSlateIcon UDMMaterialStageExpressionTextureSampleBase::GetComponentIcon() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		const TArray<UDMMaterialStageInput*>& Inputs = Stage->GetInputs();

		for (UDMMaterialStageInput* StageInput : Inputs)
		{
			if (UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(StageInput))
			{
				if (UDMMaterialValue* Value = Cast<UDMMaterialValueTexture>(InputValue->GetValue()))
				{
					return Value->GetComponentIcon();
				}
			}
		}
	}

	return Super::GetComponentIcon();
}

void UDMMaterialStageExpressionTextureSampleBase::SetClampTextureEnabled(bool bInValue)
{
	if (bClampTexture == bInValue)
	{
		return;
	}

	bClampTexture = bInValue;

	Update(this, EDMUpdateType::Value);
}

void UDMMaterialStageExpressionTextureSampleBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName ClampTextureName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSampleBase, bClampTexture);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == ClampTextureName)
	{
		Update(this, EDMUpdateType::Structure);
	}
}

void UDMMaterialStageExpressionTextureSampleBase::UpdateMask()
{
	UDMMaterialStage* BaseTextureSampleStage = GetStage();
	check(BaseTextureSampleStage);

	UDMMaterialStage* ParentMostStage = BaseTextureSampleStage;

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(BaseTextureSampleStage))
	{
		ParentMostStage = SubStage->GetParentMostStage();
	}

	UDMMaterialLayerObject* Layer = ParentMostStage->GetLayer();
	check(Layer);

	const UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(EditorOnlyData);

	UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base);
	UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask, /* Enabled Only */ true);

	if (!MaskStage)
	{
		return;
	}

	check(BaseStage);

	if (!Layer->IsTextureUVLinkEnabled() || BaseStage != ParentMostStage)
	{
		return;
	}

	const TArray<FDMMaterialStageConnection>& BaseStageInputConnections = BaseTextureSampleStage->GetInputConnectionMap();

	if (!BaseStageInputConnections.IsValidIndex(0) || BaseStageInputConnections[0].Channels.Num() != 1
		|| BaseStageInputConnections[0].Channels[0].SourceIndex < FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
	{
		return;
	}

	const int32 BaseStageInputIdx = BaseStageInputConnections[0].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
	const TArray<UDMMaterialStageInput*>& BaseStageInputs = BaseTextureSampleStage->GetInputs();
	check(BaseStageInputs.IsValidIndex(BaseStageInputIdx));

	UDMMaterialStageInputValue* BaseInputValue = Cast<UDMMaterialStageInputValue>(BaseStageInputs[BaseStageInputIdx]);

	if (!BaseInputValue || !BaseInputValue->GetValue())
	{
		return;
	}

	UTexture* BaseTexture = nullptr;
	UDMMaterialValueTexture* BaseTextureValue = Cast<UDMMaterialValueTexture>(BaseInputValue->GetValue());

	if (BaseTextureValue)
	{
		if (BaseTextureValue->HasAlpha() == false)
		{
			return;
		}

		BaseTexture = BaseTextureValue->GetValue();
	}

	if (!BaseTexture)
	{
		return;
	}

	UDMMaterialStageThroughputLayerBlend* LayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(MaskStage->GetSource());
	check(LayerBlend); // Must be a layer blend

	const TArray<FDMMaterialStageConnection>& MaskStageInputConnections = MaskStage->GetInputConnectionMap();

	if (MaskStageInputConnections.IsValidIndex(2) && MaskStageInputConnections[2].Channels.Num() == 1
		&& MaskStageInputConnections[2].Channels[0].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
	{
		const int32 MaskStageInputIdx = MaskStageInputConnections[2].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
		const TArray<UDMMaterialStageInput*>& MaskStageInputs = MaskStage->GetInputs();
		check(MaskStageInputs.IsValidIndex(MaskStageInputIdx));

		if (UDMMaterialStageInputExpression* LayerBlendInputExpression = Cast<UDMMaterialStageInputExpression>(MaskStageInputs[MaskStageInputIdx]))
		{
			if (UDMMaterialStageExpressionTextureSampleBase* LayerBlendTextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(LayerBlendInputExpression->GetMaterialStageExpression()))
			{
				UDMMaterialStage* LayerBlendTextureSampleSubStage = LayerBlendInputExpression->GetSubStage();
				const TArray<FDMMaterialStageConnection>& LayerBlendTextureSampleSubStageInputConnections = LayerBlendTextureSampleSubStage->GetInputConnectionMap();

				if (LayerBlendTextureSampleSubStageInputConnections.IsValidIndex(0) && LayerBlendTextureSampleSubStageInputConnections[0].Channels.Num() == 1
					&& LayerBlendTextureSampleSubStageInputConnections[0].Channels[0].SourceIndex >= FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT)
				{
					const int32 LayerBlendTextureSampleSubStageInputIdx = LayerBlendTextureSampleSubStageInputConnections[0].Channels[0].SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

					const TArray<UDMMaterialStageInput*>& LayerBlendTextureSampleSubStageInputs = LayerBlendTextureSampleSubStage->GetInputs();
					check(LayerBlendTextureSampleSubStageInputs.IsValidIndex(LayerBlendTextureSampleSubStageInputIdx));

					if (UDMMaterialStageInputValue* LayerBlendTextureInputValue = Cast<UDMMaterialStageInputValue>(LayerBlendTextureSampleSubStageInputs[LayerBlendTextureSampleSubStageInputIdx]))
					{
						if (UDMMaterialValueTexture* LayerBlendTextureValue = Cast<UDMMaterialValueTexture>(LayerBlendTextureInputValue->GetValue()))
						{
							if (LayerBlendTextureValue->GetClass() == BaseTextureValue->GetClass())
							{
								if (GUndo)
								{
									LayerBlendTextureValue->Modify();
								}

								LayerBlendTextureValue->SetValue(BaseTexture);
							}
							else
							{
								UDMMaterialValueTexture* NewLayerBlendTextureValue = UDMMaterialValueTexture::CreateMaterialValueTexture(EditorOnlyData, BaseTextureValue->GetValue());
								check(NewLayerBlendTextureValue);

								LayerBlendTextureInputValue->SetValue(NewLayerBlendTextureValue);

								if (GUndo)
								{
									MaskStage->Modify();
								}
							}

							LayerBlend->SetMaskChannelOverride(EAvaColorChannel::Alpha);

							return;
						}
					}
				}
			}
		}
	}

	// Couldn't find a texture to update, so create a new one.

	// 2nd input, 2nd output (Alpha)
	if (GUndo)
	{
		MaskStage->Modify();
	}

	UDMMaterialStageInputExpression::ChangeStageInput_Expression(MaskStage, 
		UDMMaterialStageExpressionTextureSample::StaticClass(), 2, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		1, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);

	UDMMaterialStageInputThroughput* InputThroughput = Cast<UDMMaterialStageInputThroughput>(MaskStage->GetInputs().Last());
	UDMMaterialStageExpressionTextureSampleBase* MaskTextureSample = nullptr;

	if (InputThroughput)
	{
		MaskTextureSample = Cast<UDMMaterialStageExpressionTextureSampleBase>(InputThroughput->GetMaterialStageThroughput());
	}	

	if (!ensureMsgf(MaskTextureSample, TEXT("Failed, after trying really hard, to get a valid Texture Sample.")))
	{
		return;
	}

	UDMMaterialStage* MaskTextureSampleStage = MaskTextureSample->GetStage();
	check(MaskTextureSampleStage);

	UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		MaskTextureSampleStage, 
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		UDMMaterialValueTexture::StaticClass(),
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	UDMMaterialStageInputValue* NewInputValue = Cast<UDMMaterialStageInputValue>(MaskTextureSampleStage->GetInputs().Last());
	check(NewInputValue);

	UDMMaterialValueTexture* NewInputTexture = Cast<UDMMaterialValueTexture>(NewInputValue->GetValue());
	check(NewInputTexture);

	NewInputTexture->SetValue(BaseTexture);

	LayerBlend->SetMaskChannelOverride(EAvaColorChannel::Alpha);
}

#undef LOCTEXT_NAMESPACE
