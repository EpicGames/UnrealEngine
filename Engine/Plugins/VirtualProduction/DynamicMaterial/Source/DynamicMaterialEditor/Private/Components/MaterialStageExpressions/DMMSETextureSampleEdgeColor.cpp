// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETextureSampleEdgeColor.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETextureSampleEdgeColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTextureSample"

UDMMaterialStageExpressionTextureSampleEdgeColor::UDMMaterialStageExpressionTextureSampleEdgeColor()
	: UDMMaterialStageExpression(
		LOCTEXT("EdgeColor", "Edge Color"),
		UMaterialExpressionTextureSample::StaticClass()
	)
{
	bInputRequired = true;
	bAllowNestedInputs = true;

	InputConnectors.Add({1, LOCTEXT("Texture", "Texture"), EDMValueType::VT_Texture});
	InputConnectors.Add({0, LOCTEXT("PixelUV", "Pixel UV"), EDMValueType::VT_Float2});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSampleEdgeColor, EdgeLocation));
	EdgeLocation = EDMEdgeLocation::TopLeft;
	
	OutputConnectors.Add({0, LOCTEXT("ColorRGB", "Color (RGB)"), EDMValueType::VT_Float3_RGB});
}

bool UDMMaterialStageExpressionTextureSampleEdgeColor::CanChangeInputType(int32 InInputIndex) const
{
	return false;
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::AddDefaultInput(int32 InInputIndex) const
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	if (InInputIndex == 1)
	{
		UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(Stage, 
			1, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			EDMValueType::VT_Float2, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);
		check(InputValue);

		UDMMaterialValueFloat2* InputFloat2 = Cast<UDMMaterialValueFloat2>(InputValue->GetValue());
		check(InputFloat2);

		InputFloat2->SetDefaultValue(FVector2D::ZeroVector);
		InputFloat2->ApplyDefaultValue();
		return;
	}

	return Super::AddDefaultInput(InInputIndex);
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	Super::AddExpressionProperties(InExpressions);

	if (InExpressions.IsEmpty())
	{
		return;
	}

	UMaterialExpressionTextureSample* TextureSample = Cast<UMaterialExpressionTextureSample>(InExpressions[0]);

	if (!TextureSample)
	{
		return;
	}

	TextureSample->SamplerSource = ESamplerSourceMode::SSM_Clamp_WorldGroupSettings;
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
	
	if (!Stage->GetInputs().IsValidIndex(1))
	{
		Super::Update(InSource, InUpdateType);
		return;
	}

	UDMMaterialStageInputValue* InputValue = CastChecked<UDMMaterialStageInputValue>(Stage->GetInputs()[1]);
	UDMMaterialValueFloat2* Float2 = CastChecked<UDMMaterialValueFloat2>(InputValue->GetValue());

	const FVector2D& Float2Value = Float2->GetValue();

	const bool bLeftX = FMath::IsNearlyEqual(Float2Value.X, 0.f);
	const bool bCenterX = FMath::IsNearlyEqual(Float2Value.X, 0.5f);
	const bool bRightX = FMath::IsNearlyEqual(Float2Value.X, 1.f);

	const bool bTopY = FMath::IsNearlyEqual(Float2Value.Y, 0.f);
	const bool bCenterY = FMath::IsNearlyEqual(Float2Value.Y, 0.5f);
	const bool bBottomY = FMath::IsNearlyEqual(Float2Value.Y, 1.f);

	const bool bIsValidX = bLeftX || bCenterX || bRightX;
	const bool bIsValidY = bTopY || bCenterY || bBottomY;

	if (!bIsValidX || !bIsValidY)
	{
		EdgeLocation = EDMEdgeLocation::Custom;
		Super::Update(InSource, InUpdateType | EDMUpdateType::Value);
		return;
	}

	bool bChangedEdgeLocation = false;

	if (bTopY && bLeftX)
	{
		if (EdgeLocation != EDMEdgeLocation::TopLeft)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::TopLeft;
		}
	}
	else if (bTopY && bCenterX)
	{
		if (EdgeLocation != EDMEdgeLocation::Top)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::Top;
		}
	}
	else if (bTopY && bRightX)
	{
		if (EdgeLocation != EDMEdgeLocation::TopRight)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::TopRight;
		}
	}
	else if (bCenterY && bLeftX)
	{
		if (EdgeLocation != EDMEdgeLocation::Left)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::Left;
		}
	}
	else if (bCenterY && bCenterX)
	{
		if (EdgeLocation != EDMEdgeLocation::Center)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::Center;
		}
	}
	else if (bCenterY && bRightX)
	{
		if (EdgeLocation != EDMEdgeLocation::Right)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::Right;
		}
	}
	else if (bBottomY && bLeftX)
	{
		if (EdgeLocation != EDMEdgeLocation::BottomLeft)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::BottomLeft;
		}
	}
	else if (bBottomY && bCenterX)
	{
		if (EdgeLocation != EDMEdgeLocation::Bottom)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::Bottom;
		}
	}
	else if (bBottomY && bRightX)
	{
		if (EdgeLocation != EDMEdgeLocation::BottomRight)
		{
			bChangedEdgeLocation = true;
			EdgeLocation = EDMEdgeLocation::BottomRight;
		}
	}

	if (bChangedEdgeLocation)
	{
		Super::Update(InSource, InUpdateType | EDMUpdateType::Value);
		return;
	}

	Super::Update(InSource, InUpdateType);
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::PreEditChange(FEditPropertyChain& InPropertyAboutToChange)
{
	PreEditEdgeLocation = EdgeLocation;
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName EdgeLocationName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTextureSampleEdgeColor, EdgeLocation);

	if (InPropertyChangedEvent.MemberProperty && InPropertyChangedEvent.MemberProperty->GetFName() == EdgeLocationName)
	{
		OnEdgeLocationChanged();
	}
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::CopyParametersFrom_Implementation(UObject* InOther)
{
	UDMMaterialStageExpressionTextureSampleEdgeColor* OtherEdgeColor = CastChecked<UDMMaterialStageExpressionTextureSampleEdgeColor>(InOther);
	OtherEdgeColor->EdgeLocation = EdgeLocation;
}

void UDMMaterialStageExpressionTextureSampleEdgeColor::OnEdgeLocationChanged()
{
	UDMMaterialStage* Stage = GetStage();
	check(Stage);
	check(Stage->GetInputs().IsValidIndex(1));

	UDMMaterialStageInputValue* InputValue = CastChecked<UDMMaterialStageInputValue>(Stage->GetInputs()[1]);
	UDMMaterialValueFloat2* Float2 = CastChecked<UDMMaterialValueFloat2>(InputValue->GetValue());

	if (EdgeLocation != EDMEdgeLocation::Custom)
	{
		FVector2D Float2Value = FVector2D::ZeroVector;

		switch (EdgeLocation)
		{
			case EDMEdgeLocation::TopLeft:
				Float2Value.X = 0;
				Float2Value.Y = 0;
				break;

			case EDMEdgeLocation::Top:
				Float2Value.X = 0.5;
				Float2Value.Y = 0;
				break;

			case EDMEdgeLocation::TopRight:
				Float2Value.X = 1.f;
				Float2Value.Y = 0;
				break;

			case EDMEdgeLocation::Left:
				Float2Value.X = 0;
				Float2Value.Y = 0.5;
				break;

			case EDMEdgeLocation::Center:
				Float2Value.X = 0.5;
				Float2Value.Y = 0.5;
				break;

			case EDMEdgeLocation::Right:
				Float2Value.X = 1.f;
				Float2Value.Y = 0.5;
				break;

			case EDMEdgeLocation::BottomLeft:
				Float2Value.X = 0;
				Float2Value.Y = 1.f;
				break;

			case EDMEdgeLocation::Bottom:
				Float2Value.X = 0.5;
				Float2Value.Y = 1.f;
				break;

			case EDMEdgeLocation::BottomRight:
				Float2Value.X = 1.f;
				Float2Value.Y = 1.f;
				break;

			default:
				checkNoEntry();
				break;
		}

		Float2->SetValue(Float2Value);
		Float2->Update(this, EDMUpdateType::Value);
	}

	if (EdgeLocation == EDMEdgeLocation::Custom || PreEditEdgeLocation == EDMEdgeLocation::Custom)
	{
		Update(this, EDMUpdateType::Value | EDMUpdateType::RefreshDetailView);
	}
	else
	{
		Update(this, EDMUpdateType::Value);
	}
}

#undef LOCTEXT_NAMESPACE
