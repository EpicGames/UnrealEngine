// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETime.h"
#include "Materials/MaterialExpressionTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETime)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTime"

UDMMaterialStageExpressionTime::UDMMaterialStageExpressionTime()
	: UDMMaterialStageExpression(
		LOCTEXT("Time", "Time"),
		UMaterialExpressionTime::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Time);

	OutputConnectors.Add({0, LOCTEXT("Time", "Time"), EDMValueType::VT_Float1});
}

#undef LOCTEXT_NAMESPACE
