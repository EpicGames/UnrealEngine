// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETransform.h"
#include "Materials/MaterialExpressionTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETransform)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTransform"

UDMMaterialStageExpressionTransform::UDMMaterialStageExpressionTransform()
	: UDMMaterialStageExpression(
		LOCTEXT("Transform", "Rotate About Axis"),
		UMaterialExpressionTransform::StaticClass()
	)
{
	bInputRequired = true;

	InputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTransform, TransformSourceType));
	TransformSourceType = EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_World;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionTransform, TransformType));
	TransformType = EMaterialVectorCoordTransform::TRANSFORM_World;

	OutputConnectors.Add({0, LOCTEXT("TransformedPosition", "Transformed Position"), EDMValueType::VT_Float3_XYZ});
}

void UDMMaterialStageExpressionTransform::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionTransform* TransformExpression = Cast<UMaterialExpressionTransform>(InExpressions[0]);
	TransformExpression->TransformSourceType = TransformSourceType;
	TransformExpression->TransformType = TransformType;
}

#undef LOCTEXT_NAMESPACE
