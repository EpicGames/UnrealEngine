// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "Materials/MaterialExpressionTransform.h"
#include "DMMSETransform.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionTransform : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionTransform();

	virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TEnumAsByte<enum EMaterialVectorCoordTransformSource> TransformSourceType;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TEnumAsByte<enum EMaterialVectorCoordTransform> TransformType;
};
