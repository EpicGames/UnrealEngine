// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSETruncate.h"
#include "Materials/MaterialExpressionTruncate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSETruncate)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTruncate"

UDMMaterialStageExpressionTruncate::UDMMaterialStageExpressionTruncate()
	: UDMMaterialStageExpressionMathBase(
		LOCTEXT("Truncate", "Truncate"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionTruncate"))
	)
{
	SetupInputs(1);
}

#undef LOCTEXT_NAMESPACE
