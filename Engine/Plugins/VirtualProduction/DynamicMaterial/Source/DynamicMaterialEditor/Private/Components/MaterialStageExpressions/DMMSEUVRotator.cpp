// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEUVRotator.h"
#include "Materials/MaterialExpressionRotator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEUVRotator)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionUVRotator"

UDMMaterialStageExpressionUVRotator::UDMMaterialStageExpressionUVRotator()
	: UDMMaterialStageExpression(
		LOCTEXT("UVRotator", "UV Rotator"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionRotator"))
	)
{
	InputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
	InputConnectors.Add({1, LOCTEXT("Time", "Time"), EDMValueType::VT_Float1});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionUVRotator, CenterX));
	CenterX = 0.5;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionUVRotator, CenterY));
	CenterY = 0.5;

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionUVRotator, Speed));
	Speed = 1;

	Menus.Add(EDMExpressionMenu::Texture);
	Menus.Add(EDMExpressionMenu::Time);

	OutputConnectors.Add({0, LOCTEXT("RotatedUV", "Rotated UV"), EDMValueType::VT_Float2});
}

void UDMMaterialStageExpressionUVRotator::AddDefaultInput(int32 InInputIndex) const
{
	if (InInputIndex == 1)
	{
		return;
	}

	UDMMaterialStageExpression::AddDefaultInput(InInputIndex);
}

void UDMMaterialStageExpressionUVRotator::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionRotator* RotatorExpression = static_cast<UMaterialExpressionRotator*>(InExpressions[0]); // Type is not exported
	RotatorExpression->CenterX = CenterX;
	RotatorExpression->CenterY = CenterY;
	RotatorExpression->Speed = Speed;
}

#undef LOCTEXT_NAMESPACE
