// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSEUVRotator.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionUVRotator : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionUVRotator();

	//~ Begin UDMMaterialStageSource
	virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;
	//~ End UDMMaterialStageSource
	
	//~ Begin UDMMaterialStageThroughput
	virtual void AddDefaultInput(int32 InInputIndex) const override;
	virtual bool SupportsLayerMaskTextureUVLink() const override { return true; }
	virtual int32 GetLayerMaskTextureUVLinkInputIndex() const override { return 0; }
	//~ End UDMMaterialStageThroughput

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	float CenterX;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	float CenterY;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	float Speed;
};
