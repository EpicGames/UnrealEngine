// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEVertexColor.h"
#include "Materials/MaterialExpressionVertexColor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEVertexColor)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionVertexColor"

UDMMaterialStageExpressionVertexColor::UDMMaterialStageExpressionVertexColor()
	: UDMMaterialStageExpression(
		LOCTEXT("VertexColor", "Vertex Color"),
		UMaterialExpressionVertexColor::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Geometry);

	OutputConnectors.Add({0, LOCTEXT("Color", "Color"), EDMValueType::VT_Float4_RGBA});
}

#undef LOCTEXT_NAMESPACE
