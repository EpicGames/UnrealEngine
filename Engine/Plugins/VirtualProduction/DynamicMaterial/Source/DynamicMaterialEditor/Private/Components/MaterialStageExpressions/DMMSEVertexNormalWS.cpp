// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEVertexNormalWS.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEVertexNormalWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionVertexNormalWS"

UDMMaterialStageExpressionVertexNormalWS::UDMMaterialStageExpressionVertexNormalWS()
	: UDMMaterialStageExpression(
		LOCTEXT("VertexNormalWS", "Vertex Normal (WS)"),
		UMaterialExpressionVertexNormalWS::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Geometry);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Normal", "Normal"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
