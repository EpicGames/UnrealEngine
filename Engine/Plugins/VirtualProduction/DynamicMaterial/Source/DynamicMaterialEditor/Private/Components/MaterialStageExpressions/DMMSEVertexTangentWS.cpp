// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEVertexTangentWS.h"
#include "Materials/MaterialExpressionVertexTangentWS.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEVertexTangentWS)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionVertexTangentWS"

UDMMaterialStageExpressionVertexTangentWS::UDMMaterialStageExpressionVertexTangentWS()
	: UDMMaterialStageExpression(
		LOCTEXT("VertexTangentWS", "Vertex Tangent (WS)"),
		UMaterialExpressionVertexTangentWS::StaticClass()
	)
{
	Menus.Add(EDMExpressionMenu::Geometry);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Tangent", "Tangent"), EDMValueType::VT_Float3_XYZ});
}

#undef LOCTEXT_NAMESPACE
