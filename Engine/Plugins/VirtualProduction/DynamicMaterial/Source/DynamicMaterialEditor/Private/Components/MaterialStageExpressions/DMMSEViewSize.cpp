// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEViewSize.h"
#include "Materials/MaterialExpressionViewSize.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEViewSize)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionViewSize"

UDMMaterialStageExpressionViewSize::UDMMaterialStageExpressionViewSize()
	: UDMMaterialStageExpression(
		LOCTEXT("ViewSize", "ViewSize"),
		UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionViewSize"))
	)
{
	Menus.Add(EDMExpressionMenu::Other);

	OutputConnectors.Add({0, LOCTEXT("Size", "Size"), EDMValueType::VT_Float2});
}

#undef LOCTEXT_NAMESPACE
