// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEWorldPosition.h"
#include "Materials/MaterialExpressionWorldPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEWorldPosition)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionWorldPosition"

UDMMaterialStageExpressionWorldPosition::UDMMaterialStageExpressionWorldPosition()
	: UDMMaterialStageExpression(
		LOCTEXT("WorldPosition", "World Position"),
		UMaterialExpressionWorldPosition::StaticClass()
	)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPosition, WorldPositionShaderOffset));
	WorldPositionShaderOffset = EWorldPositionIncludedOffsets::WPT_Default;

	Menus.Add(EDMExpressionMenu::Object);
	Menus.Add(EDMExpressionMenu::WorldSpace);

	OutputConnectors.Add({0, LOCTEXT("Position", "Position"), EDMValueType::VT_Float3_XYZ});
}

void UDMMaterialStageExpressionWorldPosition::AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const
{
	check(InExpressions.Num() == 1);

	UMaterialExpressionWorldPosition* WorldPositionExpression = Cast<UMaterialExpressionWorldPosition>(InExpressions[0]);
	WorldPositionExpression->WorldPositionShaderOffset = WorldPositionShaderOffset;
}

#undef LOCTEXT_NAMESPACE
