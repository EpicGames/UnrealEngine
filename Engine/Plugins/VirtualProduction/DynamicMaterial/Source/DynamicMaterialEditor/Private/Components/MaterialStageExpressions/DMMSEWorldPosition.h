// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "DMMSEWorldPosition.generated.h"

UCLASS(BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionWorldPosition : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionWorldPosition();

	virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TEnumAsByte<EWorldPositionIncludedOffsets> WorldPositionShaderOffset;
};
