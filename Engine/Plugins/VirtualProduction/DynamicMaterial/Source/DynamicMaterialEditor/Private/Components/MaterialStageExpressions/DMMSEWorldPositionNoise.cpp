// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageExpressions/DMMSEWorldPositionNoise.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "CoreGlobals.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSEWorldPositionNoise)

#define LOCTEXT_NAMESPACE "DMMaterialStageExpressionTextureSample"

UDMMaterialStageExpressionWorldPositionNoise::UDMMaterialStageExpressionWorldPositionNoise()
	: UDMMaterialStageExpression(
		LOCTEXT("UMaterialExpressionVectorNoise", "Noise"),
		UMaterialExpressionVectorNoise::StaticClass()
	)
	, LocationType(EDMLocationType::World)
	, ShaderOffset(EWorldPositionIncludedOffsets::WPT_Default)
	, NoiseFunction(EVectorNoiseFunction::VNF_VectorALU)
	, Quality(1)
	, bTiling(false)
	, TileSize(300)
{
	bInputRequired = true;
	bAllowNestedInputs = true;

	InputConnectors.Add({1, LOCTEXT("Scale", "Scale"), EDMValueType::VT_Float3_XYZ});
	InputConnectors.Add({1, LOCTEXT("Offset", "Offset"), EDMValueType::VT_Float3_XYZ});

	OutputConnectors.Add({0, LOCTEXT("ColorRGB", "Color (RGB)"), EDMValueType::VT_Float3_RGB});

	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, LocationType));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, ShaderOffset));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, NoiseFunction));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, Quality));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, bTiling));
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, TileSize));
}

void UDMMaterialStageExpressionWorldPositionNoise::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	check(MaterialExpressionClass.Get());

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	UMaterialExpressionWorldPosition* WorldPosition = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionWorldPosition>(UE_DM_NodeComment_Default);
	WorldPosition->WorldPositionShaderOffset = ShaderOffset;

	UMaterialExpressionDivide* Divide = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionDivide>(UE_DM_NodeComment_Default);

	switch (LocationType)
	{
		case EDMLocationType::World:
			WorldPosition->ConnectExpression(&Divide->A, 0);
			break;

		case EDMLocationType::Actor:
		{
			// Not exported
			TSubclassOf<UMaterialExpression> ActorPositionWSClass = FindClass("MaterialExpressionActorPositionWS");
			UMaterialExpression* ActorPositionWS = InBuildState->GetBuildUtils().CreateExpression(ActorPositionWSClass, UE_DM_NodeComment_Default);

			UMaterialExpressionSubtract* Subtract = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionSubtract>(UE_DM_NodeComment_Default);
			WorldPosition->ConnectExpression(&Subtract->A, 0);
			ActorPositionWS->ConnectExpression(&Subtract->B, 0);
			Subtract->ConnectExpression(&Divide->A, 0);

			break;
		}
	}

	UMaterialExpressionAdd* Add = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionAdd>(UE_DM_NodeComment_Default);
	Divide->ConnectExpression(&Add->A, 0);

	UMaterialExpressionVectorNoise* VectorNoise = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionVectorNoise>(UE_DM_NodeComment_Default);
	VectorNoise->NoiseFunction = NoiseFunction;
	VectorNoise->Quality = FMath::Clamp(Quality, 1, 4);
	VectorNoise->bTiling = bTiling ? 1 : 0;
	VectorNoise->TileSize = TileSize;

	Add->ConnectExpression(&VectorNoise->Position, 0);

	InBuildState->AddStageSourceExpressions(this, {WorldPosition, Divide, Add, VectorNoise});
}

void UDMMaterialStageExpressionWorldPositionNoise::SetLocationType(EDMLocationType InLocationType)
{
	if (LocationType == InLocationType)
	{
		return;
	}

	LocationType = InLocationType;

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageExpressionWorldPositionNoise::SetShaderOffset(EWorldPositionIncludedOffsets InShaderOffset)
{
	if (ShaderOffset == InShaderOffset)
	{
		return;
	}

	ShaderOffset = InShaderOffset;

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageExpressionWorldPositionNoise::SetNoiseFunction(EVectorNoiseFunction InNoiseFunction)
{
	if (NoiseFunction == InNoiseFunction)
	{
		return;
	}

	NoiseFunction = InNoiseFunction;

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageExpressionWorldPositionNoise::SetQuality(int32 InQuality)
{
	if (Quality == InQuality)
	{
		return;
	}

	Quality = InQuality;

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageExpressionWorldPositionNoise::SetTiling(bool bInTiling)
{
	if (bTiling == bInTiling)
	{
		return;
	}

	bTiling = bInTiling;

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageExpressionWorldPositionNoise::SetTileSize(int32 InTileSize)
{
	if (TileSize == InTileSize)
	{
		return;
	}

	TileSize = InTileSize;

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageExpressionWorldPositionNoise::AddDefaultInput(int32 InInputIndex) const
{
	Super::AddDefaultInput(InInputIndex);

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	switch (InInputIndex)
	{
		// Scale
		case 0:
		{
			UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
			check(InputValue);

			UDMMaterialValueFloat3XYZ* Float3Value = Cast<UDMMaterialValueFloat3XYZ>(InputValue->GetValue());
			check(Float3Value);

			Float3Value->SetDefaultValue({1.f, 10.f, 10.f});
			Float3Value->ApplyDefaultValue();
			break;
		}

		// Offset
		case 1:
		{
			UDMMaterialStageInputValue* InputValue = Cast<UDMMaterialStageInputValue>(Stage->GetInputs().Last());
			check(InputValue);

			UDMMaterialValueFloat3XYZ* Float3Value = Cast<UDMMaterialValueFloat3XYZ>(InputValue->GetValue());
			check(Float3Value);

			Float3Value->SetDefaultValue({0.f, 0.f, 0.f});
			Float3Value->ApplyDefaultValue();
			break;
		}

		default:
			// Do nothing
			break;
	}
}

UMaterialExpression* UDMMaterialStageExpressionWorldPositionNoise::GetExpressionForInput(const TArray<UMaterialExpression*>& InStageSourceExpressions, 
	int32 InInputIndex, int32 InExpressionInputIndex)
{
	switch (InInputIndex)
	{
		// Scale
		case 0:
			if (InStageSourceExpressions.IsValidIndex(1))
			{
				return InStageSourceExpressions[1];
			}
			break;

		// Offset
		case 1:
			if (InStageSourceExpressions.IsValidIndex(2))
			{
				return InStageSourceExpressions[2];
			}
			break;
	}

	return Super::GetExpressionForInput(InStageSourceExpressions, InInputIndex, InExpressionInputIndex);
}

void UDMMaterialStageExpressionWorldPositionNoise::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	static const FName LocationTypeName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, LocationType);
	static const FName ShaderOffsetName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, ShaderOffset);
	static const FName NoiseFunctionName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, NoiseFunction);
	static const FName QualityName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, Quality);
	static const FName TilingName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, bTiling);
	static const FName TileSizeName = GET_MEMBER_NAME_CHECKED(UDMMaterialStageExpressionWorldPositionNoise, TileSize);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == LocationTypeName
		|| PropertyName == ShaderOffsetName
		|| PropertyName == NoiseFunctionName
		|| PropertyName == QualityName
		|| PropertyName == TilingName
		|| PropertyName == TileSizeName)
	{
		Update(this, EDMUpdateType::Structure);
	}
}

#undef LOCTEXT_NAMESPACE
