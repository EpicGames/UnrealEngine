// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSEWorldPositionNoise.generated.h"

enum class EDMLocationType : uint8;
enum EVectorNoiseFunction : int;
enum EWorldPositionIncludedOffsets : int;

UCLASS(BlueprintType, Blueprintable, ClassGroup = "Material Designer", meta = (DisplayName = "Noise"))
class UDMMaterialStageExpressionWorldPositionNoise : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionWorldPositionNoise();

	//~ Begin UDMMaterialStageExpression
	virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialStageExpression

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual EDMLocationType GetLocationType() const { return LocationType; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void SetLocationType(EDMLocationType InLocationType);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual TEnumAsByte<EWorldPositionIncludedOffsets> GetShaderOffset() const { return ShaderOffset; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void SetShaderOffset(EWorldPositionIncludedOffsets InShaderOffset);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual TEnumAsByte<EVectorNoiseFunction> GetNoiseFunction() const { return NoiseFunction; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void SetNoiseFunction(EVectorNoiseFunction InNoiseFunction);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual int32 GetQuality() const { return Quality; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void SetQuality(int32 InQuality);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual bool GetTiling() const { return bTiling; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void SetTiling(bool bInTiling);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual int32 GetTileSize() const { return TileSize; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	virtual void SetTileSize(int32 InTileSize);

	//~ Begin UDMMaterialStageThroughput
	virtual void AddDefaultInput(int32 InInputIndex) const override;
	virtual UMaterialExpression* GetExpressionForInput(const TArray<UMaterialExpression*>& InStageSourceExpressions, int32 InInputIndex, int32 InExpressionInputIndex) override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetLocationType, BlueprintSetter = SetLocationType, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	EDMLocationType LocationType;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetShaderOffset, BlueprintSetter = SetShaderOffset, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	TEnumAsByte<EWorldPositionIncludedOffsets> ShaderOffset;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetNoiseFunction, BlueprintSetter = SetNoiseFunction, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	TEnumAsByte<EVectorNoiseFunction> NoiseFunction;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetQuality, BlueprintSetter = SetQuality, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true", UIMin = "1", UIMax = "4", ClampMin = "1", ClampMax = "4"))
	int32 Quality;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetTiling, BlueprintSetter = SetTiling, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true"))
	bool bTiling;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetTileSize, BlueprintSetter = SetTileSize, Category = "Material Designer",
		meta = (NotKeyframeable, AllowPrivateAccess = "true", EditCondition="bTiling", EditConditionHides))
	int32 TileSize;
};
