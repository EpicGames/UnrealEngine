// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageGradients/DMMSGLinear.h"

#include "DMDefs.h"
#include "Components/DMMaterialStageFunction.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSGLinear)

#define LOCTEXT_NAMESPACE "DMMaterialStageGradientLinear"

TSoftObjectPtr<UMaterialFunctionInterface> UDMMaterialStageGradientLinear::LinearGradientNoTileFunction = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(TEXT(
	"/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Gradients/MF_DM_LinearGradient.MF_DM_LinearGradient'"
)));

TSoftObjectPtr<UMaterialFunctionInterface> UDMMaterialStageGradientLinear::LinearGradientTileFunction = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(TEXT(
	"/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Gradients/MF_DM_LinearGradient_Tile.MF_DM_LinearGradient_Tile'"
)));

TSoftObjectPtr<UMaterialFunctionInterface> UDMMaterialStageGradientLinear::LinearGradientTileAndMirrorFunction = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(TEXT(
	"/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Gradients/MF_DM_LinearGradient_TileAndMirror.MF_DM_LinearGradient_TileAndMirror'"
)));

UDMMaterialStageGradientLinear::UDMMaterialStageGradientLinear()
	: UDMMaterialStageGradient(LOCTEXT("GradientLinear", "Linear Gradient"))
	, Tiling(ELinearGradientTileType::NoTile)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageGradientLinear, Tiling));

	MaterialFunction = GetMaterialFunctionForTilingType(Tiling);
}

void UDMMaterialStageGradientLinear::SetTilingType(ELinearGradientTileType InType)
{
	if (Tiling == InType)
	{
		return;
	}

	Tiling = InType;

	OnTilingChanged();
}

void UDMMaterialStageGradientLinear::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UDMMaterialStageGradientLinear, Tiling))
	{
		OnTilingChanged();
	}
}

void UDMMaterialStageGradientLinear::PostEditUndo()
{
	Super::PostEditUndo();

	SetMaterialFunction(GetMaterialFunctionForTilingType(Tiling));
}

UMaterialFunctionInterface* UDMMaterialStageGradientLinear::GetMaterialFunctionForTilingType(ELinearGradientTileType) const
{
	switch (Tiling)
	{
		case ELinearGradientTileType::NoTile:
			return LinearGradientNoTileFunction.LoadSynchronous();

		case ELinearGradientTileType::Tile:
			return LinearGradientTileFunction.LoadSynchronous();

		case ELinearGradientTileType::TileAndMirror:
			return LinearGradientTileAndMirrorFunction.LoadSynchronous();

		default:
			return UDMMaterialStageFunction::NoOp.LoadSynchronous();
	}
}

void UDMMaterialStageGradientLinear::OnTilingChanged()
{
	const bool bWasUpdateCalled = SetMaterialFunction(GetMaterialFunctionForTilingType(Tiling));

	if (!bWasUpdateCalled)
	{
		Update(this, EDMUpdateType::Structure);
	}
}

#undef LOCTEXT_NAMESPACE
