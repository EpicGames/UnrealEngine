// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageGradients/DMMSGRadial.h"

#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSGRadial)

#define LOCTEXT_NAMESPACE "DMMaterialStageGradientRadial"

TSoftObjectPtr<UMaterialFunctionInterface> UDMMaterialStageGradientRadial::RadialGradientFunction = TSoftObjectPtr<UMaterialFunctionInterface>(FSoftObjectPath(TEXT(
	"/Script/Engine.MaterialFunction'/DynamicMaterial/MaterialFunctions/Gradients/MF_DM_RadialGradient.MF_DM_RadialGradient'"
)));

UDMMaterialStageGradientRadial::UDMMaterialStageGradientRadial()
	: UDMMaterialStageGradient(LOCTEXT("GradientRadial", "Radial Gradient"))
{
	MaterialFunction = RadialGradientFunction.LoadSynchronous();
}

#undef LOCTEXT_NAMESPACE
