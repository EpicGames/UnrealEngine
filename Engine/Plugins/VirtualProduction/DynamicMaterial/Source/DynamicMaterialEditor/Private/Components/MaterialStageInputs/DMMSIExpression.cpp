// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSIExpression.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageExpression.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSIExpression)

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageInputExpression::InputExpressions = TArray<TStrongObjectPtr<UClass>>();

UDMMaterialStage* UDMMaterialStageInputExpression::CreateStage(TSubclassOf<UDMMaterialStageExpression> InMaterialStageExpressionClass, UDMMaterialLayerObject* InLayer)
{
	check(InMaterialStageExpressionClass);

	GetAvailableInputExpressions();
	check(InputExpressions.Contains(TStrongObjectPtr<UClass>(InMaterialStageExpressionClass.Get())));

	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageInputExpression* InputExpression = NewObject<UDMMaterialStageInputExpression>(NewStage, NAME_None, RF_Transactional);
	check(InputExpression);
	InputExpression->SetMaterialStageExpressionClass(InMaterialStageExpressionClass);

	NewStage->SetSource(InputExpression);

	return NewStage;
}

void UDMMaterialStageInputExpression::SetMaterialStageExpressionClass(TSubclassOf<UDMMaterialStageExpression> InMaterialStageExpressionClass)
{
	SetMaterialStageThroughputClass(InMaterialStageExpressionClass);
}

UDMMaterialStageExpression* UDMMaterialStageInputExpression::GetMaterialStageExpression() const
{
	return Cast<UDMMaterialStageExpression>(GetMaterialStageThroughput());
}

const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageInputExpression::GetAvailableInputExpressions()
{
	if (InputExpressions.IsEmpty())
	{
		GenerateExpressionList();
	}

	return InputExpressions;
}

UDMMaterialStageInputExpression* UDMMaterialStageInputExpression::ChangeStageSource_Expression(UDMMaterialStage* InStage, 
	TSubclassOf<UDMMaterialStageExpression> InExpressionClass)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	check(InExpressionClass);
	check(!InExpressionClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	UDMMaterialStageInputExpression* NewInputExpression = InStage->ChangeSource<UDMMaterialStageInputExpression>(
		[InExpressionClass](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputExpression>(InNewSource)->SetMaterialStageExpressionClass(InExpressionClass);
		});

	return NewInputExpression;
}

UDMMaterialStageInputExpression* UDMMaterialStageInputExpression::ChangeStageInput_Expression(UDMMaterialStage* InStage, 
	TSubclassOf<UDMMaterialStageExpression> InExpressionClass, int32 InInputIdx, int32 InInputChannel, int32 InOutputIdx, 
	int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	check(InExpressionClass);
	check(!InExpressionClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	UDMMaterialStageInputExpression* NewInputExpression = InStage->ChangeInput<UDMMaterialStageInputExpression>(
		InInputIdx, InInputChannel, InOutputIdx, InOutputChannel,
		[InExpressionClass](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputExpression>(InNewInput)->SetMaterialStageExpressionClass(InExpressionClass);
		}
	);

	return NewInputExpression;
}

TSubclassOf<UDMMaterialStageExpression> UDMMaterialStageInputExpression::GetMaterialStageExpressionClass() const
{
	return TSubclassOf<UDMMaterialStageExpression>(GetMaterialStageThroughputClass());
}

void UDMMaterialStageInputExpression::GenerateExpressionList()
{
	InputExpressions.Empty();

	const TArray<TStrongObjectPtr<UClass>>& SourceList = UDMMaterialStageExpression::GetAvailableSourceExpressions();

	for (const TStrongObjectPtr<UClass>& SourceClass : SourceList)
	{
		UDMMaterialStageExpression* StageExpressionCDO = CastChecked<UDMMaterialStageExpression>(SourceClass->GetDefaultObject(true));

		if (StageExpressionCDO->IsInputRequired())
		{
			continue;
		}

		InputExpressions.Add(SourceClass);
	}
}
