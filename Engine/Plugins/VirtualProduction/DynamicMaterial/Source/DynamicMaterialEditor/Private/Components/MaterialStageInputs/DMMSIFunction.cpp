// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSIFunction.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSIFunction)

UDMMaterialStage* UDMMaterialStageInputFunction::CreateStage(UDMMaterialLayerObject* InLayer)
{
	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageInputFunction* InputFunction = NewObject<UDMMaterialStageInputFunction>(NewStage, NAME_None, RF_Transactional);
	check(InputFunction);

	InputFunction->Init();

	NewStage->SetSource(InputFunction);

	return NewStage;
}

UDMMaterialStageInputFunction* UDMMaterialStageInputFunction::ChangeStageSource_Function(UDMMaterialStage* InStage,
	UMaterialFunctionInterface* InMaterialFunction)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	check(InMaterialFunction);

	UDMMaterialStageInputFunction* NewInputFunction = InStage->ChangeSource<UDMMaterialStageInputFunction>(
		[InMaterialFunction](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			UDMMaterialStageInputFunction* NewFunction = CastChecked<UDMMaterialStageInputFunction>(InNewSource);
			NewFunction->Init();
			NewFunction->SetMaterialFunction(InMaterialFunction);
		});

	return NewInputFunction;
}

UDMMaterialStageInputFunction* UDMMaterialStageInputFunction::ChangeStageInput_Function(UDMMaterialStage* InStage,
	UMaterialFunctionInterface* InMaterialFunction, int32 InInputIdx, int32 InInputChannel, int32 InOutputIdx,
	int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	check(InMaterialFunction);

	UDMMaterialStageInputFunction* NewInputFunction = InStage->ChangeInput<UDMMaterialStageInputFunction>(
		InInputIdx, InInputChannel, InOutputIdx, InOutputChannel,
		[InMaterialFunction](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			UDMMaterialStageInputFunction* NewFunction = CastChecked<UDMMaterialStageInputFunction>(InNewInput);
			NewFunction->Init();
			NewFunction->SetMaterialFunction(InMaterialFunction);
		}
	);

	return NewInputFunction;
}

void UDMMaterialStageInputFunction::Init()
{
	SetMaterialStageThroughputClass(UDMMaterialStageFunction::StaticClass());
}

UDMMaterialStageFunction* UDMMaterialStageInputFunction::GetMaterialStageFunction() const
{
	return Cast<UDMMaterialStageFunction>(GetMaterialStageThroughput());
}

UMaterialFunctionInterface* UDMMaterialStageInputFunction::GetMaterialFunction() const
{
	if (UDMMaterialStageFunction* StageFunction = GetMaterialStageFunction())
	{
		return StageFunction->GetMaterialFunction();
	}

	return nullptr;
}

void UDMMaterialStageInputFunction::SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction)
{
	if (UDMMaterialStageFunction* StageFunction = GetMaterialStageFunction())
	{
		StageFunction->SetMaterialFunction(InMaterialFunction);
	}
}
