// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSIGradient.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageExpression.h"
#include "Components/DMMaterialStageGradient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSIGradient)

TArray<TStrongObjectPtr<UClass>> UDMMaterialStageInputGradient::Gradients = TArray<TStrongObjectPtr<UClass>>();

UDMMaterialStage* UDMMaterialStageInputGradient::CreateStage(TSubclassOf<UDMMaterialStageGradient> InMaterialStageGradientClass, UDMMaterialLayerObject* InLayer)
{
	check(InMaterialStageGradientClass);

	GetAvailableGradients();
	check(Gradients.Contains(TStrongObjectPtr<UClass>(InMaterialStageGradientClass.Get())));

	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageInputGradient* InputExpression = NewObject<UDMMaterialStageInputGradient>(NewStage, NAME_None, RF_Transactional);
	check(InputExpression);
	InputExpression->SetMaterialStageGradientClass(InMaterialStageGradientClass);

	NewStage->SetSource(InputExpression);

	return NewStage;
}

UDMMaterialStageInputGradient* UDMMaterialStageInputGradient::ChangeStageSource_Gradient(UDMMaterialStage* InStage,
	TSubclassOf<UDMMaterialStageGradient> InGradientClass)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	check(InGradientClass);
	check(!InGradientClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	UDMMaterialStageInputGradient* NewInputGradient = InStage->ChangeSource<UDMMaterialStageInputGradient>(
		[InGradientClass](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputGradient>(InNewSource)->SetMaterialStageGradientClass(InGradientClass);
		});

	return NewInputGradient;
}

UDMMaterialStageInputGradient* UDMMaterialStageInputGradient::ChangeStageInput_Gradient(UDMMaterialStage* InStage,
	TSubclassOf<UDMMaterialStageGradient> InGradientClass, int32 InInputIdx, int32 InInputChannel, int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	check(InGradientClass);
	check(!InGradientClass->HasAnyClassFlags(UE::DynamicMaterial::InvalidClassFlags));

	UDMMaterialStageInputGradient* NewInputGradient = InStage->ChangeInput<UDMMaterialStageInputGradient>(
		InInputIdx, InInputChannel, 0, InOutputChannel,
		[InGradientClass](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputGradient>(InNewInput)->SetMaterialStageGradientClass(InGradientClass);
		}
	);

	return NewInputGradient;
}

void UDMMaterialStageInputGradient::SetMaterialStageGradientClass(TSubclassOf<UDMMaterialStageGradient> InMaterialStageGradientClass)
{
	SetMaterialStageThroughputClass(InMaterialStageGradientClass);
}

UDMMaterialStageGradient* UDMMaterialStageInputGradient::GetMaterialStageGradient() const
{
	return Cast<UDMMaterialStageGradient>(GetMaterialStageThroughput());
}

const TArray<TStrongObjectPtr<UClass>>& UDMMaterialStageInputGradient::GetAvailableGradients()
{
	if (Gradients.IsEmpty())
	{
		GenerateGradientList();
	}

	return Gradients;
}

TSubclassOf<UDMMaterialStageGradient> UDMMaterialStageInputGradient::GetMaterialStageGradientClass() const
{
	return TSubclassOf<UDMMaterialStageGradient>(GetMaterialStageThroughputClass());
}

void UDMMaterialStageInputGradient::GenerateGradientList()
{
	Gradients.Empty();

	const TArray<TStrongObjectPtr<UClass>>& SourceList = UDMMaterialStageExpression::GetAvailableSourceExpressions();

	for (const TStrongObjectPtr<UClass>& SourceClass : SourceList)
	{
		UDMMaterialStageGradient* StageGradientCDO = CastChecked<UDMMaterialStageGradient>(SourceClass->GetDefaultObject(true));

		if (StageGradientCDO->IsInputRequired())
		{
			continue;
		}

		Gradients.Add(SourceClass);
	}
}
