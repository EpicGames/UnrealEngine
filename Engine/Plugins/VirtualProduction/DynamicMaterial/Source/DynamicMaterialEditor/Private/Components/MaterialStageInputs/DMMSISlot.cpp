// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSISlot.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageThroughput.h"
#include "DMComponentPath.h"
#include "DMValueDefinition.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSISlot)

#define LOCTEXT_NAMESPACE "DMMaterialStageInputSlot"

const FString UDMMaterialStageInputSlot::SlotPathToken = FString(TEXT("Slot"));

UDMMaterialStage* UDMMaterialStageInputSlot::CreateStage(UDMMaterialSlot* InSourceSlot, EDMMaterialPropertyType InMaterialProperty,
	UDMMaterialLayerObject* InLayer)
{
	check(InSourceSlot);

	if (IsValid(InLayer))
	{
		UDMMaterialSlot* LayerSlot = InLayer->GetSlot();
		check(LayerSlot);
		check(LayerSlot != InSourceSlot);
		check(LayerSlot->GetMaterialModelEditorOnlyData() == InSourceSlot->GetMaterialModelEditorOnlyData());
	}

	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageInputSlot* InputSlot = NewObject<UDMMaterialStageInputSlot>(NewStage, NAME_None, RF_Transactional);
	check(InputSlot);

	InputSlot->SetSlot(InSourceSlot);
	InputSlot->SetMaterialProperty(InMaterialProperty);

	NewStage->SetSource(InputSlot);

	return NewStage;
}

UDMMaterialStageInputSlot* UDMMaterialStageInputSlot::ChangeStageSource_Slot(UDMMaterialStage* InStage, UDMMaterialSlot* InSlot,
	EDMMaterialPropertyType InProperty)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	check(InSlot);
	check(Slot != InSlot);
	check(InSlot->GetMaterialModelEditorOnlyData() == Slot->GetMaterialModelEditorOnlyData());

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	TArray<EDMMaterialPropertyType> SlotPropertes = ModelEditorOnlyData->GetMaterialPropertiesForSlot(InSlot);
	check(SlotPropertes.Contains(InProperty));

	UDMMaterialStageInputSlot* InputSlot = InStage->ChangeSource<UDMMaterialStageInputSlot>(
		[InSlot, InProperty](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			UDMMaterialStageInputSlot* InputSlot = CastChecked<UDMMaterialStageInputSlot>(InNewSource);
			InputSlot->SetSlot(InSlot);
			InputSlot->SetMaterialProperty(InProperty);
		});

	return InputSlot;
}

UDMMaterialStageInputSlot* UDMMaterialStageInputSlot::ChangeStageInput_Slot(UDMMaterialStage* InStage, int32 InInputIdx, 
	int32 InInputChannel, UDMMaterialSlot* InSlot, EDMMaterialPropertyType InProperty, int32 InOutputIdx, int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData == InSlot->GetMaterialModelEditorOnlyData());

	const TArray<EDMMaterialPropertyType> SlotPropertes = ModelEditorOnlyData->GetMaterialPropertiesForSlot(InSlot);
	check(SlotPropertes.Contains(InProperty));

	const TArray<EDMValueType>& SlotPropertyOutputTypes = InSlot->GetOutputConnectorTypesForMaterialProperty(InProperty);
	check(SlotPropertyOutputTypes.IsValidIndex(InOutputIdx));

	if (InOutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		check(Throughput->CanInputAcceptType(InInputIdx, SlotPropertyOutputTypes[InOutputIdx]));
	}
	else
	{
		check(UDMValueDefinitionLibrary::GetValueDefinition(SlotPropertyOutputTypes[InOutputIdx]).IsFloatType());
		check(Throughput->CanInputAcceptType(InInputIdx, EDMValueType::VT_Float1));
	}

	UDMMaterialStageInputSlot* NewInputSlot = InStage->ChangeInput<UDMMaterialStageInputSlot>(
		InInputIdx, InInputChannel, InOutputIdx, InOutputChannel, 
		[InSlot, InProperty](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			UDMMaterialStageInputSlot* NewInputSlot = CastChecked<UDMMaterialStageInputSlot>(InNewInput);
			NewInputSlot->SetSlot(InSlot);
			NewInputSlot->SetMaterialProperty(InProperty);
		}
	);

	return NewInputSlot;
}

FText UDMMaterialStageInputSlot::GetComponentDescription() const
{
	if (Slot)
	{
		if (Slot->IsComponentValid())
		{
			return Slot->GetDescription();
		}
	}
	
	return Super::GetComponentDescription();
}

FSlateIcon UDMMaterialStageInputSlot::GetComponentIcon() const
{
	if (Slot)
	{
		if (Slot->IsComponentValid())
		{
			return Slot->GetComponentIcon();
		}
	}

	return Super::GetComponentIcon();
}

FText UDMMaterialStageInputSlot::GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel)
{
	return GetComponentDescription();
}

void UDMMaterialStageInputSlot::SetSlot(UDMMaterialSlot* InSlot)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Slot == InSlot)
	{
		return;
	}

	if (UDMMaterialStage* Stage = GetStage())
	{
		if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
		{
			if (Layer->GetSlot() == InSlot)
			{
				return;
			}
		}
	}

	DeinitSlot();

	Slot = InSlot;

	InitSlot();

	UpdateOutputConnectors();

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageInputSlot::SetMaterialProperty(EDMMaterialPropertyType InMaterialProperty)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (MaterialProperty == InMaterialProperty)
	{
		return;
	}
	
	MaterialProperty = InMaterialProperty;

	UpdateOutputConnectors();

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageInputSlot::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	DeinitSlot();
}

UDMMaterialComponent* UDMMaterialStageInputSlot::GetSubComponentByPath(FDMComponentPath& InPath,
	const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == SlotPathToken)
	{
		return Slot;
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialStageInputSlot::InitSlot()
{
	if (Slot)
	{
		Slot->GetOnUpdate().AddUObject(this, &UDMMaterialStageInputSlot::OnSlotUpdated);
		Slot->GetOnConnectorsUpdateDelegate().AddUObject(this, &UDMMaterialStageInputSlot::OnSlotConnectorsUpdated);
		Slot->GetOnRemoved().AddUObject(this, &UDMMaterialStageInputSlot::OnSlotRemoved);

		if (UDMMaterialStage* Stage = GetStage())
		{
			if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
			{
				if (UDMMaterialSlot* StageSlot = Layer->GetSlot())
				{
					if (GUndo)
					{
						Slot->Modify();
					}

					Slot->ReferencedBySlot(StageSlot);
				}
			}
		}
	}
}

void UDMMaterialStageInputSlot::DeinitSlot()
{
	if (Slot)
	{
		Slot->GetOnUpdate().RemoveAll(this);
		Slot->GetOnConnectorsUpdateDelegate().RemoveAll(this);
		Slot->GetOnRemoved().RemoveAll(this);

		if (UDMMaterialStage* Stage = GetStage())
		{

			if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
			{
				if (UDMMaterialSlot* StageSlot = Layer->GetSlot())
				{
					if (GUndo)
					{
						Slot->Modify();
					}

					Slot->UnreferencedBySlot(StageSlot);
				}
			}
		}
	}
}

void UDMMaterialStageInputSlot::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	check(Slot);

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	Slot->GenerateExpressions(InBuildState);
	InBuildState->AddStageSourceExpressions(this, InBuildState->GetSlotExpressions(Slot));
}

void UDMMaterialStageInputSlot::PostLoad()
{
	Super::PostLoad();

	if (!IsComponentValid())
	{
		return;
	}

	InitSlot();
}

void UDMMaterialStageInputSlot::PostEditImport()
{
	Super::PostEditImport();

	if (!IsComponentValid())
	{
		return;
	}

	InitSlot();
}

void UDMMaterialStageInputSlot::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel,
	UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	InitSlot();
}

UDMMaterialStageInputSlot::UDMMaterialStageInputSlot()
	: Slot(nullptr)
	, MaterialProperty(EDMMaterialPropertyType::None)
{
}

void UDMMaterialStageInputSlot::OnSlotUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Slot != InSource)
	{
		return;
	}

	Update(InSource, InUpdateType);
}

void UDMMaterialStageInputSlot::OnSlotConnectorsUpdated(UDMMaterialSlot* InSlot)
{
	if (!IsComponentValid())
	{
		return;
	}

	UpdateOutputConnectors();

	UDMMaterialStage* Stage = GetStage();
	check(Stage);

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	check(Layer);

	UDMMaterialSlot* StageSlot = Layer->GetSlot();
	check(StageSlot);

	if (Layer->GetStageType(Stage) == EDMMaterialLayerStage::Base)
	{
		if (UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask))
		{
			MaskStage->Update(InSlot, EDMUpdateType::Structure);
		}
	}
	else
	{
		EDMMaterialPropertyType StageProperty = Layer->GetMaterialProperty();
		check(StageProperty != EDMMaterialPropertyType::None);

		if (UDMMaterialLayerObject* NextLayer = Layer->GetNextLayer(StageProperty, EDMMaterialLayerStage::Base))
		{
			NextLayer->GetStage(EDMMaterialLayerStage::Base)->ResetInputConnectionMap();
		}
		else
		{
			StageSlot->UpdateOutputConnectorTypes();
		}
	}
}

void UDMMaterialStageInputSlot::OnSlotRemoved(UDMMaterialComponent* InComponent, EDMComponentLifetimeState InLifetimeState)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Slot != InComponent)
	{
		return;
	}

	SetSlot(nullptr);
}

void UDMMaterialStageInputSlot::OnParentSlotRemoved(UDMMaterialComponent* InComponent, EDMComponentLifetimeState InLifetimeState)
{
	if (!IsComponentValid())
	{
		return;
	}

	check(InComponent);
	check(InComponent == Slot);

	SetSlot(nullptr);
}

void UDMMaterialStageInputSlot::UpdateOutputConnectors()
{
	if (!IsComponentValid())
	{
		return;
	}

	OutputConnectors.Empty();

	if (Slot)
	{
		UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
		check(ModelEditorOnlyData);

		const TMap<EDMMaterialPropertyType, UDMMaterialProperty*> MaterialProperties = ModelEditorOnlyData->GetMaterialProperties();

		if (MaterialProperties.Contains(MaterialProperty))
		{
			const FText MaterialPropertyName = UE::DynamicMaterialEditor::Private::GetMaterialPropertyShortDisplayName(MaterialProperty);
			const TArray<EDMValueType>& OutputTypes = Slot->GetOutputConnectorTypesForMaterialProperty(MaterialProperty);

			for (int32 OutputTypeIdx = 0; OutputTypeIdx < OutputTypes.Num(); ++OutputTypeIdx)
			{
				static const FText SlotConnectorTemplate = LOCTEXT("SlotOutputType", "{0}: Slot Output {1} ({2})");

				FText SlotConnectorName = FText::Format(
					SlotConnectorTemplate,
					MaterialPropertyName,
					FText::AsNumber(OutputTypeIdx),
					UDMValueDefinitionLibrary::GetValueDefinition(OutputTypes[OutputTypeIdx]).GetDisplayName()
				);

				OutputConnectors.Add({OutputTypeIdx, SlotConnectorName, OutputTypes[OutputTypeIdx]});
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
