// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSITextureUV.h"
#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialParameter.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageExpression.h"
#include "Components/DMTextureUV.h"
#include "DMComponentPath.h"
#include "DMDefs.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialModule.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialFunctionLibrary.h"
#include "Utils/DMUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSITextureUV)

#define LOCTEXT_NAMESPACE "DMMaterialStageInputTextureUV"

const FString UDMMaterialStageInputTextureUV::TextureUVPathToken = FString(TEXT("TextureUV"));

UDMMaterialStage* UDMMaterialStageInputTextureUV::CreateStage(UDynamicMaterialModel* InMaterialModel, UDMMaterialLayerObject* InLayer)
{
	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageInputTextureUV* InputTextureUV = NewObject<UDMMaterialStageInputTextureUV>(NewStage, NAME_None, RF_Transactional);
	check(InputTextureUV);

	InputTextureUV->Init(InMaterialModel);

	NewStage->SetSource(InputTextureUV);

	return NewStage;
}

UDMMaterialStageInputTextureUV* UDMMaterialStageInputTextureUV::ChangeStageSource_UV(UDMMaterialStage* InStage, bool bInDoUpdate)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	UDMMaterialStageInputTextureUV* InputTextureUV = InStage->ChangeSource<UDMMaterialStageInputTextureUV>(
		[MaterialModel](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputTextureUV>(InNewSource)->Init(MaterialModel);;
		});

	return InputTextureUV;
}

UDMMaterialStageInputTextureUV* UDMMaterialStageInputTextureUV::ChangeStageInput_UV(UDMMaterialStage* InStage, int32 InInputIdx,
	int32 InInputChannel, int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	UDMMaterialStageInputTextureUV* NewInputTextureUV = InStage->ChangeInput<UDMMaterialStageInputTextureUV>(
		InInputIdx, InInputChannel, 0, InOutputChannel, 
		[MaterialModel](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputTextureUV>(InNewInput)->Init(MaterialModel);
		}
	);

	return NewInputTextureUV;
}

FText UDMMaterialStageInputTextureUV::GetComponentDescription() const
{
	return LOCTEXT("TexureUV", "Texture UV");
}

FSlateIcon UDMMaterialStageInputTextureUV::GetComponentIcon() const
{
	return GetDefault<UDMTextureUV>()->GetComponentIcon();
}

FText UDMMaterialStageInputTextureUV::GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel)
{
	return LOCTEXT("TextureUV", "Texture UV");
}

void UDMMaterialStageInputTextureUV::Init(UDynamicMaterialModel* InMaterialModel)
{
	check(InMaterialModel);

	TextureUV = UDMTextureUV::CreateTextureUV(InMaterialModel);
	InitTextureUV();
}

void UDMMaterialStageInputTextureUV::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	TArray<UMaterialExpression*> Expressions = CreateTextureUVExpressions(InBuildState, TextureUV);

	AddEffects(InBuildState, Expressions);

	InBuildState->AddStageSourceExpressions(this, Expressions);
}

bool UDMMaterialStageInputTextureUV::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	if (TextureUV)
	{
		TextureUV->Modify(bInAlwaysMarkDirty);
	}

	return bSaved;
}

void UDMMaterialStageInputTextureUV::PostLoad()
{
	const bool bComponentValid = IsComponentValid();

	if (bComponentValid)
	{
		if (FDynamicMaterialModule::AreUObjectsSafe() && !TextureUV)
		{
			if (UDMMaterialStage* Stage = GetStage())
			{
				if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
				{
					if (UDMMaterialSlot* Slot = Layer->GetSlot())
					{
						if (UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData())
						{
							if (UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel())
							{
								Init(MaterialModel);

								if (IsComponentAdded())
								{
									TextureUV->SetComponentState(EDMComponentLifetimeState::Added);
								}
							}
						}
					}
				}
			}
		}
	}

	Super::PostLoad();

	if (bComponentValid)
	{
		InitTextureUV();
	}
}

void UDMMaterialStageInputTextureUV::PostEditImport()
{
	Super::PostEditImport();

	if (!IsComponentValid())
	{
		return;
	}

	InitTextureUV();
}

void UDMMaterialStageInputTextureUV::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel,
	UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (TextureUV)
	{
		if (GUndo)
		{
			TextureUV->Modify();
		}

		TextureUV->PostEditorDuplicate(InMaterialModel, this);
	}

	InitTextureUV();
}

UDMMaterialStageInputTextureUV::UDMMaterialStageInputTextureUV()
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageInputTextureUV, TextureUV));

	UpdateOutputConnectors();
}

void UDMMaterialStageInputTextureUV::UpdateOutputConnectors()
{
	if (!IsComponentValid())
	{
		return;
	}

	OutputConnectors.Empty();
	OutputConnectors.Add({0, LOCTEXT("UV", "UV"), EDMValueType::VT_Float2});
}

void UDMMaterialStageInputTextureUV::OnComponentAdded()
{
	if (!IsComponentValid())
	{
		return;
	}

	Super::OnComponentAdded();

	if (TextureUV)
	{
		if (GUndo)
		{
			TextureUV->Modify();
		}

		TextureUV->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialStageInputTextureUV::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	if (TextureUV)
	{
		if (GUndo)
		{
			TextureUV->Modify();
		}

		TextureUV->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

UDMMaterialComponent* UDMMaterialStageInputTextureUV::GetSubComponentByPath(FDMComponentPath& InPath,
	const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == TextureUVPathToken)
	{
		return TextureUV;
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialStageInputTextureUV::OnTextureUVUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InSource == TextureUV && EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		Update(InSource, InUpdateType);
	}
}

UMaterialExpressionScalarParameter* UDMMaterialStageInputTextureUV::CreateScalarParameter(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
	FName InParamName, EDMMaterialParameterGroup InParameterGroup, float InValue)
{
	UMaterialExpressionScalarParameter* NewExpression = InBuildState->GetBuildUtils().CreateExpressionParameter<UMaterialExpressionScalarParameter>(
		InParamName, 
		InParameterGroup,
		UE_DM_NodeComment_Default
	);

	check(NewExpression);

	NewExpression->DefaultValue = InValue;

	return NewExpression;
}

TArray<UMaterialExpression*> UDMMaterialStageInputTextureUV::CreateTextureUVExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UDMTextureUV* InTextureUV)
{
	if (InBuildState->IsIgnoringUVs())
	{
		UMaterialExpression* UVSourceExpression = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionTextureCoordinate>(UE_DM_NodeComment_Default);

		return {UVSourceExpression};
	}

	check(IsValid(InTextureUV));

	static const FString MaterialFunc_Name_TextureUV_Mirror_None = "MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_TextureUV.MF_DM_TextureUV'";
	static const FString MaterialFunc_Name_TextureUV_Mirror_X    = "MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_TextureUV_Mirror_X.MF_DM_TextureUV_Mirror_X'";
	static const FString MaterialFunc_Name_TextureUV_Mirror_Y    = "MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_TextureUV_Mirror_Y.MF_DM_TextureUV_Mirror_Y'";
	static const FString MaterialFunc_Name_TextureUV_Mirror_XY   = "MaterialFunction'/DynamicMaterial/MaterialFunctions/MF_DM_TextureUV_Mirror_XY.MF_DM_TextureUV_Mirror_XY'";

	using namespace UE::DynamicMaterial;

	UMaterialExpressionMaterialFunctionCall* TextureUVFunc = nullptr;

	if (InTextureUV->GetMirrorOnX() == false && InTextureUV->GetMirrorOnY() == false)
	{
		TextureUVFunc = FDMMaterialFunctionLibrary::Get().MakeExpression(
			InBuildState->GetDynamicMaterial(),
			"MF_DM_TextureUVFunc",
			MaterialFunc_Name_TextureUV_Mirror_None,
			UE_DM_NodeComment_Default
		);
	}
	// GetMirrorOnX() == true
	else if (InTextureUV->GetMirrorOnY() == false)
	{
		TextureUVFunc = FDMMaterialFunctionLibrary::Get().MakeExpression(
			InBuildState->GetDynamicMaterial(),
			"MF_DM_TextureUV_Mirror_X",
			MaterialFunc_Name_TextureUV_Mirror_X,
			UE_DM_NodeComment_Default
		);
	}
	// GetMirrorOnY() == true
	else if (InTextureUV->GetMirrorOnX() == false)
	{
		TextureUVFunc = FDMMaterialFunctionLibrary::Get().MakeExpression(
			InBuildState->GetDynamicMaterial(),
			"MF_DM_TextureUV_Mirror_Y",
			MaterialFunc_Name_TextureUV_Mirror_Y,
			UE_DM_NodeComment_Default
		);
	}
	// GetMirrorOnX() == true && // GetMirrorOnY() == true
	else
	{
		TextureUVFunc = FDMMaterialFunctionLibrary::Get().MakeExpression(
			InBuildState->GetDynamicMaterial(),
			"MF_DM_TextureUV_Mirror_XY",
			MaterialFunc_Name_TextureUV_Mirror_XY,
			UE_DM_NodeComment_Default
		);
	}

	TMap<FName, int32> NameToInputIndex;
	for (FExpressionInputIterator It{ TextureUVFunc }; It; ++It)
	{
		NameToInputIndex.Emplace(It->InputName, It.Index);
	}

	// Output nodes
	TArray<UMaterialExpression*> Nodes;

	// UV Source
	static const FName UVInputName = TEXT("UV");
	check(NameToInputIndex.Contains(UVInputName));
	TSubclassOf<UMaterialExpression> UVSourceClass = nullptr;

	switch (InTextureUV->GetUVSource())
	{
		case EDMUVSource::Texture:
			UVSourceClass = UMaterialExpressionTextureCoordinate::StaticClass();
			break;

		case EDMUVSource::ScreenPosition:
			UVSourceClass = UDMMaterialStageExpression::FindClass(TEXT("MaterialExpressionScreenPosition"));
			break;

		case EDMUVSource::WorldPosition:
			UVSourceClass = UMaterialExpressionWorldPosition::StaticClass();
			break;

		default:
			checkNoEntry();
			break;
	}

	check(UVSourceClass.Get());
	UMaterialExpression* UVSourceNode = InBuildState->GetBuildUtils().CreateExpression(UVSourceClass.Get(), UE_DM_NodeComment_Default);

	Nodes.Add(UVSourceNode);

	if (InTextureUV->GetUVSource() == EDMUVSource::WorldPosition)
	{
		UMaterialExpression* UVSourceNodeMask = InBuildState->GetBuildUtils().CreateExpressionBitMask(UVSourceNode, 0,
			FDMMaterialStageConnectorChannel::SECOND_CHANNEL + FDMMaterialStageConnectorChannel::THIRD_CHANNEL); // Y and Z

		Nodes.Add(UVSourceNodeMask);
	}

	Nodes.Last()->ConnectExpression(TextureUVFunc->GetInput(NameToInputIndex[UVInputName]), 0);

	auto CreateParameterExpression = [InTextureUV, &InBuildState, &NameToInputIndex, TextureUVFunc, &Nodes]
		(FName InPropertyName, int32 InComponent, FName InInputName, float InDefaultValue)
		{
			UMaterialExpressionScalarParameter* ParamNode = CreateScalarParameter(
				InBuildState, 
				InTextureUV->GetMaterialParameterName(InPropertyName, InComponent),
				InTextureUV->GetParameterGroup(InPropertyName, InComponent)
			);

			ParamNode->DefaultValue = InDefaultValue;
			ParamNode->ConnectExpression(TextureUVFunc->GetInput(NameToInputIndex[InInputName]), 0);

			Nodes.Add(ParamNode);
		};
	CreateParameterExpression(UDMTextureUV::NAME_Offset,   0, TEXT("OffsetX"),  InTextureUV->GetOffset().X);
	CreateParameterExpression(UDMTextureUV::NAME_Offset,   1, TEXT("OffsetY"),  InTextureUV->GetOffset().Y);
	CreateParameterExpression(UDMTextureUV::NAME_Pivot,    0, TEXT("PivotX"),   InTextureUV->GetPivot().X);
	CreateParameterExpression(UDMTextureUV::NAME_Pivot,    1, TEXT("PivotY"),   InTextureUV->GetPivot().Y);
	CreateParameterExpression(UDMTextureUV::NAME_Rotation, 0, TEXT("Rotation"), InTextureUV->GetRotation());
	CreateParameterExpression(UDMTextureUV::NAME_Tiling,   0, TEXT("TilingX"),  InTextureUV->GetTiling().X);
	CreateParameterExpression(UDMTextureUV::NAME_Tiling,   1, TEXT("TilingY"),  InTextureUV->GetTiling().Y);

	if (UMaterialExpression* GlobalOffset = InBuildState->GetGlobalExpression(UDynamicMaterialModel::GlobalOffsetValueName))
	{
		check(NameToInputIndex.Contains(UDynamicMaterialModel::GlobalOffsetParameterName));
		GlobalOffset->ConnectExpression(TextureUVFunc->GetInput(NameToInputIndex[UDynamicMaterialModel::GlobalOffsetParameterName]), 0);
	}

	if (UMaterialExpression* GlobalTiling = InBuildState->GetGlobalExpression(UDynamicMaterialModel::GlobalTilingValueName))
	{
		check(NameToInputIndex.Contains(UDynamicMaterialModel::GlobalTilingParameterName));
		GlobalTiling->ConnectExpression(TextureUVFunc->GetInput(NameToInputIndex[UDynamicMaterialModel::GlobalTilingParameterName]), 0);
	}

	if (UMaterialExpression* GlobalRotation = InBuildState->GetGlobalExpression(UDynamicMaterialModel::GlobalRotationValueName))
	{
		check(NameToInputIndex.Contains(UDynamicMaterialModel::GlobalRotationParameterName));
		GlobalRotation->ConnectExpression(TextureUVFunc->GetInput(NameToInputIndex[UDynamicMaterialModel::GlobalRotationParameterName]), 0);
	}

	// Output
	Nodes.Add(TextureUVFunc);

	return Nodes;
}

void UDMMaterialStageInputTextureUV::InitTextureUV()
{
	if (TextureUV)
	{
		if (GUndo)
		{
			TextureUV->Modify();
		}

		TextureUV->SetParentComponent(this);
		TextureUV->GetOnUpdate().AddUObject(this, &UDMMaterialStageInputTextureUV::OnTextureUVUpdated);
	}
}

void UDMMaterialStageInputTextureUV::AddEffects(const TSharedRef<FDMMaterialBuildState>& InBuildState, TArray<UMaterialExpression*>& InOutExpressions) const
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();

	if (!Layer)
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	int32 Channel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
	int32 OutputIndex = 0;
	EffectStack->ApplyEffects(InBuildState, EDMMaterialEffectTarget::TextureUV, InOutExpressions, Channel, OutputIndex);
}

#undef LOCTEXT_NAMESPACE
