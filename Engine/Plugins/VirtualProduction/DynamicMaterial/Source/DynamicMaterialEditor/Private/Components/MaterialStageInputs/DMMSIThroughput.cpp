// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageThroughput.h"
#include "Components/DMMaterialSubStage.h"
#include "DMComponentPath.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSIThroughput)

#define LOCTEXT_NAMESPACE "DMMaterialStageInputThroughput"

const FString UDMMaterialStageInputThroughput::SubStagePathToken = FString(TEXT("SubStage"));

void UDMMaterialStageInputThroughput::SetMaterialStageThroughputClass(TSubclassOf<UDMMaterialStageThroughput> InMaterialStageThroughputClass)
{
	if (GetMaterialStageThroughputClass() == InMaterialStageThroughputClass)
	{
		return;
	}

	OutputConnectors.Empty();

	if (SubStage)
	{
		if (GUndo)
		{
			SubStage->Modify();
		}

		SubStage->SetParentComponent(nullptr);
		SubStage->GetOnUpdate().RemoveAll(this);
		SubStage->SetComponentState(EDMComponentLifetimeState::Removed);
		SubStage = nullptr;
	}

	if (InMaterialStageThroughputClass.Get())
	{
		SubStage = UDMMaterialSubStage::CreateMaterialSubStage(GetStage());
		check(SubStage);

		InitSubStage();

		UDMMaterialStageThroughput* Throughput = NewObject<UDMMaterialStageThroughput>(SubStage, InMaterialStageThroughputClass, NAME_None, RF_Transactional);
		check(Throughput);

		SubStage->SetSource(Throughput);

		OutputConnectors = Throughput->GetOutputConnectors();

		if (IsComponentAdded())
		{
			SubStage->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}

	Update(this, EDMUpdateType::Structure);
}

UDMMaterialStageThroughput* UDMMaterialStageInputThroughput::GetMaterialStageThroughput() const
{
	if (SubStage)
	{
		return Cast<UDMMaterialStageThroughput>(SubStage->GetSource());
	}

	return nullptr;
}

void UDMMaterialStageInputThroughput::OnComponentAdded()
{
	UDMMaterialStageInput::OnComponentAdded();

	if (IsComponentValid() && SubStage)
	{
		SubStage->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialStageInputThroughput::OnComponentRemoved()
{	
	Super::OnComponentRemoved();

	if (SubStage)
	{
		if (GUndo)
		{
			SubStage->Modify();
		}

		SubStage->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

void UDMMaterialStageInputThroughput::GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const
{
	// Strip off the type index of the substage
	if (OutChildComponentPathComponents.IsEmpty() == false && SubStage)
	{
		if (OutChildComponentPathComponents.Last() == SubStage->GetComponentPathComponent())
		{
			OutChildComponentPathComponents.Last() = FString(TEXT("SubStage"));
		}
	}

	Super::GetComponentPathInternal(OutChildComponentPathComponents);
}

UDMMaterialComponent* UDMMaterialStageInputThroughput::GetSubComponentByPath(FDMComponentPath& InPath,
	const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == SubStagePathToken)
	{
		return SubStage;
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

void UDMMaterialStageInputThroughput::InitSubStage()
{
	if (SubStage)
	{
		if (GUndo)
		{
			SubStage->Modify();
		}

		SubStage->SetParentComponent(this);
		SubStage->GetOnUpdate().AddUObject(this, &UDMMaterialStageInputThroughput::OnSubStageUpdated);
	}
}

void UDMMaterialStageInputThroughput::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	check(SubStage);

	if (InBuildState->HasStageSource(this))
	{
		return;
	}

	SubStage->GenerateExpressions(InBuildState);
	InBuildState->AddStageSourceExpressions(this, InBuildState->GetStageExpressions(SubStage));
}

int32 UDMMaterialStageInputThroughput::GetInnateMaskOutput(int32 OutputIndex, int32 OutputChannels) const
{
	if (UDMMaterialStageThroughput* Throughput = GetMaterialStageThroughput())
	{
		return Throughput->GetInnateMaskOutput(OutputIndex, OutputChannels);
	}

	return Super::GetInnateMaskOutput(OutputIndex, OutputChannels);
}

int32 UDMMaterialStageInputThroughput::GetOutputChannelOverride(int32 InOutputIndex) const
{
	if (UDMMaterialStageThroughput* Throughput = GetMaterialStageThroughput())
	{
		return Throughput->GetOutputChannelOverride(InOutputIndex);
	}

	return Super::GetOutputChannelOverride(InOutputIndex);
}

bool UDMMaterialStageInputThroughput::IsPropertyVisible(FName Property) const
{
	if (UDMMaterialStageThroughput* Throughput = GetMaterialStageThroughput())
	{
		return Throughput->IsPropertyVisible(Property);
	}

	return Super::IsPropertyVisible(Property);
}

bool UDMMaterialStageInputThroughput::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	if (SubStage)
	{
		SubStage->Modify(bInAlwaysMarkDirty);
	}

	return bSaved;
}

void UDMMaterialStageInputThroughput::PostLoad()
{
	Super::PostLoad();

	if (!IsComponentValid())
	{
		return;
	}

	InitSubStage();
}

void UDMMaterialStageInputThroughput::PostEditImport()
{
	Super::PostEditImport();

	if (!IsComponentValid())
	{
		return;
	}

	InitSubStage();
}

void UDMMaterialStageInputThroughput::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel,
	UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (SubStage)
	{
		if (GUndo)
		{
			SubStage->Modify();
		}

		SubStage->PostEditorDuplicate(InMaterialModel, this);
	}

	InitSubStage();
}

FText UDMMaterialStageInputThroughput::GetComponentDescription() const
{
	if (UDMMaterialStageThroughput* Throughput = GetMaterialStageThroughput())
	{
		return Throughput->GetComponentDescription();
	}

	if (TSubclassOf<UDMMaterialStageThroughput> MaterialStageThroughputClass = GetMaterialStageThroughputClass())
	{
		check(MaterialStageThroughputClass.Get());
		UDMMaterialStageThroughput* ThroughputCDO = Cast<UDMMaterialStageThroughput>(MaterialStageThroughputClass->GetDefaultObject(/* Create if needed */ true));

		return ThroughputCDO->GetComponentDescription();
	}

	return Super::GetComponentDescription();
}

FSlateIcon UDMMaterialStageInputThroughput::GetComponentIcon() const
{
	if (UDMMaterialStageThroughput* Throughput = GetMaterialStageThroughput())
	{
		return Throughput->GetComponentIcon();
	}

	if (TSubclassOf<UDMMaterialStageThroughput> MaterialStageThroughputClass = GetMaterialStageThroughputClass())
	{
		check(MaterialStageThroughputClass.Get());
		UDMMaterialStageThroughput* ThroughputCDO = Cast<UDMMaterialStageThroughput>(MaterialStageThroughputClass->GetDefaultObject(/* Create if needed */ true));

		return ThroughputCDO->GetComponentIcon();
	}

	return Super::GetComponentIcon();
}

FText UDMMaterialStageInputThroughput::GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel)
{
	if (TSubclassOf<UDMMaterialStageThroughput> MaterialStageThroughputClass = GetMaterialStageThroughputClass())
	{
		UDMMaterialStageThroughput* ThroughputCDO = Cast<UDMMaterialStageThroughput>(MaterialStageThroughputClass->GetDefaultObject(true));

		return ThroughputCDO->GetDescription();
	}

	return FText::GetEmpty();
}

TSubclassOf<UDMMaterialStageThroughput> UDMMaterialStageInputThroughput::GetMaterialStageThroughputClass() const
{
	if (UDMMaterialStageThroughput* Throughput = GetMaterialStageThroughput())
	{
		return Throughput->GetClass();
	}

	return nullptr;
}

UDMMaterialStageInputThroughput::UDMMaterialStageInputThroughput()
	: SubStage(nullptr)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageInputThroughput, SubStage));
}

void UDMMaterialStageInputThroughput::OnSubStageUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (InSource == SubStage)
	{
		Update(InSource, InUpdateType);
	}
}

#undef LOCTEXT_NAMESPACE
