// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MaterialStageInputs/DMMSIValue.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialParameter.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DMComponentPath.h"
#include "DynamicMaterialEditorSettings.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMSIValue)

#define LOCTEXT_NAMESPACE "DMMaterialStageInputValue"

const FString UDMMaterialStageInputValue::ValuePathToken = FString(TEXT("Value"));

UDMMaterialStage* UDMMaterialStageInputValue::CreateStage(UDMMaterialValue* InValue, UDMMaterialLayerObject* InLayer)
{
	check(InValue);

	const FDMUpdateGuard Guard;

	UDMMaterialStage* NewStage = UDMMaterialStage::CreateMaterialStage(InLayer);

	UDMMaterialStageInputValue* InputValue = NewObject<UDMMaterialStageInputValue>(NewStage, NAME_None, RF_Transactional);
	check(InputValue);
	InputValue->SetValue(InValue);

	NewStage->SetSource(InputValue);

	return NewStage;
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageSource_NewLocalValue(UDMMaterialStage* InStage, EDMValueType InValueType)
{
	return ChangeStageSource_NewLocalValue(
		InStage,
		UDMValueDefinitionLibrary::GetValueDefinition(InValueType).GetValueClass()
	);
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageSource_NewLocalValue(UDMMaterialStage* InStage, TSubclassOf<UDMMaterialValue> InValueClass)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	// Don't add via the builder, just parent it to the builder.
	// It won't appear in the global value list.
	UDMMaterialValue* NewValue = UDMMaterialValue::CreateMaterialValue(MaterialModel, TEXT(""), InValueClass, true);
	check(NewValue);

	UDMMaterialStageInputValue* InputValue = InStage->ChangeSource<UDMMaterialStageInputValue>(
		[NewValue](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputValue>(InNewSource)->SetValue(NewValue);
		});

	return InputValue;
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageSource_Value(UDMMaterialStage* InStage, UDMMaterialValue* InValue)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	UDMMaterialStageInputValue* InputValue = InStage->ChangeSource<UDMMaterialStageInputValue>(
		[InValue](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputValue>(InNewSource)->SetValue(InValue);
		});

	return InputValue;
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageSource_NewValue(UDMMaterialStage* InStage, EDMValueType InValueType)
{
	return ChangeStageSource_NewValue(
		InStage,
		UDMValueDefinitionLibrary::GetValueDefinition(InValueType).GetValueClass()
	);
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageSource_NewValue(UDMMaterialStage* InStage, 
	TSubclassOf<UDMMaterialValue> InValueClass)
{
	check(InStage);

	if (!InStage->CanChangeSource())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	UDMMaterialValue* NewValue = MaterialModel->AddValue(InValueClass);
	check(NewValue);

	UDMMaterialStageInputValue* InputValue = InStage->ChangeSource<UDMMaterialStageInputValue>(
		[NewValue](UDMMaterialStage* InStage, UDMMaterialStageSource* InNewSource)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputValue>(InNewSource)->SetValue(NewValue);
		});

	return InputValue;
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(UDMMaterialStage* InStage, int32 InInputIdx, 
	int32 InInputChannel, EDMValueType InValueType, int32 InOutputChannel)
{
	return ChangeStageInput_NewLocalValue(
		InStage,
		InInputIdx,
		InInputChannel,
		UDMValueDefinitionLibrary::GetValueDefinition(InValueType).GetValueClass(),
		InOutputChannel
	);
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(UDMMaterialStage* InStage,
	int32 InInputIdx, int32 InInputChannel, TSubclassOf<UDMMaterialValue> InValueClass, int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));
	check(Throughput->CanInputAcceptType(InInputIdx, InValueClass.GetDefaultObject()->GetType()));

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	// Don't add via the builder, just parent it to the builder.
	// It won't appear in the global value list.
	UDMMaterialValue* NewValue = UDMMaterialValue::CreateMaterialValue(MaterialModel, TEXT(""), InValueClass, true);
	check(NewValue);

	UDMMaterialStageInputValue* NewInputValue = InStage->ChangeInput<UDMMaterialStageInputValue>(
		InInputIdx, InInputChannel, 0, InOutputChannel,
		[NewValue](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputValue>(InNewInput)->SetValue(NewValue);
		}
	);

	NewInputValue->ApplyDefaultLayerSettings();

	NewInputValue->ApplyWholeLayerValue();

	return NewInputValue;
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageInput_Value(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel, UDMMaterialValue* InValue, int32 OutputChannel)
{
	check(InStage);
	check(InValue);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));

	if (OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		check(Throughput->CanInputAcceptType(InInputIdx, InValue->GetType()));
	}
	else
	{
		check(UDMValueDefinitionLibrary::GetValueDefinition(InValue->GetType()).IsFloatType());
		check(Throughput->CanInputAcceptType(InInputIdx, EDMValueType::VT_Float1));
	}

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);
	check(MaterialModel == InValue->GetMaterialModel());

	UDMMaterialStageInputValue* NewInputValue = InStage->ChangeInput<UDMMaterialStageInputValue>(
		InInputIdx, InInputChannel, 0, OutputChannel,
		[InValue](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputValue>(InNewInput)->SetValue(InValue);
		}
	);

	NewInputValue->ApplyWholeLayerValue();

	return NewInputValue;
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageInput_NewValue(UDMMaterialStage* InStage, int32 InInputIdx, 
	int32 InInputChannel, EDMValueType InValueType, int32 InOutputChannel)
{
	return ChangeStageInput_NewValue(
		InStage,
		InInputIdx,
		InInputChannel,
		UDMValueDefinitionLibrary::GetValueDefinition(InValueType).GetValueClass(),
		InOutputChannel
	);
}

UDMMaterialStageInputValue* UDMMaterialStageInputValue::ChangeStageInput_NewValue(UDMMaterialStage* InStage, int32 InInputIdx, 
	int32 InInputChannel, TSubclassOf<UDMMaterialValue> InValueClass, int32 InOutputChannel)
{
	check(InStage);

	UDMMaterialStageSource* Source = InStage->GetSource();
	check(Source);

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(Source);
	check(Throughput);

	const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
	check(InputConnectors.IsValidIndex(InInputIdx));
	check(Throughput->CanInputAcceptType(InInputIdx, InValueClass.GetDefaultObject()->GetType()));

	UDMMaterialLayerObject* Layer = InStage->GetLayer();
	check(Layer);

	UDMMaterialSlot* Slot = Layer->GetSlot();
	check(Slot);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	check(ModelEditorOnlyData);

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();
	check(MaterialModel);

	UDMMaterialValue* NewValue = MaterialModel->AddValue(InValueClass);
	check(NewValue);

	UDMMaterialStageInputValue* NewInputValue = InStage->ChangeInput<UDMMaterialStageInputValue>(
		InInputIdx, InInputChannel, 0, InOutputChannel,
		[NewValue](UDMMaterialStage* InStage, UDMMaterialStageInput* InNewInput)
		{
			const FDMUpdateGuard Guard;
			CastChecked<UDMMaterialStageInputValue>(InNewInput)->SetValue(NewValue);
		}
	);

	NewInputValue->ApplyWholeLayerValue();

	return NewInputValue;
}

FText UDMMaterialStageInputValue::GetComponentDescription() const
{
	if (!Value->IsComponentValid())
	{
		return FText::GetEmpty();
	}

	if (Value->GetParameter())
	{
		if (Value->IsLocal())
		{
			static const FText Template = LOCTEXT("ComponentDescriptionLocalNamed", "{0}");

			return FText::Format(Template, Value->GetTypeName());

		}
		else
		{
			static const FText Template = LOCTEXT("ComponentDescriptionGlobalNamed", "{0} (Global)");

			return FText::Format(Template, Value->GetTypeName());
		}
	}
	else
	{
		if (Value->IsLocal())
		{
			static const FText Template = LOCTEXT("ComponentDescriptionLocal", "{0}");

			return FText::Format(Template, Value->GetTypeName());

		}
		else
		{
			static const FText Template = LOCTEXT("ComponentDescriptionGlobal", "{0} (Global)");

			return FText::Format(Template, Value->GetTypeName());
		}
	}
}

FSlateIcon UDMMaterialStageInputValue::GetComponentIcon() const
{
	if (Value && Value->IsComponentValid())
	{
		return Value->GetComponentIcon();
	}

	return Super::GetComponentIcon();
}

FText UDMMaterialStageInputValue::GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel)
{
	if (!Value->IsComponentValid())
	{
		return FText::GetEmpty();
	}

	if (Value->IsLocal())
	{
		return LOCTEXT("LocalValue", "Local Value");
	}

	return FText::Format(
		LOCTEXT("ChannelDescriptionGlobal", "Value {0} (Global)"),
		FText::FromName(Value->GetMaterialParameterName())
	);
}

void UDMMaterialStageInputValue::SetValue(UDMMaterialValue* InValue)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value == InValue)
	{
		return;
	}

	if (InValue)
	{
		if (UDMMaterialStage* Stage = GetStage())
		{
			if (UDMMaterialLayerObject* Layer = Stage->GetLayer())
			{
				if (UDMMaterialSlot* Slot = Layer->GetSlot())
				{
					if (UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData())
					{
						if (ModelEditorOnlyData->GetMaterialModel() != InValue->GetMaterialModel())
						{
							return;
						}
					}
				}
			}
		}
	}

	if (Value)
	{
		DeinitInputValue();

		if (!IsSharedStageValue())
		{
			if (GUndo)
			{
				Value->Modify();
			}

			Value->SetComponentState(EDMComponentLifetimeState::Removed);
		}
	}

	Value = InValue;

	if (Value)
	{
		InitInputValue();

		if (IsComponentAdded())
		{
			if (GUndo)
			{
				Value->Modify();
			}

			Value->SetComponentState(EDMComponentLifetimeState::Added);
		}
	}

	UpdateOutputConnectors();

	Update(this, EDMUpdateType::Structure);
}

void UDMMaterialStageInputValue::GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
{
	if (!IsComponentValid() || !IsComponentAdded())
	{
		return;
	}

	if (InBuildState->HasStageSource(this))
	{
		return;
	}
	
	if (!Value)
	{
		return;
	}

	Value->GenerateExpression(InBuildState);
	InBuildState->AddStageSourceExpressions(this, InBuildState->GetValueExpressions(Value));
}

int32 UDMMaterialStageInputValue::GetInnateMaskOutput(int32 OutputIndex, int32 OutputChannels) const
{
	if (OutputIndex == 0 && Value)
	{
		return Value->GetInnateMaskOutput(OutputChannels);
	}

	return UDMMaterialStageSource::GetInnateMaskOutput(OutputIndex, OutputChannels);
}

void UDMMaterialStageInputValue::ApplyDefaultLayerSettings()
{
	if (!Value)
	{
		return;
	}

	const EDMValueType ValueType = Value->GetType();

	switch (ValueType)
	{
		case EDMValueType::VT_Float3_RGB:
		case EDMValueType::VT_Float4_RGBA:
		case EDMValueType::VT_Texture:
			break;

		// Only the above 3 are supported.
		default:
			return;
	}

	const UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = GetTypedParent<UDMMaterialLayerObject>(/* Allow Subclasses */ true);

	if (!Layer)
	{
		return;
	}

	const EDMMaterialPropertyType Property = Layer->GetMaterialProperty();

	if (Property == EDMMaterialPropertyType::None)
	{
		return;
	}

	const FDMDefaultMaterialPropertySlotValue& DefaultSlotValue = Settings->GetDefaultSlotValue(Property);

	switch (ValueType)
	{
		case EDMValueType::VT_Float3_RGB:
		{
			UDMMaterialValueFloat3RGB* Float3Value = Cast<UDMMaterialValueFloat3RGB>(Value);
			Float3Value->SetDefaultValue(DefaultSlotValue.Color);
			Float3Value->ApplyDefaultValue();
			break;
		}

		case EDMValueType::VT_Float4_RGBA:
		{
			UDMMaterialValueFloat4* Float4Value = Cast<UDMMaterialValueFloat4>(Value);
			Float4Value->SetDefaultValue(DefaultSlotValue.Color);
			Float4Value->ApplyDefaultValue();
			break;
		}

		case EDMValueType::VT_Texture:
		{
			if (UTexture* Texture = DefaultSlotValue.Texture.LoadSynchronous())
			{
				UDMMaterialValueTexture* TextureValue = Cast<UDMMaterialValueTexture>(Value);

				if (TextureValue->GetClass() == UDMMaterialValueTexture::StaticClass() || !TextureValue->GetDefaultValue())
				{
					TextureValue->SetDefaultValue(Texture);
					TextureValue->ApplyDefaultValue();
				}
			}
			break;
		}
	}
}

bool UDMMaterialStageInputValue::Modify(bool bInAlwaysMarkDirty)
{
	const bool bSaved = Super::Modify(bInAlwaysMarkDirty);

	if (Value)
	{
		Value->Modify(bInAlwaysMarkDirty);
	}

	return bSaved;
}

void UDMMaterialStageInputValue::OnComponentAdded()
{
	Super::OnComponentAdded();

	if (!IsComponentValid())
	{
		return;
	}

	if (Value)
	{
		if (GUndo)
		{
			Value->Modify();
		}

		Value->SetComponentState(EDMComponentLifetimeState::Added);
	}
}

void UDMMaterialStageInputValue::OnComponentRemoved()
{
	Super::OnComponentRemoved();

	if (Value && !IsSharedStageValue())
	{
		if (GUndo)
		{
			Value->Modify();
		}

		Value->SetComponentState(EDMComponentLifetimeState::Removed);
	}
}

UDMMaterialComponent* UDMMaterialStageInputValue::GetSubComponentByPath(FDMComponentPath& InPath,
	const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == ValuePathToken)
	{
		return Value;
	}

	return Super::GetSubComponentByPath(InPath, InPathSegment);
}

bool UDMMaterialStageInputValue::IsSharedStageValue() const
{
	if (!Value)
	{
		return false;
	}

	UDMMaterialStage* MyStage = GetTypedParent<UDMMaterialStage>(/* bInAllowSubclasses */ true);

	if (!MyStage)
	{
		return false;
	}

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(MyStage))
	{
		MyStage = SubStage->GetParentMostStage();
	}

	UDMMaterialLayerObject* Layer = MyStage->GetLayer();

	if (!Layer)
	{
		return false;
	}

	bool bIsBase = Layer->GetStageType(MyStage) != EDMMaterialLayerStage::Mask;

	UDMMaterialStage* OtherStage = Layer->GetStage(bIsBase ? EDMMaterialLayerStage::Mask : EDMMaterialLayerStage::Base);

	if (!OtherStage)
	{
		return false;
	}

	for (UDMMaterialStageInput* OtherInput : OtherStage->GetInputs())
	{
		if (this == OtherInput)
		{
			return true;
		}

		if (UDMMaterialStageInputValue* OtherInputValue = Cast<UDMMaterialStageInputValue>(OtherInput))
		{
			if (Value == OtherInputValue->GetValue())
			{
				return true;
			}
		}
	}

	return false;
}

void UDMMaterialStageInputValue::InitInputValue()
{
	if (Value)
	{
		if (Value->IsLocal())
		{
			if (GUndo)
			{
				Value->Modify();
			}

			Value->SetParentComponent(this);
		}
	}
}

void UDMMaterialStageInputValue::DeinitInputValue()
{
	if (Value)
	{
		if (Value->IsLocal())
		{
			if (GUndo)
			{
				Value->Modify();
			}

			Value->SetParentComponent(nullptr);
		}
	}
}

void UDMMaterialStageInputValue::PostLoad()
{
	Super::PostLoad();

	if (!IsComponentValid())
	{
		return;
	}

	InitInputValue();
}

void UDMMaterialStageInputValue::PostEditImport()
{
	Super::PostEditImport();

	if (!IsComponentValid())
	{
		return;
	}

	InitInputValue();
}

void UDMMaterialStageInputValue::PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent)
{
	Super::PostEditorDuplicate(InMaterialModel, InParent);

	if (Value)
	{
		if (GUndo)
		{
			Value->Modify();
		}

		Value->PostEditorDuplicate(InMaterialModel, this);
	}

	InitInputValue();
}

bool UDMMaterialStageInputValue::IsPropertyVisible(FName InProperty) const
{
	if (InProperty == GET_MEMBER_NAME_CHECKED(UDMMaterialStageInputValue, Value))
	{
		return IsTextureValueVisible();
	}

	return Super::IsPropertyVisible(InProperty);
}

UDMMaterialStageInputValue::UDMMaterialStageInputValue()
	: Value(nullptr)
{
	EditableProperties.Add(GET_MEMBER_NAME_CHECKED(UDMMaterialStageInputValue, Value));
}

void UDMMaterialStageInputValue::OnValueUpdated(UDMMaterialComponent* InComponent, EDMUpdateType InUpdateType)
{
	if (!IsComponentValid())
	{
		return;
	}

	if (Value != InComponent)
	{
		return;
	}

	Update(InComponent, EDMUpdateType::Value);
}

void UDMMaterialStageInputValue::UpdateOutputConnectors()
{
	OutputConnectors.Empty();
	OutputConnectors.Add({0, LOCTEXT("MaterialValue", "Value"), Value ? Value->GetType() : EDMValueType::VT_None});
}

void UDMMaterialStageInputValue::ApplyWholeLayerValue()
{
	if (!Value.Get() || !Value->IsWholeLayerValue())
	{
		return;
	}

	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();

	if (!Layer)
	{
		return;
	}

	if (!Layer
		|| Layer->GetStageType(Stage) != EDMMaterialLayerStage::Base
		|| !Layer->IsStageEnabled(EDMMaterialLayerStage::Mask))
	{
		return;
	}

	UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask);

	if (!IsValid(MaskStage))
	{
		return;
	}

	if (GUndo)
	{
		MaskStage->Modify();
	}

	ChangeStageInput_Value(
		MaskStage,
		UDMMaterialStageThroughputLayerBlend::InputMaskSource,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		Value.Get(),
		FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
	);
}

bool UDMMaterialStageInputValue::IsTextureValueVisible() const
{
	UDMMaterialValueTexture* TextureValue = Cast<UDMMaterialValueTexture>(Value);

	if (!TextureValue)
	{
		return true;
	}

	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return true;
	}

	if (UDMMaterialSubStage* SubStage = Cast<UDMMaterialSubStage>(Stage))
	{
		Stage = SubStage->GetParentMostStage();

		if (!Stage)
		{
			return true;
		}
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();

	if (!Layer)
	{
		return true;
	}

	UDMMaterialStage* MaskStage = Layer->GetFirstEnabledStage(EDMMaterialLayerStage::Mask);

	if (MaskStage != Stage)
	{
		return true;
	}

	UDMMaterialStageThroughputLayerBlend* LayerBlend = Cast<UDMMaterialStageThroughputLayerBlend>(Stage->GetSource());

	if (!LayerBlend)
	{
		return true;
	}

	UDMMaterialStageInputExpression* InputExpression = Cast<UDMMaterialStageInputExpression>(LayerBlend->GetInputMask());

	if (!InputExpression)
	{
		return true;
	}

	UDMMaterialStageExpressionTextureSample* TextureSample = Cast<UDMMaterialStageExpressionTextureSample>(InputExpression->GetMaterialStageExpression());

	if (!TextureSample)
	{
		return true;
	}

	return !TextureSample->GetUseBaseTexture() || !TextureSample->CanUseBaseTexture();
}

#undef LOCTEXT_NAMESPACE
