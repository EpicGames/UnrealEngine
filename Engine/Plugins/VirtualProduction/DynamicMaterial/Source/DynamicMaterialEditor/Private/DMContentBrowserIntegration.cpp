// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMContentBrowserIntegration.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "CoreGlobals.h"
#include "DMTextureSet.h"
#include "DMTextureSetBlueprintFunctionLibrary.h"
#include "DMTextureSetContentBrowserIntegration.h"
#include "Engine/Texture.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Materials/Material.h"
#include "Misc/MessageDialog.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "PackageTools.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Utils/DMTextureSetFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "FDMContentBrowserIntegration"

FDelegateHandle FDMContentBrowserIntegration::TextureSetPopulateHandle;
FDelegateHandle FDMContentBrowserIntegration::ContentBrowserAssetHandle;

void FDMContentBrowserIntegration::Integrate()
{
	Disintegrate();

	TextureSetPopulateHandle = FDMTextureSetContentBrowserIntegration::GetPopulateExtenderDelegate().AddStatic(&FDMContentBrowserIntegration::ExtendMenu);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FDMContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserAssetHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FDMContentBrowserIntegration::Disintegrate()
{
	if (TextureSetPopulateHandle.IsValid())
	{
		FDMTextureSetContentBrowserIntegration::GetPopulateExtenderDelegate().Remove(TextureSetPopulateHandle);
		TextureSetPopulateHandle.Reset();
	}

	if (ContentBrowserAssetHandle.IsValid())
	{
		if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
		{
			TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule->GetAllAssetViewContextMenuExtenders();

			CBMenuExtenderDelegates.RemoveAll(
				[](const FContentBrowserMenuExtender_SelectedAssets& InElement)
				{
					return InElement.GetHandle() == ContentBrowserAssetHandle;
				});

			ContentBrowserAssetHandle.Reset();
		}
	}
}

void FDMContentBrowserIntegration::ExtendMenu(FMenuBuilder& InMenuBuilder, const TArray<FAssetData>& InSelectedAssets)
{
	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreateMaterialDesignerMaterialFromTextureSet", "Create Material Designer Material"),
		LOCTEXT("CreateMaterialDesignerInstanceFromTextureSetTooltip", "Creates a Material Designer Material in the content browser using a Texture Set."),
		FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
		FUIAction(FExecuteAction::CreateStatic(&FDMContentBrowserIntegration::CreateMaterialDesignerMaterialFromTextureSet, InSelectedAssets))
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("UpdateMaterialDesignerInstanceFromTextureSetAdd", "Update Material Designer Material (Add)"),
		LOCTEXT("UpdateMaterialDesignerInstanceFromTextureAddSetTooltip", "Updates the opened Material Designer Material using a Texture Set, adding new layers to the Material Model."),
		FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
		FUIAction(FExecuteAction::CreateStatic(&FDMContentBrowserIntegration::UpdateMaterialDesignerMaterialFromTextureSet, InSelectedAssets, /* Replace */ false))
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("UpdateMaterialDesignerInstanceFromTextureSetReplace", "Update Material Designer Material (Replace)"),
		LOCTEXT("UpdateMaterialDesignerInstanceFromTextureSetReplaceTooltip", "Updates the opened Material Designer Material using a Texture Set, replacing layers in the Material Model."),
		FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
		FUIAction(FExecuteAction::CreateStatic(&FDMContentBrowserIntegration::UpdateMaterialDesignerMaterialFromTextureSet, InSelectedAssets, /* Replace */ true))
	);
}

void FDMContentBrowserIntegration::CreateMaterialDesignerMaterialFromTextureSet(TArray<FAssetData> InSelectedAssets)
{
	if (InSelectedAssets.IsEmpty())
	{
		return;
	}

	UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssetsInteractive(
		InSelectedAssets,
		FDMTextureSetBuilderOnComplete::CreateStatic(
			&FDMContentBrowserIntegration::OnCreateMaterialDesignerMaterialFromTextureSetComplete,
			InSelectedAssets[0].PackagePath.ToString()
		)
	);
}

void FDMContentBrowserIntegration::OnCreateMaterialDesignerMaterialFromTextureSetComplete(UDMTextureSet* InTextureSet, bool bInAccepted,
	FString InPath)
{
	if (!InTextureSet || !bInAccepted)
	{
		return;
	}

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(GetMutableDefault<UDynamicMaterialInstanceFactory>()->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(),
		GetTransientPackage(),
		NAME_None,
		RF_Transactional,
		/* Context */ nullptr,
		GWarn
	));

	if (!NewInstance)
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
		MaterialDesignerModule.OpenMaterial(NewInstance, nullptr, /* Invoke Tab */ true);
	};

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(NewInstance);

	if (!EditorOnlyData)
	{
		return;
	}

	EditorOnlyData->SetChannelListPreset("All");

	if (!UDMTextureSetFunctionLibrary::AddTextureSetToModel(EditorOnlyData, InTextureSet, /* Replace */ true))
	{
		return;
	}

	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	FString UniquePackageName;
	FString UniqueAssetName;

	const FString BasePackageName = InPath / TEXT("MD_NewMaterial");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), UniquePackageName, UniqueAssetName);

	UPackage* Package = CreatePackage(*UniquePackageName);

	if (!Package)
	{
		return;
	}

	NewInstance->SetFlags(RF_Standalone | RF_Public);
	NewInstance->Rename(*UniqueAssetName, Package, REN_DontCreateRedirectors);

	FAssetRegistryModule::AssetCreated(NewInstance);
}

void FDMContentBrowserIntegration::UpdateMaterialDesignerMaterialFromTextureSet(TArray<FAssetData> InSelectedAssets, bool bInReplace)
{
	if (InSelectedAssets.IsEmpty())
	{
		return;
	}

	UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssetsInteractive(
		InSelectedAssets,
		FDMTextureSetBuilderOnComplete::CreateStatic(
			&FDMContentBrowserIntegration::OnUpdateMaterialDesignerMaterialFromTextureSetComplete,
			bInReplace
		)
	);
}

void FDMContentBrowserIntegration::OnUpdateMaterialDesignerMaterialFromTextureSetComplete(UDMTextureSet* InTextureSet, bool bInAccepted,
	bool bInReplace)
{
	if (!InTextureSet || !bInAccepted)
	{
		return;
	}

	IDynamicMaterialEditorModule& DynamicMaterialEditorModule = IDynamicMaterialEditorModule::Get();

	UDynamicMaterialModelBase* Model = DynamicMaterialEditorModule.GetOpenedMaterialModel(nullptr);

	if (!Model)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(Model);

	if (!EditorOnlyData)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddTextureSet", "Add Texture Set"));
	EditorOnlyData->Modify();

	const bool bSuccess = UDMTextureSetFunctionLibrary::AddTextureSetToModel(EditorOnlyData, InTextureSet, bInReplace);

	if (!bSuccess)
	{
		Transaction.Cancel();
	}
}

TSharedRef<FExtender> FDMContentBrowserIntegration::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	bool bHasMaterialDesignerAsset = false;

	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (UClass* AssetClass = SelectedAsset.GetClass(EResolveClass::Yes))
		{
			if (AssetClass->IsChildOf<UDynamicMaterialInstance>() || AssetClass->IsChildOf<UDynamicMaterialModel>())
			{
				bHasMaterialDesignerAsset = true;
				break;
			}
		}
	}

	if (!bHasMaterialDesignerAsset)
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		"GetAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateLambda(
			[InSelectedAssets](FMenuBuilder& InMenuBuilder)
			{
				InMenuBuilder.AddMenuEntry(
					LOCTEXT("CreateInstance", "Create Material Designer Instance"),
					LOCTEXT("CreateInstanceTooltip", "Create a Material Designer Instance from a Material Designer Material."),
					FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
					FUIAction(FExecuteAction::CreateStatic(&FDMContentBrowserIntegration::CreateInstance, InSelectedAssets))
				);
			}
		)
	);

	return Extender;
}

void FDMContentBrowserIntegration::CreateInstance(TArray<FAssetData> InSelectedAssets)
{
	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (UClass* AssetClass = SelectedAsset.GetClass(EResolveClass::Yes))
		{
			if (AssetClass->IsChildOf<UDynamicMaterialModel>())
			{
				CreateModelInstance(Cast<UDynamicMaterialModel>(SelectedAsset.GetAsset()));
				break;
			}

			if (AssetClass->IsChildOf<UDynamicMaterialInstance>())
			{
				CreateMaterialInstance(Cast<UDynamicMaterialInstance>(SelectedAsset.GetAsset()));
				break;
			}
		}
	}
}

void FDMContentBrowserIntegration::CreateModelInstance(UDynamicMaterialModel* InModel)
{
	if (!InModel)
	{
		return;
	}

	UMaterial* ParentMaterial = InModel->GetGeneratedMaterial();

	if (!ParentMaterial)
	{
		return;
	}

	if (!ParentMaterial->HasAnyFlags(RF_Public))
	{
		const EAppReturnType::Type Result = FMessageDialog::Open(
			EAppMsgType::YesNo,
			LOCTEXT("ExportMaterialFromModel", 
				"Generating a Material Designer Instance requires that the Generated Material be exported from its package.\n\n"
				"The package containing the material will be saved. This may be a level.\n\n"
				"Continue?")
		);

		switch (Result)
		{
			case EAppReturnType::Yes:
				ParentMaterial->Modify(/* Always Mark Dirty */ true);
				ParentMaterial->SetFlags(RF_Public);
				UPackageTools::SavePackagesForObjects({InModel});
				break;

			default:
				return;
		}
	}

	UDynamicMaterialModelDynamic* ModelDynamic = UDynamicMaterialModelDynamic::Create(GetTransientPackage(), InModel);

	if (!ModelDynamic)
	{
		return;
	}

	const FString CurrentName = TEXT("MDM_") + UDMMaterialModelFunctionLibrary::RemoveAssetPrefix(InModel->GetName());

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString PackageName, AssetName;
	AssetTools.CreateUniqueAssetName(CurrentName, TEXT(""), PackageName, AssetName);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : "/Game";

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return;
	}

	PackageName = FPaths::GetBaseFilename(*SaveObjectPath, false);

	UPackage* Package = CreatePackage(*PackageName);

	if (!Package)
	{
		return;
	}

	AssetName = FPaths::GetBaseFilename(*SaveObjectPath, true);

	ModelDynamic->SetFlags(RF_Standalone | RF_Public);
	ModelDynamic->Rename(*AssetName, Package, REN_DontCreateRedirectors);

	FAssetRegistryModule::AssetCreated(ModelDynamic);
}

void FDMContentBrowserIntegration::CreateMaterialInstance(UDynamicMaterialInstance* InInstance)
{
	if (!InInstance)
	{
		return;
	}

	UDynamicMaterialModel* Model = InInstance->GetMaterialModel();

	if (!Model)
	{
		return;
	}

	UMaterial* ParentMaterial = Model->GetGeneratedMaterial();

	if (!ParentMaterial)
	{
		return;
	}

	if (!ParentMaterial->HasAnyFlags(RF_Public) || !Model->HasAnyFlags(RF_Public))
	{
		const EAppReturnType::Type Result = FMessageDialog::Open(
			EAppMsgType::YesNo,
			LOCTEXT("ExportMaterialFromInstance",
				"Generating a Material Designer Instance requires that the Generated Material and Material Model be exported from this package.\n\n"
				"The package containing the material will be saved. This may be a level.\n\n"
				"Continue?")
		);

		switch (Result)
		{
			case EAppReturnType::Yes:
				Model->Modify(/* Always Mark Dirty */ true);
				Model->SetFlags(RF_Public);
				ParentMaterial->Modify(/* Always Mark Dirty */ true);
				ParentMaterial->SetFlags(RF_Public);
				UPackageTools::SavePackagesForObjects({InInstance});
				break;

			default:
				return;
		}
	}

	UDynamicMaterialModelDynamic* ModelDynamic = UDynamicMaterialModelDynamic::Create(GetTransientPackage(), Model);

	if (!ModelDynamic)
	{
		return;
	}

	UDynamicMaterialInstance* Instance = NewObject<UDynamicMaterialInstance>(
		GetTransientPackage(), 
		MakeUniqueObjectName(GetTransientPackage(), UDynamicMaterialInstance::StaticClass(), TEXT("MaterialDesigner"))
	);

	if (!Instance)
	{
		return;
	}

	Instance->SetMaterialModel(ModelDynamic);
	ModelDynamic->SetDynamicMaterialInstance(Instance);
	Instance->InitializeMIDPublic();

	const FString CurrentName = TEXT("MDI_") + UDMMaterialModelFunctionLibrary::RemoveAssetPrefix(InInstance->GetName());

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString PackageName, AssetName;
	AssetTools.CreateUniqueAssetName(CurrentName, TEXT(""), PackageName, AssetName);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : "/Game";

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return;
	}

	PackageName = FPaths::GetBaseFilename(*SaveObjectPath, false);
	UPackage* Package = CreatePackage(*PackageName);

	if (!Package)
	{
		return;
	}

	AssetName = FPaths::GetBaseFilename(*SaveObjectPath, true);

	Instance->SetFlags(RF_Standalone | RF_Public | RF_Transactional);
	Instance->Rename(*AssetName, Package, REN_DontCreateRedirectors);

	FAssetRegistryModule::AssetCreated(Instance);
}

#undef LOCTEXT_NAMESPACE
