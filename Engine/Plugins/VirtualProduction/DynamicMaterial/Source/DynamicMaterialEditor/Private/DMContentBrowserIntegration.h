// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/ContainersFwd.h"
#include "Material/DynamicMaterialInstance.h"

class FExtender;
class FMenuBuilder;
class UDMTextureSet;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
struct FAssetData;

class FDMContentBrowserIntegration
{
public:
	static void Integrate();

	static void Disintegrate();

	static void UpdateMaterialDesignerMaterialFromTextureSet(TArray<FAssetData> InSelectedAssets, bool bInReplace);

protected:
	static FDelegateHandle TextureSetPopulateHandle;
	static FDelegateHandle ContentBrowserAssetHandle;

	static void ExtendMenu(FMenuBuilder& InMenuBuilder, const TArray<FAssetData>& InSelectedAssets);

	static void CreateMaterialDesignerMaterialFromTextureSet(TArray<FAssetData> InSelectedAssets);

	static void OnCreateMaterialDesignerMaterialFromTextureSetComplete(UDMTextureSet* InTextureSet, bool bInAccepted, FString InPath);

	static void OnUpdateMaterialDesignerMaterialFromTextureSetComplete(UDMTextureSet* InTextureSet, bool bInAccepted, bool bInReplace);

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets);

	static void CreateInstance(TArray<FAssetData> InSelectedAssets);

	static void CreateModelInstance(UDynamicMaterialModel* InModel);

	static void CreateMaterialInstance(UDynamicMaterialInstance* InInstance);
};
