// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMObjectMaterialProperty.h"

#include "Components/PrimitiveComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Internationalization/Text.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Model/DynamicMaterialModel.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMObjectMaterialProperty)

#define LOCTEXT_NAMESPACE "DMObjectMaterialProperty"

FDMObjectMaterialProperty::FDMObjectMaterialProperty()
{
}

FDMObjectMaterialProperty::FDMObjectMaterialProperty(UPrimitiveComponent* InOuter, int32 InIndex)
	: OuterWeak(InOuter)
	, Index(InIndex)
{
}

FDMObjectMaterialProperty::FDMObjectMaterialProperty(UObject* InOuter, FProperty* InProperty, int32 InIndex)
	: OuterWeak(InOuter)
	, Property(InProperty)
	, Index(InIndex)
{
}

UObject* FDMObjectMaterialProperty::GetOuter() const
{
	return OuterWeak.Get();
}

FProperty* FDMObjectMaterialProperty::GetProperty() const
{
	return Property;
}

int32 FDMObjectMaterialProperty::GetIndex() const
{
	return Index;
}

UDynamicMaterialModelBase* FDMObjectMaterialProperty::GetMaterialModelBase() const
{
	if (UDynamicMaterialInstance* Instance = GetMaterial())
	{
		return Instance->GetMaterialModelBase();
	}

	return nullptr;
}

UDynamicMaterialInstance* FDMObjectMaterialProperty::GetMaterial() const
{
	return Cast<UDynamicMaterialInstance>(GetMaterialInterface());
}

UMaterialInterface* FDMObjectMaterialProperty::GetMaterialInterface() const
{
	UObject* Outer = OuterWeak.Get();

	if (!Outer)
	{
		return nullptr;
	}

	if (Property != nullptr)
	{
		UMaterialInterface* Material = nullptr;

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);

			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Outer));

				if (ArrayHelper.IsValidIndex(Index))
				{
					void* Value = ArrayHelper.GetRawPtr(Index);
					Material = *static_cast<UMaterialInterface**>(Value);
				}
			}
		}
		else
		{
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);

			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				Property->GetValue_InContainer(Outer, &Material);
			}
		}

		return Material;
	}
	else if (Index != INDEX_NONE)
	{
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Outer))
		{
			if (Index >= 0 && Index < Component->GetNumMaterials())
			{
				return Component->GetMaterial(Index);
			}
		}
	}

	return nullptr;
}

DYNAMICMATERIALEDITOR_API void FDMObjectMaterialProperty::SetMaterialSetterDelegate(const FDMSetMaterialObjectProperty& InDelegate)
{
	MaterialSetterDelegate = InDelegate;
}

void FDMObjectMaterialProperty::SetMaterial(UDynamicMaterialInstance* InDynamicMaterial)
{
	if (MaterialSetterDelegate.IsBound())
	{
		if (MaterialSetterDelegate.Execute(*this, InDynamicMaterial))
		{
			return;
		}
	}

	UObject* Outer = OuterWeak.Get();

	if (!Outer)
	{
		return;
	}

	if (GUndo)
	{
		Outer->Modify();
	}

	if (Property != nullptr)
	{
		Outer->PreEditChange(Property);

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);

			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Outer));

				if (ArrayHelper.IsValidIndex(Index))
				{
					*reinterpret_cast<UMaterialInterface**>(ArrayHelper.GetRawPtr(Index)) = InDynamicMaterial;
				}
			}
		}
		else
		{
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);

			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				Property->SetValue_InContainer(Outer, &InDynamicMaterial);
			}
		}

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		Outer->PostEditChangeProperty(PropertyChangedEvent);
	}
	else if (Index != INDEX_NONE)
	{
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Outer))
		{
			if (Index >= 0 && Index < Component->GetNumMaterials())
			{
				Component->SetMaterial(Index, InDynamicMaterial);
			}
		}
	}
}

bool FDMObjectMaterialProperty::IsValid() const
{
	UObject* Outer = OuterWeak.Get();

	if (!Outer)
	{
		return false;
	}

	if (Property != nullptr)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);

			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Outer));

				return ArrayHelper.IsValidIndex(Index);
			}
		}
		else
		{
			FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);

			if (ObjectProperty && ObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				return true;
			}
		}

		return false;
	}
	else if (Index != INDEX_NONE)
	{
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Outer))
		{
			if (Index >= 0 && Index < Component->GetNumMaterials())
			{
				return true;
			}
		}
	}

	return false;
}

FText FDMObjectMaterialProperty::GetPropertyName(bool bInIgnoreNewStatus) const
{
	UObject* Outer = OuterWeak.Get();

	if (!Outer)
	{
		return FText::GetEmpty();
	}

	UDynamicMaterialModelBase* MaterialModelBase = GetMaterialModelBase();

	if (Property != nullptr)
	{
		FText PropertyNameText = Property->GetDisplayNameText();

		if (CastField<FArrayProperty>(Property))
		{
			PropertyNameText = FText::Format(
				LOCTEXT("PropertyNameFormatArray", "{0} [{1}]"),
				PropertyNameText,
				FText::AsNumber(Index)
			);
		}

		if (MaterialModelBase || bInIgnoreNewStatus)
		{
			return FText::Format(
				LOCTEXT("PropertyNameFormat", "{0}"),
				PropertyNameText
			);
		}
		else
		{
			return FText::Format(
				LOCTEXT("PropertyNameFormatNew", "{0} (Create New)"),
				PropertyNameText
			);
		}
	}
	else if (Index != INDEX_NONE)
	{
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Outer))
		{
			if (Index >= 0 && Index <= Component->GetNumMaterials())
			{
				if (MaterialModelBase || bInIgnoreNewStatus)
				{
					return FText::Format(
						LOCTEXT("MaterialListNameFormat", "Element {0}"),
						FText::AsNumber(Index)
					);
				}
				else
				{
					return FText::Format(
						LOCTEXT("MaterialListNameFormatNew", "Element {0} (Create New)"),
						FText::AsNumber(Index)
					);
				}
			}
		}

		return FText::GetEmpty();
	}

	return FText::GetEmpty();
}

void FDMObjectMaterialProperty::Reset()
{
	OuterWeak = nullptr;
	Property = nullptr;
	Index = INDEX_NONE;
}

bool FDMObjectMaterialProperty::IsProperty() const
{
	return !!Property;
}

bool FDMObjectMaterialProperty::IsElement() const
{
	return !Property;
}

#undef LOCTEXT_NAMESPACE
