// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "DMWorldSubsystem.h"

#include "GameFramework/Actor.h"
#include "LevelEditor/DMLevelEditorIntegration.h"
#include "Model/DynamicMaterialModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMWorldSubsystem)

UDMWorldSubsystem::UDMWorldSubsystem()
	: KeyframeHandler(nullptr)
{
	// Default fallback implementation
	InvokeTabDelegate.BindWeakLambda(this, [this]() { FDMLevelEditorIntegration::InvokeTabForWorld(GetWorld()); });
}

UDynamicMaterialModelBase* UDMWorldSubsystem::ExecuteGetCustomEditorModelDelegate()
{
	if (CustomModelEditorGetDelegate.IsBound())
	{
		return CustomModelEditorGetDelegate.Execute();
	}

	// An unbound delegate means there is no model to get.
	return nullptr;
}

void UDMWorldSubsystem::ExecuteSetCustomEditorModelDelegate(UDynamicMaterialModelBase* InMaterialModel)
{
	if (!InMaterialModel || IsValid(InMaterialModel))
	{
		CustomModelEditorSetDelegate.ExecuteIfBound(InMaterialModel);
	}
}

void UDMWorldSubsystem::ExecuteCustomObjectPropertyEditorDelegate(const FDMObjectMaterialProperty& InObjectProperty)
{
	CustomObjectPropertyEditorDelegate.ExecuteIfBound(InObjectProperty);
}

void UDMWorldSubsystem::ExecuteSetCustomEditorActorDelegate(AActor* InActor)
{
	if (!InActor || IsValid(InActor))
	{
		CustomActorEditorDelegate.ExecuteIfBound(InActor);
	}
}

bool UDMWorldSubsystem::ExecuteIsValidDelegate(UDynamicMaterialModelBase* InMaterialModel)
{
	if (!IsValid(InMaterialModel))
	{
		return false;
	}

	if (IsValidDelegate.IsBound())
	{
		return IsValidDelegate.Execute(InMaterialModel);
	}

	// An unbound delegate means it is valid.
	return true;
}

bool UDMWorldSubsystem::ExecuteMaterialValueSetterDelegate(const FDMObjectMaterialProperty& InObjectProperty, UDynamicMaterialInstance* InMaterialInstance)
{
	if (SetMaterialValueDelegate.IsBound())
	{
		return SetMaterialValueDelegate.Execute(InObjectProperty, InMaterialInstance);
	}

	// An unbound delegate means that no setting took place.
	return false;
}

void UDMWorldSubsystem::ExecuteInvokeTabDelegate()
{
	InvokeTabDelegate.ExecuteIfBound();
}
