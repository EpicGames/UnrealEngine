// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/Customizations/DMStageEditDetailsCustomization.h"

#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "DMStageEditDetailsCustomization"

TSharedRef<IPropertyTypeCustomization> FDMStageEditDetailsCustomization::MakeInstance()
{
	return MakeShared<FDMStageEditDetailsCustomization>();
}

void FDMStageEditDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle
	, FDetailWidgetRow& HeaderRow
	, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}

#undef LOCTEXT_NAMESPACE
