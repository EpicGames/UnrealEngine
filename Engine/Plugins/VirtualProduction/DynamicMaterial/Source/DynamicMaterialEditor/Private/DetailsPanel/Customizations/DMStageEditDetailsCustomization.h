// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"

/**
 * Material Designer Stage Edit Details Customization
 */
class FDMStageEditDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, 
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	//~ End IPropertyTypeCustomization

private:
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
