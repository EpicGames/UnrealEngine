// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/DMMaterialInterfaceTypeCustomizer.h"

#include "DetailsPanel/Widgets/SDMDetailsPanelMaterialInterfaceWidget.h"
#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

bool FDMMaterialInterfaceTypeIdentifier::IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const
{
	return InPropertyHandle.GetNumOuterObjects() > 0;
}

TSharedRef<IPropertyTypeCustomization> FDMMaterialInterfaceTypeCustomizer::MakeInstance()
{
	return MakeShared<FDMMaterialInterfaceTypeCustomizer>();
}

void FDMMaterialInterfaceTypeCustomizer::CustomizeHeader(TSharedRef<IPropertyHandle> EditorPropertyHandle, class FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		EditorPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SDMDetailsPanelMaterialInterfaceWidget, EditorPropertyHandle)
		.ThumbnailPool(CustomizationUtils.GetThumbnailPool())
	];
}

void FDMMaterialInterfaceTypeCustomizer::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

FDMMaterialInterfaceTypeCustomizer::FDMMaterialInterfaceTypeCustomizer()
{
}
