// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"

#include "Templates/SharedPointer.h"

class FDMMaterialInterfaceTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	//~ Begin IPropertyTypeIdentifier
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override;
	//~ End IPropertyTypeIdentifier
};

class FDMMaterialInterfaceTypeCustomizer : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FDMMaterialInterfaceTypeCustomizer();

	/** BEGIN IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> EditorPropertyHandle, class FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> EditorPropertyHandle, class IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	/** END IPropertyTypeCustomization interface */
};
