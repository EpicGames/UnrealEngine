// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/DMPropertyTypeCustomizer.h"

#include "DetailsPanel/Widgets/SDMDetailsPanelTabSpawner.h"
#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

TSharedRef<IPropertyTypeCustomization> FDMPropertyTypeCustomizer::MakeInstance()
{
	return MakeShared<FDMPropertyTypeCustomizer>();
}

void FDMPropertyTypeCustomizer::CustomizeHeader(TSharedRef<IPropertyHandle> EditorPropertyHandle, class FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		EditorPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SDMDetailsPanelTabSpawner, EditorPropertyHandle)
	];
}

void FDMPropertyTypeCustomizer::CustomizeChildren(TSharedRef<IPropertyHandle> EditorPropertyHandle, class IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

FDMPropertyTypeCustomizer::FDMPropertyTypeCustomizer()
{

}
