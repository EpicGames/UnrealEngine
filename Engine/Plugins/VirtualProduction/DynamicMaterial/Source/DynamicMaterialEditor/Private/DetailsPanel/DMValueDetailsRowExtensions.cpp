// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/DMValueDetailsRowExtensions.h"

#include "Components/DMMaterialValue.h"
#include "Components/DMTextureUV.h"
#include "Delegates/IDelegateInstance.h"
#include "DetailRowMenuContext.h"
#include "DynamicMaterialEditorModule.h"
#include "Framework/SlateDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Styling/CoreStyle.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "DMValueRowExtensions"

namespace UE::DynamicMaterialEditor::Private
{
	constexpr int32 ComponentX = 0;
	constexpr int32 ComponentY = 1;
	const FName PropertyEditorModuleName = "PropertyEditor";
	const FText ParameterNameMenuEntry = LOCTEXT("ParameterName", "Parameter Name");
	const FText ParameterNameMenuEntryX = LOCTEXT("ParameterNameX", "Parameter Name X");
	const FText ParameterNameMenuEntryY = LOCTEXT("ParameterNameY", "Parameter Name Y");
	const FText ParameterExposeMenuEntry = LOCTEXT("ParameterExpose", "Expose Parameter");
	const FText ParameterExposeMenuEntryX = LOCTEXT("ParameterExposeX", "Expose Parameter X");
	const FText ParameterExposeMenuEntryY = LOCTEXT("ParameterExposeY", "Expose Parameter Y");
	const FText SetParameterNameToolTip = LOCTEXT("SetParameterNameToolTip", "Set the name of the parameter this property is exposed as within the generated material.");
	const FText ExposeParameterToolTip = LOCTEXT("ExposeParameterToolTip", "When unchecked, parameter will appear in the \"99 - Uncategorized\" category.");
}

FDMValueDetailsRowExtensions& FDMValueDetailsRowExtensions::Get()
{
	static FDMValueDetailsRowExtensions Instance;
	return Instance;
}

FDMValueDetailsRowExtensions::~FDMValueDetailsRowExtensions()
{
	UnregisterRowExtensions();
}

void FDMValueDetailsRowExtensions::RegisterRowExtensions()
{
	using namespace UE::DynamicMaterialEditor::Private;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
	FOnGenerateGlobalRowExtension& RowExtensionDelegate = PropertyEditorModule.GetGlobalRowExtensionDelegate();
	RowExtensionHandle = RowExtensionDelegate.AddStatic(&HandleCreatePropertyRowExtension);
}

void FDMValueDetailsRowExtensions::UnregisterRowExtensions()
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (RowExtensionHandle.IsValid() && FModuleManager::Get().IsModuleLoaded(PropertyEditorModuleName))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
		PropertyEditorModule.GetGlobalRowExtensionDelegate().Remove(RowExtensionHandle);
		RowExtensionHandle.Reset();
	}
}

void FDMValueDetailsRowExtensions::HandleCreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions)
{
	if (!InArgs.Property && !InArgs.PropertyHandle.IsValid())
	{
		return;
	}

	UToolMenus* Menus = UToolMenus::Get();
	check(Menus);

	UToolMenu* ContextMenu = Menus->FindMenu(UE::PropertyEditor::RowContextMenuName);

	static const FName DetailViewRowExtensionName = "DMValueRowExtensionContextSection";

	if (ContextMenu->ContainsSection(DetailViewRowExtensionName))
	{
		return;
	}

	ContextMenu->AddDynamicSection(
		DetailViewRowExtensionName,
		FNewToolMenuDelegate::CreateStatic(&FillPropertyRightClickMenu)
	);
}

void FDMValueDetailsRowExtensions::FillPropertyRightClickMenu(UToolMenu* InToolMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDetailRowMenuContext* RowMenuContext = InToolMenu->FindContext<UDetailRowMenuContext>();

	if (!RowMenuContext)
	{
		return;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle;

	for (const TSharedPtr<IPropertyHandle>& ContextPropertyHandle : RowMenuContext->PropertyHandles)
	{
		if (ContextPropertyHandle.IsValid())
		{
			PropertyHandle = ContextPropertyHandle;
			break;
		}
	}

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	if (TSharedPtr<IDetailsView> DetailsView = RowMenuContext->DetailsView.Pin())
	{
		if (!DetailsView->IsPropertyEditingEnabled())
		{
			return;
		}
	}

	if (PropertyHandle->IsEditConst() || !PropertyHandle->IsEditable())
	{
		return;
	}

	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);

	if (Outers.IsEmpty())
	{
		return;
	}

	static const FName MaterialDesignerMenuName = "MaterialDesigner";
	static const FText MaterialDesignerSectionName = LOCTEXT("MaterialDesigner", "Material Designer");

	FText CurrentParameterName;

	FToolMenuSection& Section = InToolMenu->AddSection(MaterialDesignerMenuName, MaterialDesignerSectionName);

	if (UDMMaterialValue* Value = Cast<UDMMaterialValue>(Outers[0]))
	{
		FillPropertyRightClickMenu_Value(Section, Value);
	}
	else if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(Outers[0]))
	{
		FillPropertyRightClickMenu_TextureUV(Section, TextureUV, PropertyHandle->GetProperty()->GetFName());
	}
}

void FDMValueDetailsRowExtensions::FillPropertyRightClickMenu_TextureUV(FToolMenuSection& InSection, UDMTextureUV* InTextureUV, FName InPropertyName)
{
	using namespace UE::DynamicMaterialEditor::Private;

	TWeakObjectPtr<UDMTextureUV> TextureUVWeak = InTextureUV;

	auto CreateParameterExposeMenuEntry = [&InSection, &TextureUVWeak, &InPropertyName](const FText& InMenuText, int32 InComponent)
		{
			FUIAction ExposeTextureUVXAction;
			ExposeTextureUVXAction.ExecuteAction = FExecuteAction::CreateLambda(
				[TextureUVWeak, InPropertyName, InComponent]()
				{
					if (UDMTextureUV* TextureUV = TextureUVWeak.Get())
					{
						TextureUV->SetShouldExposeParameter(InPropertyName, InComponent, !TextureUV->GetShouldExposeParameter(InPropertyName, InComponent));
					}
				});

			ExposeTextureUVXAction.CanExecuteAction = FCanExecuteAction::CreateLambda(
				[TextureUVWeak]()
				{
					return TextureUVWeak.IsValid();
				});

			ExposeTextureUVXAction.GetActionCheckState = FGetActionCheckState::CreateLambda(
				[TextureUVWeak, InPropertyName, InComponent]()
				{
					if (UDMTextureUV* TextureUV = TextureUVWeak.Get())
					{
						if (TextureUV->GetShouldExposeParameter(InPropertyName, InComponent))
						{
							return ECheckBoxState::Checked;
						}
					}

					return ECheckBoxState::Unchecked;
				});

			const FName MenuNameBase = *(InPropertyName.ToString() + TEXT("Expose"));
			const FName MenuName = FName(MenuNameBase, InComponent);

			InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
				MenuName,
				InMenuText,
				ExposeParameterToolTip,
				TAttribute<FSlateIcon>(),
				FToolUIActionChoice(ExposeTextureUVXAction),
				EUserInterfaceActionType::ToggleButton
			));
		};

	auto CreateParameterNameMenuEntry = [&InSection, &TextureUVWeak, &InPropertyName](const FText& InMenuText, int32 InComponent)
		{
			FNewToolMenuChoice RenameChoice(FOnGetContent::CreateLambda(
				[TextureUVWeak, InPropertyName, InComponent]() -> TSharedRef<SWidget>
				{
					UDMTextureUV* TextureUV = TextureUVWeak.Get();

					if (!IsValid(TextureUV))
					{
						return SNullWidget::NullWidget;
					}

					return SNew(SEditableTextBox)
						.Text(FText::FromName(TextureUV->GetMaterialParameterName(InPropertyName, InComponent)))
						.OnVerifyTextChanged_Static(&VerifyParameterName)
						.AllowContextMenu(false)
						.ClearKeyboardFocusOnCommit(true)
						.MinDesiredWidth(100.f)
						.OnTextCommitted_Lambda(
							[TextureUVWeak, InPropertyName, InComponent](const FText& InText, ETextCommit::Type InCommitType)
							{
								if (InCommitType != ETextCommit::OnEnter)
								{
									return;
								}

								UDMTextureUV* TextureUV = TextureUVWeak.Get();

								if (!IsValid(TextureUV))
								{
									return;
								}

								FText Unused;

								if (!VerifyParameterName(InText, Unused))
								{
									return;
								}

								SetTextureUVParameterName(TextureUVWeak, InPropertyName, InComponent, FName(*InText.ToString()));
							});
				}));

			const FName MenuNameBase = *(InPropertyName.ToString() + TEXT("Name"));
			const FName MenuName = FName(MenuNameBase, InComponent);

			InSection.AddSubMenu(
				MenuName,
				InMenuText,
				SetParameterNameToolTip,
				RenameChoice,
				/* Open submenu on click */ true,
				TAttribute<FSlateIcon>(),
				/* Should close after selection */ false
			);
		};

	if (InPropertyName == UDMTextureUV::NAME_Rotation)
	{
		CreateParameterExposeMenuEntry(ParameterExposeMenuEntry, ComponentX);
		CreateParameterNameMenuEntry(ParameterNameMenuEntry, ComponentX);
	}

	if (InPropertyName == UDMTextureUV::NAME_Offset 
		|| InPropertyName == UDMTextureUV::NAME_Pivot
		|| InPropertyName == UDMTextureUV::NAME_Tiling)
	{
		CreateParameterExposeMenuEntry(ParameterExposeMenuEntryX, ComponentX);
		CreateParameterExposeMenuEntry(ParameterExposeMenuEntryY, ComponentY);
		CreateParameterNameMenuEntry(ParameterNameMenuEntryX, ComponentX);
		CreateParameterNameMenuEntry(ParameterNameMenuEntryY, ComponentY);
	}
}

void FDMValueDetailsRowExtensions::FillPropertyRightClickMenu_Value(FToolMenuSection& InSection, UDMMaterialValue* InValue)
{
	using namespace UE::DynamicMaterialEditor::Private;

	TWeakObjectPtr<UDMMaterialValue> ValueWeak = InValue;

	FUIAction ExposeValueAction;
	ExposeValueAction.ExecuteAction = FExecuteAction::CreateLambda(
		[ValueWeak]()
		{
			if (UDMMaterialValue* Value = ValueWeak.Get())
			{
				Value->SetShouldExposeParameter(!Value->GetShouldExposeParameter());
			}
		});

	ExposeValueAction.CanExecuteAction = FCanExecuteAction::CreateLambda(
		[ValueWeak]()
		{
			return ValueWeak.IsValid();
		});

	ExposeValueAction.GetActionCheckState = FGetActionCheckState::CreateLambda(
		[ValueWeak]()
		{
			if (UDMMaterialValue* Value = ValueWeak.Get())
			{
				if (Value->GetShouldExposeParameter())
				{
					return ECheckBoxState::Checked;
				}
			}

			return ECheckBoxState::Unchecked;
		});

	InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"ExposeParameter",
		LOCTEXT("ExposeParameter", "Expose Parameter"),
		ExposeParameterToolTip,
		TAttribute<FSlateIcon>(),
		FToolUIActionChoice(ExposeValueAction),
		EUserInterfaceActionType::ToggleButton
	));

	FNewToolMenuChoice RenameChoice(FOnGetContent::CreateLambda(
		[ValueWeak]() -> TSharedRef<SWidget>
		{
			UDMMaterialValue* Value = ValueWeak.Get();

			if (!IsValid(Value))
			{
				return SNullWidget::NullWidget;
			}

			return SNew(SEditableTextBox)
				.Text(FText::FromName(Value->GetMaterialParameterName()))
				.OnVerifyTextChanged_Static(&VerifyParameterName)
				.AllowContextMenu(false)
				.ClearKeyboardFocusOnCommit(true)
				.MinDesiredWidth(100.f)
				.OnTextCommitted_Lambda(
					[ValueWeak](const FText& InText, ETextCommit::Type InCommitType)
					{
						if (InCommitType != ETextCommit::OnEnter)
						{
							return;
						}

						UDMMaterialValue* Value = ValueWeak.Get();

						if (!IsValid(Value))
						{
							return;
						}

						FText Unused;

						if (!VerifyParameterName(InText, Unused))
						{
							return;
						}

						SetValueParameterName(ValueWeak, FName(*InText.ToString()));
					});
		}));

	InSection.AddSubMenu(
		"RenameParameter",
		ParameterNameMenuEntry,
		SetParameterNameToolTip,
		RenameChoice,
		/* Open submenu on click */ true,
		TAttribute<FSlateIcon>(),
		/* Should close after selection */ false
	);
}

void FDMValueDetailsRowExtensions::SetValueParameterName(TWeakObjectPtr<UDMMaterialValue> InValueWeak, FName InName)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDMMaterialValue* Value = InValueWeak.Get();

	if (!Value)
	{
		return;
	}

	if (InName.IsNone())
	{
		return;
	}

	const FName CurrentName = Value->GetMaterialParameterName();

	if (CurrentName == InName)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetParameterName", "Set Parameter Name"));
	Value->Modify();
	Value->SetParameterName(InName);

	UE_LOGF(LogDynamicMaterialEditor, Log, "Parameter renamed to: %ls", *Value->GetMaterialParameterName().ToString());
}

void FDMValueDetailsRowExtensions::SetTextureUVParameterName(TWeakObjectPtr<UDMTextureUV> InTextureUVWeak, FName InPropertyName, 
	int32 InComponent, FName InName)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDMTextureUV* TextureUV = InTextureUVWeak.Get();

	if (!TextureUV)
	{
		return;
	}

	if (InName.IsNone())
	{
		return;
	}

	const FName CurrentName = TextureUV->GetMaterialParameterName(InPropertyName, InComponent);

	if (CurrentName == InName)
	{
		return;
	}

	TextureUV->SetMaterialParameterName(InPropertyName, InComponent, InName);
	UE_LOGF(LogDynamicMaterialEditor, Log, "Parameter renamed to: %ls", *TextureUV->GetMaterialParameterName(InPropertyName, InComponent).ToString());
}

bool FDMValueDetailsRowExtensions::VerifyParameterName(const FText& InValue, FText& OutErrorText)
{
	static const FText TooShortError = LOCTEXT("TooShortError", "Min 3 characters.");
	static const FText TooLongError = LOCTEXT("TooLongError", "Max 50 characters.");
	static const FText InvalidCharacterError = LOCTEXT("InvalidCharacterError", "Valid characters are A-Z, A-z, _ and -");

	static const TArray<TPair<TCHAR, TCHAR>> ValidCharacterRanges = {
		{'A', 'Z'},
		{'a', 'z'},
		{'0', '9'},
		{'-', '-'},
		{'_', '_'}
	};

	const FString ValueStr = InValue.ToString();
	const int32 ValueLen = ValueStr.Len();

	if (ValueLen < 3)
	{
		OutErrorText = TooShortError;
		return false;
	}

	if (ValueLen > 50)
	{
		OutErrorText = TooLongError;
		return false;
	}

	for (int32 Index = 0; Index < ValueLen; ++Index)
	{
		bool bInRange = false;

		for (const TPair<TCHAR, TCHAR>& ValidCharacterRange : ValidCharacterRanges)
		{
			if (ValueStr[Index] >= ValidCharacterRange.Key && ValueStr[Index] <= ValidCharacterRange.Value)
			{
				bInRange = true;
				break;
			}
		}

		if (!bInRange)
		{
			OutErrorText = InvalidCharacterError;
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
