// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/WeakObjectPtrFwd.h"

class FName;
class UDMMaterialValue;
class UDMTextureUV;
class UToolMenu;
struct FOnGenerateGlobalRowExtensionArgs;
struct FPropertyRowExtensionButton;
struct FToolMenuContext;
struct FToolMenuSection;

class FDMValueDetailsRowExtensions
{
public:
	static FDMValueDetailsRowExtensions& Get();

	~FDMValueDetailsRowExtensions();

	void RegisterRowExtensions();

	void UnregisterRowExtensions();

private:
	FDelegateHandle RowExtensionHandle;

	static void HandleCreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions);

	static void FillPropertyRightClickMenu(UToolMenu* InToolMenu);

	static void FillPropertyRightClickMenu_TextureUV(FToolMenuSection& InSection, UDMTextureUV* InTextureUV, FName InPropertyName);

	static void FillPropertyRightClickMenu_Value(FToolMenuSection& InSection, UDMMaterialValue* InValue);

	static void SetValueParameterName(TWeakObjectPtr<UDMMaterialValue> InValueWeak, FName InName);

	static void SetTextureUVParameterName(TWeakObjectPtr<UDMTextureUV> InTextureUVWeak,
		FName InPropertyName, int32 InComponent, FName InName);

	static bool VerifyParameterName(const FText& InValue, FText& OutErrorText);
};
