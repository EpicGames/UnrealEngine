// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/Widgets/SDMDetailsPanelMaterialInterfaceWidget.h"

#include "Components/PrimitiveComponent.h"
#include "DMObjectMaterialProperty.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/DynamicMaterialModel.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMDetailsPanelMaterialInterfaceWidget"

void SDMDetailsPanelMaterialInterfaceWidget::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return;
	}

	UObject* Value = nullptr;
	InPropertyHandle->GetValue(Value);

	UDynamicMaterialInstance* Material = GetMaterialDesignerMaterial();

	FObjectPropertyBase* const ObjectProperty = CastField<FObjectPropertyBase>(PropertyHandle->GetProperty());

	UClass* const ObjectClass = ObjectProperty
		? ToRawPtr(ObjectProperty->PropertyClass)
		: ToRawPtr(UMaterialInterface::StaticClass());

	TSharedRef<SVerticalBox> Container = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowClear(true)
			.AllowedClass(ObjectClass)
			.DisplayBrowse(true)
			.DisplayCompactSize(false)
			.DisplayThumbnail(true)
			.DisplayUseSelected(true)
			.EnableContentPicker(true)
			.PropertyHandle(PropertyHandle)
			.ThumbnailPool(InArgs._ThumbnailPool)
		];

	const UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (Settings && Settings->bAddDetailsPanelButton)
	{
		Container->AddSlot()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(10.f, 5.f, 10.f, 5.f)
			.AutoHeight()
			[
				SNew(SButton)
				.OnClicked(this, &SDMDetailsPanelMaterialInterfaceWidget::OnButtonClicked)
				.IsEnabled(InPropertyHandle, &IPropertyHandle::IsEditable)
				.Content()
				[
					SNew(STextBlock)
					.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
					.Text(this, &SDMDetailsPanelMaterialInterfaceWidget::GetButtonText)
				]
			];
	}

	ChildSlot
	[
		Container
	];
}

UObject* SDMDetailsPanelMaterialInterfaceWidget::GetAsset() const
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return nullptr;
	}

	UObject* Value = nullptr;
	PropertyHandle->GetValue(Value);

	return Value;
}

UDynamicMaterialInstance* SDMDetailsPanelMaterialInterfaceWidget::GetMaterialDesignerMaterial() const
{
	return Cast<UDynamicMaterialInstance>(GetAsset());
}

void SDMDetailsPanelMaterialInterfaceWidget::SetAsset(UObject* NewAsset)
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return;
	}

	PropertyHandle->SetValueFromFormattedString(NewAsset ? NewAsset->GetPathName() : "");
}

void SDMDetailsPanelMaterialInterfaceWidget::SetMaterialDesignerMaterial(UDynamicMaterialInstance* InMaterial)
{
	SetAsset(InMaterial);
}

FText SDMDetailsPanelMaterialInterfaceWidget::GetButtonText() const
{
	if (GetMaterialDesignerMaterial())
	{
		return LOCTEXT("OpenMaterialDesignerModel", "Edit with Material Designer");
	}

	return LOCTEXT("CreateMaterialDesignerModel", "Create with Material Designer");
}

FReply SDMDetailsPanelMaterialInterfaceWidget::OnButtonClicked()
{
	if (GetMaterialDesignerMaterial())
	{
		return OpenMaterialDesignerTab();
	}

	return CreateMaterialDesignerMaterial();
}

FReply SDMDetailsPanelMaterialInterfaceWidget::CreateMaterialDesignerMaterial()
{
	UDynamicMaterialInstance* Instance = GetMaterialDesignerMaterial();

	// We already have an instance, so we don't need to create one
	if (Instance)
	{
		return FReply::Handled();
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty() || !IsValid(OuterObjects[0]))
	{
		return FReply::Handled();
	}

	const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
	constexpr bool bInvokeTab = true;

	if (FProperty* Property = PropertyHandle->GetProperty())
	{
		if (Property->IsA<FObjectPropertyBase>())
		{
			MaterialDesignerModule.OpenMaterialObjectProperty({OuterObjects[0], Property}, OuterObjects[0]->GetWorld(), bInvokeTab);
		}
	}

	return FReply::Handled();
}

FReply SDMDetailsPanelMaterialInterfaceWidget::ClearMaterialDesignerMaterial()
{
	UDynamicMaterialInstance* Instance = GetMaterialDesignerMaterial();

	// We don't have an instance, so we don't need to clear it (and don't clear non-MDIs)
	if (!Instance)
	{
		return FReply::Handled();
	}

	SetMaterialDesignerMaterial(nullptr);

	return FReply::Handled();
}

FReply SDMDetailsPanelMaterialInterfaceWidget::OpenMaterialDesignerTab()
{
	UDynamicMaterialInstance* Instance = GetMaterialDesignerMaterial();

	// We don't have a MDI, so don't try to open it.
	if (!Instance)
	{
		return FReply::Handled();
	}

	const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
	constexpr bool bInvokeTab = true;

	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);

	if (Outers.Num() == 0)
	{
		MaterialDesignerModule.OpenMaterialModel(Instance->GetMaterialModel(), nullptr, bInvokeTab);
		return FReply::Handled();
	}

	UWorld* OuterWorld = Outers[0]->GetWorld();
	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Outers[0]);

	if (!PrimitiveComponent)
	{
		MaterialDesignerModule.OpenMaterialModel(Instance->GetMaterialModel(), OuterWorld, bInvokeTab);
		return FReply::Handled();
	}

	const int32 NumMaterials = PrimitiveComponent->GetNumMaterials();

	for (int32 Index = 0; Index < NumMaterials; ++Index)
	{
		if (PrimitiveComponent->GetMaterial(Index) == Instance)
		{
			MaterialDesignerModule.OpenMaterialObjectProperty({PrimitiveComponent, Index}, OuterWorld, bInvokeTab);
			return FReply::Handled();
		}
	}

	MaterialDesignerModule.OpenMaterialModel(Instance->GetMaterialModel(), OuterWorld, bInvokeTab);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
