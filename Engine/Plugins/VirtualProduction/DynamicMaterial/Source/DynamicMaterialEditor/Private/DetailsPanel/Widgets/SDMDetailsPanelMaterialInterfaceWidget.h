// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"

class FAssetThumbnailPool;
class FText;
class IPropertyHandle;
class UDynamicMaterialInstance;
class UObject;

class SDMDetailsPanelMaterialInterfaceWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMDetailsPanelMaterialInterfaceWidget) {}
	SLATE_ARGUMENT(TSharedPtr<FAssetThumbnailPool>, ThumbnailPool)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

protected:
	TSharedPtr<IPropertyHandle> PropertyHandle;

	UObject* GetAsset() const;
	void SetAsset(UObject* NewAsset);

	UDynamicMaterialInstance* GetMaterialDesignerMaterial() const;
	void SetMaterialDesignerMaterial(UDynamicMaterialInstance* InMaterial);

	FReply OnButtonClicked();
	FReply CreateMaterialDesignerMaterial();
	FReply ClearMaterialDesignerMaterial();
	FReply OpenMaterialDesignerTab();

	FText GetButtonText() const;
};
