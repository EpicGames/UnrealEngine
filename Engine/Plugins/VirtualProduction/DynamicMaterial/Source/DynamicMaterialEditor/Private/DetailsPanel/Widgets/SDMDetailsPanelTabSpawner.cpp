// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/Widgets/SDMDetailsPanelTabSpawner.h"

#include "AssetToolsModule.h"
#include "DMObjectMaterialProperty.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "IAssetTools.h"
#include "IDynamicMaterialEditorModule.h"
#include "Model/DynamicMaterialModelBase.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMDetailsPanelTabSpawner"

void SDMDetailsPanelTabSpawner::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return;
	}

	UObject* Value = nullptr;
	InPropertyHandle->GetValue(Value);

	TSharedRef<SVerticalBox> Container = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(10.f, 5.f, 10.f, 5.f)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowClear(true)
			.AllowedClass(UDynamicMaterialModelBase::StaticClass())
			.DisplayBrowse(true)
			.DisplayThumbnail(false)
			.DisplayCompactSize(true)
			.DisplayUseSelected(true)
			.EnableContentPicker(true)
			.ObjectPath(this, &SDMDetailsPanelTabSpawner::GetEditorPath)
			.OnObjectChanged(this, &SDMDetailsPanelTabSpawner::OnEditorChanged)
		];

	const UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (Settings && Settings->bAddDetailsPanelButton)
	{
		Container->AddSlot()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(10.f, 5.f, 10.f, 5.f)
			.AutoHeight()
			[
				SNew(SButton)
				.OnClicked(this, &SDMDetailsPanelTabSpawner::OnButtonClicked)
				.Content()
				[
					SNew(STextBlock)
					.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
					.Text(this, &SDMDetailsPanelTabSpawner::GetButtonText)
				]
			];
	}

	ChildSlot
	[
		Container
	];
}

UDynamicMaterialModelBase* SDMDetailsPanelTabSpawner::GetMaterialModelBase() const
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return nullptr;
	}

	UObject* Value = nullptr;
	PropertyHandle->GetValue(Value);

	return Cast<UDynamicMaterialModelBase>(Value);
}

void SDMDetailsPanelTabSpawner::SetMaterialModelBase(UDynamicMaterialModelBase* InNewModel)
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return;
	}

	PropertyHandle->SetValueFromFormattedString(InNewModel ? InNewModel->GetPathName() : "");
}

FText SDMDetailsPanelTabSpawner::GetButtonText() const
{
	if (GetMaterialModelBase())
	{
		return LOCTEXT("OpenMaterialDesignerModel", "Edit with Material Designer");
	}

	return LOCTEXT("CreateMaterialDesignerModel", "Create with Material Designer");
}

FReply SDMDetailsPanelTabSpawner::OnButtonClicked()
{
	if (GetMaterialModelBase())
	{
		return OpenDynamicMaterialModelTab();
	}

	return CreateDynamicMaterialModel();
}

FReply SDMDetailsPanelTabSpawner::CreateDynamicMaterialModel()
{
	UDynamicMaterialModelBase* MaterialModelBase = GetMaterialModelBase();

	// We already have a builder, so we don't need to create one
	if (MaterialModelBase)
	{
		return FReply::Handled();
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty() || !IsValid(OuterObjects[0]))
	{
		return FReply::Handled();
	}

	const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
	constexpr bool bInvokeTab = true;

	if (FProperty* Property = PropertyHandle->GetProperty())
	{
		if (Property->IsA<FObjectPropertyBase>())
		{
			MaterialDesignerModule.OpenMaterialObjectProperty({OuterObjects[0], Property}, OuterObjects[0]->GetWorld(), bInvokeTab);
		}
	}

	return FReply::Handled();
}

FReply SDMDetailsPanelTabSpawner::ClearDynamicMaterialModel()
{
	UDynamicMaterialModelBase* MaterialModelBase = GetMaterialModelBase();

	// We don't have a builder, so we don't need to clear it
	if (!MaterialModelBase)
	{
		return FReply::Handled();
	}

	SetMaterialModelBase(nullptr);

	return FReply::Handled();
}

FReply SDMDetailsPanelTabSpawner::OpenDynamicMaterialModelTab()
{
	UDynamicMaterialModelBase* MaterialModelBase = GetMaterialModelBase();

	// We don't have a builder, so we can't open it
	if (!MaterialModelBase)
	{
		return FReply::Handled();
	}

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.OpenEditorForAssets({MaterialModelBase});

	return FReply::Handled();
}

FString SDMDetailsPanelTabSpawner::GetEditorPath() const
{
	UDynamicMaterialModelBase* MaterialModelBase = GetMaterialModelBase();

	return MaterialModelBase ? MaterialModelBase->GetPathName() : "";
}

void SDMDetailsPanelTabSpawner::OnEditorChanged(const FAssetData& InAssetData)
{
	SetMaterialModelBase(Cast<UDynamicMaterialModelBase>(InAssetData.GetAsset()));
}

#undef LOCTEXT_NAMESPACE
