// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"

class FString;
class FText;
class IPropertyHandle;
class UDynamicMaterialModelBase;
struct FAssetData;

class SDMDetailsPanelTabSpawner : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDMDetailsPanelTabSpawner) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

protected:
	TSharedPtr<IPropertyHandle> PropertyHandle;

	UDynamicMaterialModelBase* GetMaterialModelBase() const;
	void SetMaterialModelBase(UDynamicMaterialModelBase* InNewModel);

	FReply OnButtonClicked();
	FReply CreateDynamicMaterialModel();
	FReply ClearDynamicMaterialModel();
	FReply OpenDynamicMaterialModelTab();

	FText GetButtonText() const;
	FString GetEditorPath() const;
	void OnEditorChanged(const FAssetData& InAssetData);
};
