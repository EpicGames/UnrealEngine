// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsPanel/Widgets/SDMMaterialListExtensionWidget.h"

#include "AssetToolsModule.h"
#include "Components/PrimitiveComponent.h"
#include "DMObjectMaterialProperty.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorStyle.h"
#include "Editor.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "IAssetTools.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "MaterialList.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialListExtensionWidget"

void SDMMaterialListExtensionWidget::Construct(const FArguments& InArgs, const TSharedRef<FMaterialItemView>& InMaterialItemView, 
	UPrimitiveComponent* InCurrentComponent, IDetailLayoutBuilder& InDetailBuilder)
{
	MaterialItemViewWeak = InMaterialItemView;
	CurrentComponentWeak = InCurrentComponent;

	if (!CurrentComponentWeak.IsValid())
	{
		return;
	}

	// @formatter:off
	ChildSlot
	[
		SNew(SBox)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(SButton)
			.OnClicked(this, &SDMMaterialListExtensionWidget::OnButtonClicked)
			.Content()
			[
				SNew(STextBlock)
				.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
				.Text(this, &SDMMaterialListExtensionWidget::GetButtonText)
			]
		]
	];
	// @formatter:on
}

UObject* SDMMaterialListExtensionWidget::GetAsset() const
{
	UPrimitiveComponent* CurrentComponent = CurrentComponentWeak.Get();
	if (!ensure(CurrentComponent))
	{
		return nullptr;
	}

	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeak.Pin();
	if (!ensure(MaterialItemView.IsValid()))
	{
		return nullptr;
	}

	UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialItemView->GetMaterialListItem().Material.Get());
	if (!MaterialInterface)
	{
		return nullptr;
	}

	return MaterialInterface;
}

UDynamicMaterialInstance* SDMMaterialListExtensionWidget::GetMaterialDesignerMaterial() const
{
	return Cast<UDynamicMaterialInstance>(GetAsset());
}

void SDMMaterialListExtensionWidget::SetAsset(UObject* NewAsset)
{
	UPrimitiveComponent* CurrentComponent = CurrentComponentWeak.Get();
	if (!ensure(CurrentComponent))
	{
		return;
	}

	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeak.Pin();
	if (!ensure(MaterialItemView.IsValid()))
	{
		return;
	}

	UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(NewAsset);
	if (!MaterialInterface)
	{
		return;
	}

	MaterialItemView->ReplaceMaterial(MaterialInterface);
}

void SDMMaterialListExtensionWidget::SetMaterialDesignerMaterial(UDynamicMaterialInstance* InMaterial)
{
	if (MaterialItemViewWeak.IsValid())
	{
		if (UPrimitiveComponent* CurrentComponent = CurrentComponentWeak.Get())
		{
			if (UWorld* World = CurrentComponent->GetWorld())
			{
				if (UDMWorldSubsystem* WorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>())
				{
					if (WorldSubsystem->GetInvokeTabDelegate().IsBound())
					{
						if (WorldSubsystem->GetMaterialValueSetterDelegate().IsBound())
						{
							const FMaterialListItem& ListItem = MaterialItemViewWeak.Pin()->GetMaterialListItem();
							const FDMObjectMaterialProperty MaterialProperty(CurrentComponent, ListItem.SlotIndex);

							if (WorldSubsystem->ExecuteMaterialValueSetterDelegate(MaterialProperty, InMaterial))
							{
								return;
							}
						}
					}
				}
			}
		}
	}

	SetAsset(InMaterial);
}

FText SDMMaterialListExtensionWidget::GetButtonText() const
{
	if (GetMaterialDesignerMaterial())
	{
		return LOCTEXT("OpenMaterialDesignerModel", "Edit with Material Designer");
	}

	return LOCTEXT("CreateMaterialDesignerModel", "Create with Material Designer");
}

FReply SDMMaterialListExtensionWidget::OnButtonClicked()
{
	if (GetMaterialDesignerMaterial())
	{
		return OpenMaterialDesignerTab();
	}

	return CreateMaterialDesignerMaterial();
}

FReply SDMMaterialListExtensionWidget::CreateMaterialDesignerMaterial()
{
	UDynamicMaterialInstance* Material = GetMaterialDesignerMaterial();

	// We already have an instance, so we don't need to create one
	if (Material)
	{
		return FReply::Handled();
	}

	TSharedPtr<FMaterialItemView> ListItem = MaterialItemViewWeak.Pin();

	if (!ListItem.IsValid())
	{
		return FReply::Handled();
	}

	UPrimitiveComponent* Component = CurrentComponentWeak.Get();

	if (!Component)
	{
		return FReply::Handled();
	}

	const FDMObjectMaterialProperty MaterialProperty(Component, ListItem->GetMaterialListItem().SlotIndex);

	constexpr bool bInvokeTab = true;

	const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
	MaterialDesignerModule.OpenMaterialObjectProperty(MaterialProperty, Component->GetWorld(), bInvokeTab);

	return FReply::Handled();
}

FReply SDMMaterialListExtensionWidget::ClearMaterialDesignerMaterial()
{
	UDynamicMaterialInstance* Material = GetMaterialDesignerMaterial();

	// We don't have an instance, so we don't need to clear it (or any other asset in its place)
	if (!Material)
	{
		return FReply::Handled();
	}

	SetMaterialDesignerMaterial(nullptr);

	return FReply::Handled();
}

FReply SDMMaterialListExtensionWidget::OpenMaterialDesignerTab()
{
	UPrimitiveComponent* CurrentComponent = CurrentComponentWeak.Get();
	if (!ensure(CurrentComponent))
	{
		return FReply::Handled();
	}

	const TSharedPtr<FMaterialItemView> MaterialItemView = MaterialItemViewWeak.Pin();
	if (!ensure(MaterialItemView.IsValid()))
	{
		return FReply::Handled();
	}

	if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		UDynamicMaterialInstance* MaterialInstance = Cast<UDynamicMaterialInstance>(CurrentComponent->GetMaterial(MaterialItemView->GetMaterialListItem().SlotIndex));
		if (!MaterialInstance)
		{
			return FReply::Handled();
		}

		UDynamicMaterialModelBase* MaterialModel = MaterialInstance->GetMaterialModelBase();
		if (!MaterialModel)
		{
			return FReply::Handled();
		}

		IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
		AssetTools.OpenEditorForAssets({MaterialModel->GetGeneratedMaterial()});

		return FReply::Handled();
	}

	IDynamicMaterialEditorModule::Get().OpenMaterialObjectProperty({CurrentComponent, MaterialItemView->GetMaterialListItem().SlotIndex},
		CurrentComponent->GetWorld(), /* Invoke Tab */ true);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
