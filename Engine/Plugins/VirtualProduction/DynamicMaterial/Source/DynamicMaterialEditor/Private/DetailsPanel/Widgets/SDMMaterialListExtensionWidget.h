// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FMaterialItemView;
class FText;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UDynamicMaterialInstance;
class UPrimitiveComponent;

class SDMMaterialListExtensionWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMMaterialListExtensionWidget) {}
	SLATE_END_ARGS()

	virtual ~SDMMaterialListExtensionWidget() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<FMaterialItemView>& InMaterialItemView, UPrimitiveComponent* InCurrentComponent,
		IDetailLayoutBuilder& InDetailBuilder);

protected:
	TWeakPtr<FMaterialItemView> MaterialItemViewWeak;
	TWeakObjectPtr<UPrimitiveComponent> CurrentComponentWeak;

	UObject* GetAsset() const;
	void SetAsset(UObject* NewAsset);

	UDynamicMaterialInstance* GetMaterialDesignerMaterial() const;
	void SetMaterialDesignerMaterial(UDynamicMaterialInstance* InMaterial);

	FReply OnButtonClicked();
	FReply CreateMaterialDesignerMaterial();
	FReply ClearMaterialDesignerMaterial();
	FReply OpenMaterialDesignerTab();

	FText GetButtonText() const;
};
