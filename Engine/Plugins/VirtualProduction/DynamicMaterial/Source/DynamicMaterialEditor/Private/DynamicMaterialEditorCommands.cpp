// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialEditorCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "MaterialDesignerCommands"

FDynamicMaterialEditorCommands::FDynamicMaterialEditorCommands()
	: TCommands<FDynamicMaterialEditorCommands>(TEXT("MaterialDesigner")
		, LOCTEXT("MaterialDesigner", "MaterialDesigner")
		, NAME_None
		, FAppStyle::GetAppStyleSetName()
	)
{
}

void FDynamicMaterialEditorCommands::RegisterCommands()
{
	UI_COMMAND(OpenEditorSettingsWindow
		, "Open Material Designer Settings..."
		, "Opens the Material Designer settings window."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(AddDefaultLayer
		, "Add Layer"
		, "Adds a new default texture layer at the top of the slot currently being edited."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Insert));

	UI_COMMAND(InsertDefaultLayerAbove
		, "Insert Layer Above"
		, "Inserts a new default texture layer above the selected layer in the slot currently being edited."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Insert, EModifierKey::Shift));

	UI_COMMAND(SelectLayerBaseStage
		, "Select Base Stage"
		, "Selects the base stage of the currently selected layer."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Comma));

	UI_COMMAND(SelectLayerMaskStage
		, "Select Mask Stage"
		, "Selects the mask stage of the currently selected layer."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Period));

	UI_COMMAND(MoveLayerUp
		, "Move Layer Up"
		, "Moves a layer up in the order, moving it closer to the top of the layer stack."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::LeftBracket, EModifierKey::Alt));

	UI_COMMAND(MoveLayerDown
		, "Move Layer Down"
		, "Moves a layer down in the order, moving it closer to the bottom of the layer stack."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::RightBracket, EModifierKey::Alt));

	UI_COMMAND(SetCustomPreviewMesh
		, "Custom Mesh"
		, "Sets the preview mesh to a custom mesh specified in the settings."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(NavigateForward
		, "Navigate Forward"
		, "Navigates forward in the page history."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::ThumbMouseButton2));

	UI_COMMAND(NavigateBack
		, "Navigate Back"
		, "Navigates back in the page history."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::ThumbMouseButton));

	TSharedPtr<FUICommandInfo> SelectLayer1;
	TSharedPtr<FUICommandInfo> SelectLayer2;
	TSharedPtr<FUICommandInfo> SelectLayer3;
	TSharedPtr<FUICommandInfo> SelectLayer4;
	TSharedPtr<FUICommandInfo> SelectLayer5;
	TSharedPtr<FUICommandInfo> SelectLayer6;
	TSharedPtr<FUICommandInfo> SelectLayer7;
	TSharedPtr<FUICommandInfo> SelectLayer8;
	TSharedPtr<FUICommandInfo> SelectLayer9;
	TSharedPtr<FUICommandInfo> SelectLayer10;

	UI_COMMAND(SelectLayer1
		, "Select Layer 1"
		, "Select the first layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::One));

	UI_COMMAND(SelectLayer2
		, "Select Layer 2"
		, "Select the second layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Two));

	UI_COMMAND(SelectLayer3
		, "Select Layer 3"
		, "Select the third layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Three));

	UI_COMMAND(SelectLayer4
		, "Select Layer 4"
		, "Select the fourth layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Four));

	UI_COMMAND(SelectLayer5
		, "Select Layer 5"
		, "Select the fifth layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Five));

	UI_COMMAND(SelectLayer6
		, "Select Layer 6"
		, "Select the sixth layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Six));

	UI_COMMAND(SelectLayer7
		, "Select Layer 7"
		, "Select the seventh layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Seven));

	UI_COMMAND(SelectLayer8
		, "Select Layer 8"
		, "Select the eighth layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Eight));

	UI_COMMAND(SelectLayer9
		, "Select Layer 9"
		, "Select the ninth layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Nine));

	UI_COMMAND(SelectLayer10
		, "Select Layer 10"
		, "Select the tenth layer (from the top of the stack)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Zero));

	SelectLayers = {SelectLayer1, SelectLayer2, SelectLayer3, SelectLayer4, SelectLayer5,
		SelectLayer6, SelectLayer7, SelectLayer8, SelectLayer9, SelectLayer10};

	TSharedPtr<FUICommandInfo> SetOpacity10;
	TSharedPtr<FUICommandInfo> SetOpacity20;
	TSharedPtr<FUICommandInfo> SetOpacity30;
	TSharedPtr<FUICommandInfo> SetOpacity40;
	TSharedPtr<FUICommandInfo> SetOpacity50;
	TSharedPtr<FUICommandInfo> SetOpacity60;
	TSharedPtr<FUICommandInfo> SetOpacity70;
	TSharedPtr<FUICommandInfo> SetOpacity80;
	TSharedPtr<FUICommandInfo> SetOpacity90;
	TSharedPtr<FUICommandInfo> SetOpacity100;

	UI_COMMAND(SetOpacity10
		, "Set Opacity to 10%"
		, "Sets the opacity of the selected layer to 10%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity20
		, "Set Opacity to 20%"
		, "Sets the opacity of the selected layer to 20%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity30
		, "Set Opacity to 30%"
		, "Sets the opacity of the selected layer to 30%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity40
		, "Set Opacity to 40%"
		, "Sets the opacity of the selected layer to 40%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity50
		, "Set Opacity to 50%"
		, "Sets the opacity of the selected layer to 50%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity60
		, "Set Opacity to 60%"
		, "Sets the opacity of the selected layer to 60%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity70
		, "Set Opacity to 70%"
		, "Sets the opacity of the selected layer to 70%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity80
		, "Set Opacity to 80%"
		, "Sets the opacity of the selected layer to 80%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity90
		, "Set Opacity to 90%"
		, "Sets the opacity of the selected layer to 90%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetOpacity100
		, "Set Opacity to 100%"
		, "Sets the opacity of the selected layer to 100%."
		, EUserInterfaceActionType::Button
		, FInputChord());

	/** Cannot find the key directly as then it will activate without the fake modifier key, 'V'. */
	SetOpacities = {
		{EKeys::One,   {0.1f, SetOpacity10.ToSharedRef()}},
		{EKeys::Two,   {0.2f, SetOpacity20.ToSharedRef()}},
		{EKeys::Three, {0.3f, SetOpacity30.ToSharedRef()}},
		{EKeys::Four,  {0.4f, SetOpacity40.ToSharedRef()}},
		{EKeys::Five,  {0.5f, SetOpacity50.ToSharedRef()}},
		{EKeys::Six,   {0.6f, SetOpacity60.ToSharedRef()}},
		{EKeys::Seven, {0.7f, SetOpacity70.ToSharedRef()}},
		{EKeys::Eight, {0.8f, SetOpacity80.ToSharedRef()}},
		{EKeys::Nine,  {0.9f, SetOpacity90.ToSharedRef()}},
		{EKeys::Zero,  {1.0f, SetOpacity100.ToSharedRef()}}
	};
}

#undef LOCTEXT_NAMESPACE
