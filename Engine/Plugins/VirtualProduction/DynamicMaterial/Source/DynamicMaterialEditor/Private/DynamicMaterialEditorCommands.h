// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FDynamicMaterialEditorCommands : public TCommands<FDynamicMaterialEditorCommands>
{
public:
	FDynamicMaterialEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> OpenEditorSettingsWindow;
	TSharedPtr<FUICommandInfo> AddDefaultLayer;
	TSharedPtr<FUICommandInfo> InsertDefaultLayerAbove;
	TSharedPtr<FUICommandInfo> SelectLayerBaseStage;
	TSharedPtr<FUICommandInfo> SelectLayerMaskStage;
	TSharedPtr<FUICommandInfo> MoveLayerUp;
	TSharedPtr<FUICommandInfo> MoveLayerDown;

	TSharedPtr<FUICommandInfo> SetCustomPreviewMesh;

	TSharedPtr<FUICommandInfo> NavigateForward;
	TSharedPtr<FUICommandInfo> NavigateBack;

	// Individual commands are initialised as SelectLayerX
	TArray<TSharedPtr<FUICommandInfo>, TInlineAllocator<10>> SelectLayers;

	struct FOpacityCommand
	{
		const float Opacity;
		TSharedRef<FUICommandInfo> Command;
	};

	// Individual commands are initialised as SetOpacityX
	TMap<FKey, FOpacityCommand> SetOpacities;
};
