// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialEditorModule.h"

#include "Components/ActorComponent.h"
#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialEffectFunction.h"
#include "Components/DMMaterialStageThroughput.h"
#include "Components/DMMaterialValueDynamic.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "Components/MaterialStageInputs/DMMSIFunction.h"
#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Components/PrimitiveComponent.h"
#include "DetailsPanel/DMMaterialInterfaceTypeCustomizer.h"
#include "DetailsPanel/DMPropertyTypeCustomizer.h"
#include "DetailsPanel/DMValueDetailsRowExtensions.h"
#include "DetailsPanel/Widgets/SDMMaterialListExtensionWidget.h"
#include "DMContentBrowserIntegration.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorCommands.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "DynamicMaterialModule.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "LevelEditor/DMLevelEditorIntegration.h"
#include "Material/DynamicMaterialInstance.h"
#include "MaterialList.h"
#include "Model/DMMaterialModelDefaults.h"
#include "Model/DMOnWizardCompleteCallback.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMInputThroughputPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMMaterialEffectFunctionPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMMaterialStageFunctionPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMMaterialValueDynamicPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMMaterialValuePropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMStagePropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMTextureUVDynamicPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMTextureUVPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMThroughputPropertyRowGenerator.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Utils/DynamicMaterialInstanceThumbnailRenderer.h"
#include "UI/Widgets/SDMMaterialDesigner.h"

DEFINE_LOG_CATEGORY(LogDynamicMaterialEditor);

namespace UE::DynamicMaterialEditor::Private
{
	FDelegateHandle MateriaListWidgetsDelegate;

	void AddMaterialListWidgets(const TSharedRef<FMaterialItemView>& InMaterialItemView,
		UActorComponent* InCurrentComponent,
		IDetailLayoutBuilder& InDetailBuilder,
		TArray<TSharedPtr<SWidget>>& OutExtensions)
	{
		const UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

		if (!Settings || !Settings->bAddDetailsPanelButton)
		{
			return;
		}

		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InCurrentComponent))
		{
			OutExtensions.Add(SNew(SDMMaterialListExtensionWidget,
				InMaterialItemView,
				PrimitiveComponent,
				InDetailBuilder
			));
		}
	}
}

const FName FDynamicMaterialEditorModule::TabId = TEXT("MaterialDesigner");
TMap<UClass*, FDMComponentPropertyRowGeneratorDelegate> FDynamicMaterialEditorModule::ComponentPropertyRowGenerators;
TMap<UClass*, FDMGetObjectMaterialPropertiesDelegate> FDynamicMaterialEditorModule::CustomMaterialPropertyGenerators;
FDMOnUIValueUpdate FDynamicMaterialEditorModule::OnUIValueUpdate;
TArray<TSharedRef<IDMOnWizardCompleteCallback>> FDynamicMaterialEditorModule::OnWizardCompleteCallbacks;

void FDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate(UClass* InClass, FDMComponentPropertyRowGeneratorDelegate InComponentPropertyRowGeneratorDelegate)
{
	ComponentPropertyRowGenerators.Emplace(InClass, InComponentPropertyRowGeneratorDelegate);
}

FDMComponentPropertyRowGeneratorDelegate FDynamicMaterialEditorModule::GetComponentPropertyRowGeneratorDelegate(UClass* InClass)
{
	UClass* FoundClass = nullptr;

	for (const TPair<UClass*, FDMComponentPropertyRowGeneratorDelegate>& Pair : ComponentPropertyRowGenerators)
	{
		// If we have an exact match, return the delegate immediately.
		if (Pair.Key == InClass)
		{
			return Pair.Value;
		}

		if (InClass->IsChildOf(Pair.Key))
		{
			// If we haven't got anything or the current class is a more specific generator.
			if (FoundClass == nullptr || Pair.Key->IsChildOf(FoundClass))
			{
				FoundClass = Pair.Key;
			}
		}
	}

	if (!FoundClass)
	{
		// Return an invalid lambda
		static FDMComponentPropertyRowGeneratorDelegate NullLambda;
		return NullLambda;
	}

	return ComponentPropertyRowGenerators[FoundClass];
}

void FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	FDMComponentPropertyRowGeneratorDelegate RowGenerator = GetComponentPropertyRowGeneratorDelegate(InParams.Object->GetClass());
	RowGenerator.ExecuteIfBound(InParams);
}

void FDynamicMaterialEditorModule::RegisterCustomMaterialPropertyGenerator(UClass* InClass, FDMGetObjectMaterialPropertiesDelegate InGenerator)
{
	if (!InClass || !InGenerator.IsBound())
	{
		return;
	}

	CustomMaterialPropertyGenerators.FindOrAdd(InClass) = InGenerator;
}

void FDynamicMaterialEditorModule::RegisterMaterialModelCreatedCallback(const TSharedRef<IDMOnWizardCompleteCallback> InCallback)
{
	OnWizardCompleteCallbacks.Add(InCallback);

	OnWizardCompleteCallbacks.StableSort(
		[](const TSharedRef<IDMOnWizardCompleteCallback>& InA, const TSharedRef<IDMOnWizardCompleteCallback>& InB)
		{
			return InA.Get() < InB.Get();
		}
	);
}

void FDynamicMaterialEditorModule::UnregisterMaterialModelCreatedCallback(const TSharedRef<IDMOnWizardCompleteCallback> InCallback)
{
	OnWizardCompleteCallbacks.Remove(InCallback);
}

void FDynamicMaterialEditorModule::OnWizardComplete(UDynamicMaterialModel* InModel)
{
	if (!IsValid(InModel))
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(InModel);
	UObject* Outer = InModel->GetOuter();
	UActorComponent* OuterComponent = InModel->GetTypedOuter<UActorComponent>();
	AActor* OuterActor = InModel->GetTypedOuter<AActor>();

	const FDMOnWizardCompleteCallbackParams Params = {
		InModel,
		EditorOnlyData,
		Outer,
		OuterComponent,
		OuterActor
	};

	for (const TSharedRef<IDMOnWizardCompleteCallback>& OnWizardCompleteCallback : OnWizardCompleteCallbacks)
	{
		OnWizardCompleteCallback->OnModelCreated(Params);
	}
}

FDMGetObjectMaterialPropertiesDelegate FDynamicMaterialEditorModule::GetCustomMaterialPropertyGenerator(UClass* InClass)
{
	if (InClass)
	{
		if (CustomMaterialPropertyGenerators.Contains(InClass))
		{
			return CustomMaterialPropertyGenerators[InClass];
		}
	}

	return FDMGetObjectMaterialPropertiesDelegate();
}

FDynamicMaterialEditorModule& FDynamicMaterialEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<FDynamicMaterialEditorModule>(ModuleName);
}

FDynamicMaterialEditorModule::FDynamicMaterialEditorModule()
	: CommandList(MakeShared<FUICommandList>())
{
}

void FDynamicMaterialEditorModule::StartupModule()
{
	FDynamicMaterialEditorStyle::Get();
	FDynamicMaterialEditorCommands::Register();
	FDMContentBrowserIntegration::Integrate();
	MapCommands();

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout(UDynamicMaterialModelEditorOnlyData::StaticClass()->GetFName()
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMPropertyTypeCustomizer::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(UMaterialInterface::StaticClass()->GetFName()
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMMaterialInterfaceTypeCustomizer::MakeInstance), MakeShared<FDMMaterialInterfaceTypeIdentifier>());

	using namespace UE::DynamicMaterialEditor::Private;

	MateriaListWidgetsDelegate = FMaterialList::OnAddMaterialItemViewExtraBottomWidget.AddStatic(&AddMaterialListWidgets);

	FDMLevelEditorIntegration::Initialize();

	FDMMaterialModelDefaults::RegisterDefaultsDelegates();

	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialComponent,            FDMComponentPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialStage,                FDMStagePropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialValue,                FDMMaterialValuePropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialValueDynamic,         FDMMaterialValueDynamicPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMTextureUV,                    FDMTextureUVPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMTextureUVDynamic,             FDMTextureUVDynamicPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialStageThroughput,      FDMThroughputPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialStageInputThroughput, FDMInputThroughputPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialEffectFunction,       FDMMaterialEffectFunctionPropertyRowGenerator>();
	IDynamicMaterialEditorModule::RegisterComponentPropertyRowGeneratorDelegate<UDMMaterialStageInputFunction,   FDMMaterialStageFunctionPropertyRowGenerator>();

	UDMMaterialValueTexture::GetDefaultRGBTexture.BindLambda([]()
		{
			const FDMDefaultMaterialPropertySlotValue& DefaultValue = UDynamicMaterialEditorSettings::Get()->GetDefaultSlotValue(EDMMaterialPropertyType::BaseColor);

			if (UTexture* DefaultTexture = DefaultValue.Texture.LoadSynchronous())
			{
				return DefaultTexture;
			}

			return UDynamicMaterialEditorSettings::Get()->DefaultMask.LoadSynchronous();
		});

	FDMValueDetailsRowExtensions::Get().RegisterRowExtensions();

	UThumbnailManager::Get().RegisterCustomRenderer(UDynamicMaterialInstance::StaticClass(), UDynamicMaterialInstanceThumbnailRenderer::StaticClass());
}

void FDynamicMaterialEditorModule::ShutdownModule()
{
	FDynamicMaterialEditorCommands::Unregister();
	FDMContentBrowserIntegration::Disintegrate();

	if (FDynamicMaterialModule::AreUObjectsSafe())
	{
		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomPropertyTypeLayout(UDynamicMaterialModelEditorOnlyData::StaticClass()->GetFName());
		}

		UThumbnailManager::Get().UnregisterCustomRenderer(UDynamicMaterialInstance::StaticClass());
	}

	using namespace UE::DynamicMaterialEditor::Private;
	FMaterialList::OnAddMaterialItemViewExtraBottomWidget.Remove(MateriaListWidgetsDelegate);

	FDMLevelEditorIntegration::Shutdown();

	FDMMaterialModelDefaults::UnregsiterDefaultsDelegates();

	UDMMaterialValueTexture::GetDefaultRGBTexture.Unbind();

	FDMValueDetailsRowExtensions::Get().UnregisterRowExtensions();

	FDMWidgetLibrary::Get().ClearData();
}

void FDynamicMaterialEditorModule::OpenMaterialModel(UDynamicMaterialModelBase* InMaterialModel, UWorld* InWorld, bool bInInvokeTab) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (TSharedPtr<SDMMaterialDesigner> Designer = FDMLevelEditorIntegration::GetMaterialDesignerForWorld(InWorld))
	{
		if (bInInvokeTab)
		{
			FDMLevelEditorIntegration::InvokeTabForWorld(InWorld);
		}

		Designer->OpenMaterialModelBase(InMaterialModel);
	}
	else if (IsValid(InWorld))
	{
		if (UDMWorldSubsystem* DMWorldSubsystem = InWorld->GetSubsystem<UDMWorldSubsystem>())
		{
			if (bInInvokeTab)
			{
				DMWorldSubsystem->ExecuteInvokeTabDelegate();
			}

			DMWorldSubsystem->ExecuteSetCustomEditorModelDelegate(InMaterialModel);
		}
	}
}

void FDynamicMaterialEditorModule::OpenMaterialObjectProperty(const FDMObjectMaterialProperty& InObjectProperty,
	UWorld* InWorld, bool bInInvokeTab) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (TSharedPtr<SDMMaterialDesigner> Designer = FDMLevelEditorIntegration::GetMaterialDesignerForWorld(InWorld))
	{
		if (bInInvokeTab)
		{
			FDMLevelEditorIntegration::InvokeTabForWorld(InWorld);
		}

		Designer->OpenObjectMaterialProperty(InObjectProperty);
	}
	else if (IsValid(InWorld))
	{
		if (UDMWorldSubsystem* DMWorldSubsystem = InWorld->GetSubsystem<UDMWorldSubsystem>())
		{
			if (bInInvokeTab)
			{
				DMWorldSubsystem->ExecuteInvokeTabDelegate();
			}

			DMWorldSubsystem->ExecuteCustomObjectPropertyEditorDelegate(InObjectProperty);
		}
	}
}

void FDynamicMaterialEditorModule::OpenMaterial(UDynamicMaterialInstance* InMaterial, UWorld* InWorld, 
	bool bInInvokeTab) const
{
	if (IsValid(InMaterial))
	{
		if (UDynamicMaterialModel* InstanceModel = InMaterial->GetMaterialModel())
		{
			OpenMaterialModel(InstanceModel, InWorld, bInInvokeTab);
		}
	}
}

void FDynamicMaterialEditorModule::OnActorSelected(AActor* InActor, UWorld* InWorld, bool bInInvokeTab) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (TSharedPtr<SDMMaterialDesigner> Designer = FDMLevelEditorIntegration::GetMaterialDesignerForWorld(InWorld))
	{
		if (bInInvokeTab)
		{
			FDMLevelEditorIntegration::InvokeTabForWorld(InWorld);
		}

		Designer->OnActorSelected(InActor);
	}
	else if (IsValid(InWorld))
	{
		if (UDMWorldSubsystem* DMWorldSubsystem = InWorld->GetSubsystem<UDMWorldSubsystem>())
		{
			if (bInInvokeTab)
			{
				DMWorldSubsystem->ExecuteInvokeTabDelegate();
			}

			DMWorldSubsystem->ExecuteSetCustomEditorActorDelegate(InActor);
		}
	}
}

void FDynamicMaterialEditorModule::ClearDynamicMaterialModel(UWorld* InWorld) const
{
	OpenMaterialModel(nullptr, InWorld, /* Invoke tab */ false);
}

IDMWidgetLibrary& FDynamicMaterialEditorModule::GetWidgetLibrary() const
{
	return FDMWidgetLibrary::Get();
}

TSharedRef<SWidget> FDynamicMaterialEditorModule::CreateEditor(UDynamicMaterialModelBase* InMaterialModelBase, UWorld* InAssetEditorWorld)
{
	TSharedRef<SDMMaterialDesigner> NewDesigner = SNew(SDMMaterialDesigner);
	NewDesigner->OpenMaterialModelBase(InMaterialModelBase);

	if (IsValid(InAssetEditorWorld))
	{
		UDMWorldSubsystem* WorldSubsystem = InAssetEditorWorld->GetSubsystem<UDMWorldSubsystem>();

		if (IsValid(WorldSubsystem))
		{
			WorldSubsystem->GetGetCustomEditorModelDelegate().BindSP(NewDesigner, &SDMMaterialDesigner::GetOriginalMaterialModelBase);
			WorldSubsystem->GetSetCustomEditorActorDelegate().BindSP(NewDesigner, &SDMMaterialDesigner::OnActorSelected);

			TWeakPtr<SDMMaterialDesigner> NewDesignerWeak = NewDesigner;

			WorldSubsystem->GetSetCustomEditorModelDelegate().BindSPLambda(
				NewDesigner, 
				[NewDesignerWeak](UDynamicMaterialModelBase* InMaterialModelBase)
				{
					NewDesignerWeak.Pin()->OpenMaterialModelBase(InMaterialModelBase);
				});

			WorldSubsystem->GetCustomObjectPropertyEditorDelegate().BindSPLambda(
				NewDesigner,
				[NewDesignerWeak](const FDMObjectMaterialProperty& InObjectProperty)
				{
					NewDesignerWeak.Pin()->OpenObjectMaterialProperty(InObjectProperty);
				});
		}
	}

	return NewDesigner;
}

void FDynamicMaterialEditorModule::OpenEditor(UWorld* InWorld) const
{
	if (!IsValid(InWorld))
	{
		FDMLevelEditorIntegration::InvokeTabForWorld(InWorld);
	}
	else if (UDMWorldSubsystem* DMWorldSubsystem = InWorld->GetSubsystem<UDMWorldSubsystem>())
	{
		DMWorldSubsystem->ExecuteInvokeTabDelegate();
	}
}

UDynamicMaterialModelBase* FDynamicMaterialEditorModule::GetOpenedMaterialModel(UWorld* InWorld) const
{
	if (TSharedPtr<SDMMaterialDesigner> Designer = FDMLevelEditorIntegration::GetMaterialDesignerForWorld(InWorld))
	{
		return Designer->GetOriginalMaterialModelBase();
	}

	if (IsValid(InWorld))
	{
		if (UDMWorldSubsystem* DMWorldSubsystem = InWorld->GetSubsystem<UDMWorldSubsystem>())
		{
			return DMWorldSubsystem->ExecuteGetCustomEditorModelDelegate();
		}
	}

	return nullptr;
}

void FDynamicMaterialEditorModule::MapCommands()
{
	const FDynamicMaterialEditorCommands& DMEditorCommands = FDynamicMaterialEditorCommands::Get();

	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	CommandList->MapAction(
		DMEditorCommands.OpenEditorSettingsWindow,
		FExecuteAction::CreateUObject(Settings, &UDynamicMaterialEditorSettings::OpenEditorSettingsWindow)
	);
}

void FDynamicMaterialEditorModule::UnmapCommands()
{
	const FDynamicMaterialEditorCommands& DMEditorCommands = FDynamicMaterialEditorCommands::Get();

	auto UnmapAction = [this](const TSharedPtr<const FUICommandInfo>& InCommandInfo)
	{
		if (CommandList->IsActionMapped(InCommandInfo))
		{
			CommandList->UnmapAction(InCommandInfo);
		}
	};

	UnmapAction(DMEditorCommands.OpenEditorSettingsWindow);
}

IMPLEMENT_MODULE(FDynamicMaterialEditorModule, DynamicMaterialEditor)
