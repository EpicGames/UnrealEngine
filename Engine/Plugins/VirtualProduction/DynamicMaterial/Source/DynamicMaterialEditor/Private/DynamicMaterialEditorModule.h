// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDynamicMaterialEditorModule.h"

#include "DMEDefs.h"
#include "Templates/SharedPointer.h"

class AActor;
class FDMMaterialFunctionLibrary;
class FUICommandList;
class IAssetTypeActions;
class ILevelEditor;
class SDMMaterialComponentEditor;
class SDMMaterialEditor;
class SDockTab;
class SWidget;
class UDMMaterialComponent;
class UDMMaterialStageSource;
class UDynamicMaterialInstance;
class UDynamicMaterialModelBase;
struct FDMPropertyHandleGenerateParams;

DECLARE_LOG_CATEGORY_EXTERN(LogDynamicMaterialEditor, Log, All);

namespace UE::DynamicMaterialEditor
{
	constexpr bool bMultipleSlotPropertiesEnabled = false;
	constexpr bool bGlobalValuesEnabled = false;
	constexpr bool bAdvancedSlotsEnabled = false;
}

DECLARE_MULTICAST_DELEGATE(FDMOnUIValueUpdate);

/** Takes a UMaterialValue and returns the widget used to edit it. */
DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<SWidget>, FDMCreateValueEditWidgetDelegate, const TSharedPtr<SDMMaterialComponentEditor>&, UDMMaterialValue*);

/**
 * Material Designer - Build your own materials in a slimline editor!
 */
class FDynamicMaterialEditorModule : public IDynamicMaterialEditorModule
{
public:
	static const FName TabId;
	static FDMOnUIValueUpdate::RegistrationType& GetOnUIValueUpdate() { return OnUIValueUpdate; }

	static FDynamicMaterialEditorModule& Get();

	static FDMComponentPropertyRowGeneratorDelegate GetComponentPropertyRowGeneratorDelegate(UClass* InClass);
	static void GeneratorComponentPropertyRows(FDMComponentPropertyRowGeneratorParams& InParams);

	static FDMGetObjectMaterialPropertiesDelegate GetCustomMaterialPropertyGenerator(UClass* InClass);

	/** With a provided world, the editor will bind to the MD world subsystem to receive model changes. */
	static TSharedRef<SWidget> CreateEditor(UDynamicMaterialModelBase* InMaterialModelBase, UWorld* InAssetEditorWorld);

	FDynamicMaterialEditorModule();

	const TSharedRef<FUICommandList>& GetCommandList() const { return CommandList; }

	void OnWizardComplete(UDynamicMaterialModel* InModel);

	//~ Begin IDynamicMaterialEditorModule
	virtual void RegisterComponentPropertyRowGeneratorDelegate(UClass* InClass, FDMComponentPropertyRowGeneratorDelegate InComponentPropertyRowGeneratorDelegate) override;
	virtual void RegisterCustomMaterialPropertyGenerator(UClass* InClass, FDMGetObjectMaterialPropertiesDelegate InGenerator) override;
	virtual void RegisterMaterialModelCreatedCallback(const TSharedRef<IDMOnWizardCompleteCallback> InCallback)  override;
	virtual void UnregisterMaterialModelCreatedCallback(const TSharedRef<IDMOnWizardCompleteCallback> InCallback) override;
	virtual void OpenEditor(UWorld* InWorld) const override;
	virtual UDynamicMaterialModelBase* GetOpenedMaterialModel(UWorld* InWorld) const override;
	virtual void OpenMaterialModel(UDynamicMaterialModelBase* InMaterialModel, UWorld* InWorld, bool bInInvokeTab) const override;
	virtual void OpenMaterialObjectProperty(const FDMObjectMaterialProperty& InObjectProperty, UWorld* InWorld, bool bInInvokeTab) const override;
	virtual void OpenMaterial(UDynamicMaterialInstance* InMaterial, UWorld* InWorld, bool bInInvokeTab) const override;
	virtual void OnActorSelected(AActor* InActor, UWorld* InWorld, bool bInInvokeTab) const override;
	virtual void ClearDynamicMaterialModel(UWorld* InWorld) const override;
	virtual IDMWidgetLibrary& GetWidgetLibrary() const override;
	//~ End IDynamicMaterialEditorModule

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

protected:
	static TMap<UClass*, FDMComponentPropertyRowGeneratorDelegate> ComponentPropertyRowGenerators;
	static TMap<UClass*, FDMGetObjectMaterialPropertiesDelegate> CustomMaterialPropertyGenerators;
	static FDMOnUIValueUpdate OnUIValueUpdate;
	static TArray<TSharedRef<IDMOnWizardCompleteCallback>> OnWizardCompleteCallbacks;

	TSharedRef<FUICommandList> CommandList;

	void MapCommands();
	void UnmapCommands();
};
