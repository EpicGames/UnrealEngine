// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialEditorSettings.h"

#include "AssetRegistry/AssetData.h"
#include "Components/DMMaterialComponent.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "DynamicMaterialEditorModule.h"
#include "Engine/AssetManager.h"
#include "GenericPlatform/GenericApplication.h"
#include "Interfaces/IPluginManager.h"
#include "ISettingsModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Misc/Paths.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialEditorSettings)

#define LOCTEXT_NAMESPACE "MaterialDesignerSettings"

namespace UE::DynamicMaterialEditor::Private
{
	const TMap<EDMMaterialPropertyType, FDMDefaultMaterialPropertySlotValue> DefaultSlotValues = {
		{EDMMaterialPropertyType::BaseColor,           TSoftObjectPtr<UTexture>(FSoftObjectPath(TEXT("/Script/Engine.Texture2D'/DynamicMaterial/T_Default_Texture.T_Default_Texture'")))},
		{EDMMaterialPropertyType::EmissiveColor,       TSoftObjectPtr<UTexture>(FSoftObjectPath(TEXT("/Script/Engine.Texture2D'/DynamicMaterial/Textures/SlotDefaults/T_MD_Emissive.T_MD_Emissive'")))},
		{EDMMaterialPropertyType::Opacity,             FLinearColor::White},
		{EDMMaterialPropertyType::OpacityMask,         FLinearColor::White},
		{EDMMaterialPropertyType::Metallic,            FLinearColor::Black},
		{EDMMaterialPropertyType::Specular,            FLinearColor::Gray},
		{EDMMaterialPropertyType::Roughness,           FLinearColor::Gray},
		{EDMMaterialPropertyType::Normal,              FLinearColor::Blue}, // +Z
		{EDMMaterialPropertyType::AmbientOcclusion,    FLinearColor::White},
		{EDMMaterialPropertyType::Displacement,        FLinearColor::Gray},
		{EDMMaterialPropertyType::SubsurfaceColor,     FLinearColor::Black},
		{EDMMaterialPropertyType::SurfaceThickness,    FLinearColor::Black},
	};
}

FDMDefaultMaterialPropertySlotValue::FDMDefaultMaterialPropertySlotValue()
	: DefaultType(EDMDefaultMaterialPropertySlotValueType::Texture)
	, Color(FLinearColor::White)
{
}

FDMDefaultMaterialPropertySlotValue::FDMDefaultMaterialPropertySlotValue(const TSoftObjectPtr<UTexture>& InTexture)
	: DefaultType(EDMDefaultMaterialPropertySlotValueType::Texture)
	, Texture(InTexture)
	, Color(FLinearColor::Black)
{
}

FDMDefaultMaterialPropertySlotValue::FDMDefaultMaterialPropertySlotValue(const FLinearColor& InColor)
	: DefaultType(EDMDefaultMaterialPropertySlotValueType::Color)
	, Color(InColor)
{
}

FDMDefaultMaterialPropertySlotValue::FDMDefaultMaterialPropertySlotValue(EDMDefaultMaterialPropertySlotValueType InDefaultType, 
	const TSoftObjectPtr<UTexture>& InTexture, const FLinearColor& InColor)
	: DefaultType(InDefaultType)
	, Texture(InTexture)
	, Color(InColor)
{
}

bool FDMMaterialChannelListPreset::IsPropertyEnabled(EDMMaterialPropertyType InProperty) const
{
	switch (InProperty)
	{
		case EDMMaterialPropertyType::BaseColor:
			return bBaseColor;

		case EDMMaterialPropertyType::EmissiveColor:
			return bEmissive;

		case EDMMaterialPropertyType::Opacity:
		case EDMMaterialPropertyType::OpacityMask:
			return bOpacity;

		case EDMMaterialPropertyType::Roughness:
			return bRoughness;

		case EDMMaterialPropertyType::Specular:
			return bSpecular;

		case EDMMaterialPropertyType::Metallic:
			return bMetallic;

		case EDMMaterialPropertyType::Normal:
			return bNormal;

		case EDMMaterialPropertyType::PixelDepthOffset:
			return bPixelDepthOffset;

		case EDMMaterialPropertyType::WorldPositionOffset:
			return bWorldPositionOffset;

		case EDMMaterialPropertyType::AmbientOcclusion:
			return bAmbientOcclusion;

		case EDMMaterialPropertyType::Anisotropy:
			return bAnisotropy;

		case EDMMaterialPropertyType::Refraction:
			return bRefraction;

		case EDMMaterialPropertyType::Tangent:
			return bTangent;

		case EDMMaterialPropertyType::Displacement:
			return bDisplacement;

		case EDMMaterialPropertyType::SubsurfaceColor:
			return bSubsurfaceColor;

		case EDMMaterialPropertyType::SurfaceThickness:
			return bSurfaceThickness;

		default:
			return false;
	}
}

UDynamicMaterialEditorSettings::UDynamicMaterialEditorSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("Material Designer");

	bFollowSelection = true;
	bAddDetailsPanelButton = false;
	bAutomaticallyCopyParametersToSourceMaterial = true;
	bAutomaticallyCompilePreviewMaterial = false;
	bAutomaticallyApplyToSourceOnPreviewCompile = true;
	LiveEditMode = EDMLiveEditMode::LiveEditOn;
	bUseLinearColorForVectors = true;

	ResetAllLayoutSettings();

	DefaultMask = TSoftObjectPtr<UTexture>(FSoftObjectPath(TEXT("/Script/Engine.Texture2D'/Engine/EngineResources/WhiteSquareTexture.WhiteSquareTexture'")));

	CustomPreviewMesh = TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(TEXT("/Script/Engine.StaticMesh'/Engine/EngineMeshes/SM_MatPreviewMesh_01.SM_MatPreviewMesh_01'")));

	FDMMaterialChannelListPreset Opaque;
	Opaque.Name = TEXT("Opaque");
	Opaque.bBaseColor = true;
	Opaque.bEmissive = true;
	Opaque.DefaultBlendMode = BLEND_Opaque;
	Opaque.DefaultShadingModel = EDMMaterialShadingModel::DefaultLit;
	Opaque.UsageFlag |= EDMMaterialFlagsUsage::DMMFU_StaticMesh | EDMMaterialFlagsUsage::DMMFU_NiagaraMeshParticles;
	Opaque.GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);

	FDMMaterialChannelListPreset Emissive;
	Emissive.Name = TEXT("Emissive");
	Emissive.bEmissive = true;
	Emissive.DefaultBlendMode = BLEND_Opaque;
	Emissive.DefaultShadingModel = EDMMaterialShadingModel::Unlit;
	Emissive.UsageFlag |= EDMMaterialFlagsUsage::DMMFU_StaticMesh | EDMMaterialFlagsUsage::DMMFU_NiagaraMeshParticles;
	Emissive.GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);

	FDMMaterialChannelListPreset Translucent;
	Translucent.Name = TEXT("Translucent");
	Translucent.bEmissive = true;
	Translucent.bOpacity = true;
	Translucent.DefaultBlendMode = BLEND_Translucent;
	Translucent.DefaultShadingModel = EDMMaterialShadingModel::Unlit;
	Translucent.UsageFlag |= EDMMaterialFlagsUsage::DMMFU_StaticMesh | EDMMaterialFlagsUsage::DMMFU_NiagaraMeshParticles;
	Translucent.GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);

	FDMMaterialChannelListPreset PBR;
	PBR.Name = TEXT("PBR");
	PBR.bBaseColor = true;
	PBR.bEmissive = true;
	PBR.bOpacity = true;
	PBR.bMetallic = true;
	PBR.bSpecular = true;
	PBR.bRoughness = true;
	PBR.bNormal = true;
	PBR.bAmbientOcclusion = true;
	PBR.bDisplacement = true;
	PBR.DefaultBlendMode = BLEND_Opaque;
	PBR.DefaultShadingModel = EDMMaterialShadingModel::DefaultLit;
	PBR.UsageFlag |= EDMMaterialFlagsUsage::DMMFU_StaticMesh | EDMMaterialFlagsUsage::DMMFU_NiagaraMeshParticles;
	PBR.GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);

	FDMMaterialChannelListPreset All;
	All.Name = TEXT("All");
	All.bBaseColor = true;
	All.bEmissive = true;
	All.bOpacity = true;
	All.bMetallic = true;
	All.bSpecular = true;
	All.bRoughness = true;
	All.bNormal = true;
	All.bAmbientOcclusion = true;
	All.bAnisotropy = true;
	All.bPixelDepthOffset = true;
	All.bRefraction = true;
	All.bTangent = true;
	All.bWorldPositionOffset = true;
	All.bDisplacement = true;
	All.bSubsurfaceColor = true;
	All.bSurfaceThickness = true;
	All.DefaultBlendMode = BLEND_Opaque;
	All.DefaultShadingModel = EDMMaterialShadingModel::DefaultLit;
	All.UsageFlag |= EDMMaterialFlagsUsage::DMMFU_StaticMesh | EDMMaterialFlagsUsage::DMMFU_NiagaraMeshParticles;
	All.GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);

	MaterialChannelPresets.Add(PBR);
	MaterialChannelPresets.Add(Opaque);
	MaterialChannelPresets.Add(Emissive);
	MaterialChannelPresets.Add(Translucent);
	MaterialChannelPresets.Add(All);		
}

UDynamicMaterialEditorSettings* UDynamicMaterialEditorSettings::Get()
{
	UDynamicMaterialEditorSettings* DefaultSettings = GetMutableDefault<UDynamicMaterialEditorSettings>();
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		DefaultSettings->SetFlags(RF_Transactional);
	}
	return DefaultSettings;
}

bool UDynamicMaterialEditorSettings::IsUseLinearColorForVectorsEnabled()
{
	if (const UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		return Settings->bUseLinearColorForVectors;
	}

	return true;
}

void UDynamicMaterialEditorSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if (!bValidatedPresets)
	{
		for (FDMMaterialChannelListPreset& Preset : MaterialChannelPresets)
		{
			Preset.bBaseColor = Preset.Name != TEXT("Emissive");
			Preset.bEmissive = true;
		}

		bValidatedPresets = true;
	}

	if (!bConvertedVarsToFlags)
	{
		ConvertDeprecatedVarsToFlags();
		bConvertedVarsToFlags = true;
	}
}

void UDynamicMaterialEditorSettings::PreEditChange(FEditPropertyChain& InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);

	PreEditPresetNames.Empty();
	PreEditPresetNames.Reserve(MaterialChannelPresets.Num());

	for (const FDMMaterialChannelListPreset& Preset : MaterialChannelPresets)
	{
		PreEditPresetNames.Add(Preset.Name);
	}
}

void UDynamicMaterialEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, Layout)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, bUseFullChannelNamesInTopSlimLayout)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, SplitterLocation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, bPreviewImagesUseTextureUVs)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, PreviewMesh)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, bShowPreviewBackground)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, StagePreviewSize)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, PropertyPreviewSize))
	{
		OnSettingsChanged.Broadcast(InPropertyChangedEvent);
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, bPreviewImagesUseTextureUVs))
	{
		for (UDMMaterialComponent* Component : TObjectRange<UDMMaterialComponent>())
		{
			Component->MarkComponentDirty();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, MaterialChannelPresets))
	{
		EnsureUniqueChannelPresetNames();
	}
}

void UDynamicMaterialEditorSettings::OpenEditorSettingsWindow() const
{
	static ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	SettingsModule.ShowViewer(GetContainerName(), CategoryName, SectionName);
}

void UDynamicMaterialEditorSettings::ResetAllLayoutSettings()
{
	Layout = EDMMaterialEditorLayout::Top;
	bUseFullChannelNamesInTopSlimLayout = false;
	SplitterLocation = 0.5;
	PreviewSplitterLocation = 0.333f;
	ThumbnailSize = 256.f;
	PreviewMesh = EDMMaterialPreviewMesh::Plane;
	bShowPreviewBackground = true;
	bPreviewImagesUseTextureUVs = true;
	bUVVisualizerVisible = true;
	StagePreviewSize = 40.f;
	PropertyPreviewSize = 64.f;

	FPropertyChangedEvent PropertyChangeEvent(UDynamicMaterialEditorSettings::StaticClass()->FindPropertyByName(
		GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, Layout)
	));
	OnSettingsChanged.Broadcast(PropertyChangeEvent);
}

TArray<FDMMaterialEffectList> UDynamicMaterialEditorSettings::GetEffectList() const
{
	TArray<FDMMaterialEffectList> Effects;

	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();

	if (!AssetRegistry)
	{
		return Effects;
	}

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);

	if (!Plugin.IsValid())
	{
		return Effects;
	}

	const FString PluginEffectPath = Plugin->GetMountedAssetPath() / "MaterialFunctions" / "Effects";

	TArray<FName> AssetPaths = {*PluginEffectPath};
	AssetPaths.Append(UDynamicMaterialEditorSettings::Get()->CustomEffectsFolders);

	TArray<FAssetData> Assets;
	AssetRegistry->GetAssetsByPaths(AssetPaths, Assets, /* bRecursive */ true, /* Only Assets on Disk */ true);

	TArray<FString> AssetPathStrings;
	AssetPathStrings.Reserve(AssetPaths.Num());
	Algo::Transform(AssetPaths, AssetPathStrings, [](const FName& InElement) { return InElement.ToString(); });

	for (FString& AssetPathString : AssetPathStrings)
	{
		if (AssetPathString.EndsWith(TEXT("/")) || AssetPathString.EndsWith(TEXT("\\")))
		{
			AssetPathString = AssetPathString.LeftChop(1);
		}
	}

	auto FindBasePath = [&AssetPathStrings](const FString& InPath)
		{
			for (const FString& AssetPathString : AssetPathStrings)
			{
				if (InPath.StartsWith(AssetPathString))
				{
					return AssetPathString;
				}
			}

			return FString("");
		};

	for (const FAssetData& Asset : Assets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass || !AssetClass->IsChildOf(UMaterialFunctionInterface::StaticClass()))
		{
			continue;
		}

		const FString AssetPath = Asset.GetObjectPathString();
		const FString AssetBasePath = FindBasePath(AssetPath);

		FString Path = AssetPath;
		FString Category = "";

		// Reduce from /BasePath/Category/Random/OtherPaths/Asset.Asset to /BasePath and Category
		while (true)
		{
			const FString ParentPath = FPaths::GetPath(Path);

			if (ParentPath == AssetBasePath)
			{
				Category = Path.Mid(AssetBasePath.Len() + 1);
				break;
			}
			else if (ParentPath.IsEmpty())
			{
				break;
			}

			Path = ParentPath;
		}

		if (Category.IsEmpty())
		{
			continue;
		}

		FDMMaterialEffectList* EffectList = Effects.FindByPredicate(
			[Category](const FDMMaterialEffectList& InElement)
			{
				return InElement.Name == Category;
			});

		if (!EffectList)
		{
			Effects.Add({Category, {}});
			EffectList = &Effects.Last();
		}

		TSoftObjectPtr<UMaterialFunctionInterface> SoftPtr;
		SoftPtr = Asset.GetSoftObjectPath();

		EffectList->Effects.Add(SoftPtr);
	}

	return Effects;
}

const FDMDefaultMaterialPropertySlotValue& UDynamicMaterialEditorSettings::GetDefaultSlotValue(EDMMaterialPropertyType InProperty) const
{
	if (const FDMDefaultMaterialPropertySlotValue* OverridePtr = DefaultSlotValueOverrides.Find(InProperty))
	{
		switch (OverridePtr->DefaultType)
		{
			case EDMDefaultMaterialPropertySlotValueType::Texture:
				if (!OverridePtr->Texture.IsNull() && OverridePtr->Texture.LoadSynchronous())
				{
					return *OverridePtr;
				}
				break;

			case EDMDefaultMaterialPropertySlotValueType::Color:
				return *OverridePtr;
		}
	}
	
	if (const FDMDefaultMaterialPropertySlotValue* DefaultPtr = UE::DynamicMaterialEditor::Private::DefaultSlotValues.Find(InProperty))
	{
		switch (DefaultPtr->DefaultType)
		{
			case EDMDefaultMaterialPropertySlotValueType::Texture:
				if (!DefaultPtr->Texture.IsNull() && DefaultPtr->Texture.LoadSynchronous())
				{
					return *DefaultPtr;
				}
				break;

			case EDMDefaultMaterialPropertySlotValueType::Color:
				return *DefaultPtr;
		}
	}

	static const FDMDefaultMaterialPropertySlotValue Default = {GetDefault<UDMMaterialValueFloat3RGB>()->GetValue()};

	return Default;
}

const FDMMaterialChannelListPreset* UDynamicMaterialEditorSettings::GetPresetByName(FName InName) const
{
	for (const FDMMaterialChannelListPreset& Preset : MaterialChannelPresets)
	{
		if (InName == Preset.Name)
		{
			return &Preset;
		}
	}

	return nullptr;
}

FOnFinishedChangingProperties::RegistrationType& UDynamicMaterialEditorSettings::GetOnSettingsChanged()
{
	return OnSettingsChanged;
}

void UDynamicMaterialEditorSettings::EnsureUniqueChannelPresetNames()
{
	// This is splitting the *text* portion of the FName, not the number part.
	// So an FName might me Foo5_1 and this would return "Foo" and 5.
	auto SplitName = [](FString InName, FString& OutName, int32& OutNumber)
		{
			const int32 NameLen = InName.Len();

			for (int32 CharIndex = NameLen - 1; CharIndex >= 0; --CharIndex)
			{
				if (InName[CharIndex] >= '0' && InName[CharIndex] <= '9')
				{
					continue;
				}

				OutName = InName.Left(CharIndex + 1);

				if (CharIndex < (NameLen - 1))
				{
					OutNumber = FCString::Atoi(*InName.Mid(CharIndex + 1));
				}
				else
				{
					OutNumber = 0;
				}

				return;
			}

			OutName = "";

			// Every char was a digit.
			if (NameLen > 0)
			{
				OutNumber = FCString::Atoi(*InName);
			}
			else
			{
				OutNumber = 0;
			}
		};

	const int32 Count = MaterialChannelPresets.Num();

	for (int32 IndexBase = 0; IndexBase < Count; ++IndexBase)
	{
		// Name hasn't changed, don't try to fix it
		if (PreEditPresetNames.IsValidIndex(IndexBase) && PreEditPresetNames[IndexBase] == MaterialChannelPresets[IndexBase].Name)
		{
			continue;
		}

		for (int32 IndexCheck = 0; IndexCheck < Count; ++IndexCheck)
		{
			if (IndexCheck == IndexBase)
			{
				continue;
			}

			const bool bEqual = MaterialChannelPresets[IndexBase].Name.IsEqual(
				MaterialChannelPresets[IndexCheck].Name,
				ENameCase::IgnoreCase,
				/* Check number */ false
			);

			if (!bEqual)
			{
				continue;
			}

			UE_LOGF(LogDynamicMaterialEditor, Warning, "Duplicate channel list preset name detected.");

			FString BaseName;
			int32 NumberSuffix;
			SplitName(MaterialChannelPresets[IndexBase].Name.GetPlainNameString(), BaseName, NumberSuffix);

			if (NumberSuffix < 2)
			{
				NumberSuffix = 2;
			}

			for (int32 IndexSameNameCheck = 0; IndexSameNameCheck < Count; ++IndexSameNameCheck)
			{
				if (IndexSameNameCheck == IndexBase)
				{
					continue;
				}

				FString BaseNameCheck;
				int32 NumberSuffixCheck;
				SplitName(MaterialChannelPresets[IndexSameNameCheck].Name.GetPlainNameString(), BaseNameCheck, NumberSuffixCheck);

				if (BaseNameCheck.Equals(BaseName, ESearchCase::IgnoreCase))
				{
					NumberSuffix = FMath::Max(NumberSuffix, NumberSuffixCheck + 1);
				}
			}

			MaterialChannelPresets[IndexBase].Name = *(BaseName + FString::FromInt(NumberSuffix));
		}
	}

	PreEditPresetNames.Empty();
}

void UDynamicMaterialEditorSettings::ConvertDeprecatedVarsToFlags()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	for (FDMMaterialChannelListPreset& Preset : MaterialChannelPresets)
	{
		if (Preset.bDefaultTwoSided_DEPRECATED)
		{
			Preset.GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);
		}

		if (Preset.bDefaultAnimated_DEPRECATED)
		{
			Preset.MotionFlags |= static_cast<uint8>(EDMMaterialFlagsMotion::PixelAnimation);
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#undef LOCTEXT_NAMESPACE
