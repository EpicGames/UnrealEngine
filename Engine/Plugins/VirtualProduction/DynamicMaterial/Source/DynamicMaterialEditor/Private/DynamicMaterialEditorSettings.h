// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DMDefs.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/Texture.h"
#include "PropertyEditorDelegates.h"
#include "UObject/SoftObjectPtr.h"
#include "DynamicMaterialEditorSettings.generated.h"

class FModifierKeysState;
class UDynamicMaterialModel;
class UMaterialFunctionInterface;
enum EOrientation : int;
struct FDMMaterialChannelListPreset;
struct FPropertyChangedEvent;

USTRUCT(BlueprintType)
struct FDMMaterialEffectList
{
	GENERATED_BODY()

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Effects")
	FString Name;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Effects")
	TArray<TSoftObjectPtr<UMaterialFunctionInterface>> Effects;
};

UENUM(BlueprintType)
enum class EDMDefaultMaterialPropertySlotValueType : uint8
{
	Texture,
	Color
};

USTRUCT(BlueprintType)
struct FDMDefaultMaterialPropertySlotValue
{
	GENERATED_BODY()

	FDMDefaultMaterialPropertySlotValue();
	FDMDefaultMaterialPropertySlotValue(const TSoftObjectPtr<UTexture>& InTexture);
	FDMDefaultMaterialPropertySlotValue(const FLinearColor& InColor);
	FDMDefaultMaterialPropertySlotValue(EDMDefaultMaterialPropertySlotValueType InDefaultType, 
		const TSoftObjectPtr<UTexture>& InTexture, const FLinearColor& InColor);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Material Designer")
	EDMDefaultMaterialPropertySlotValueType DefaultType;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Material Designer")
	TSoftObjectPtr<UTexture> Texture;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Material Designer")
	FLinearColor Color;
};

USTRUCT(BlueprintType)
struct FDMMaterialChannelListPreset
{
	GENERATED_BODY()

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	FName Name;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bBaseColor = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bEmissive = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bOpacity = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bRoughness = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bSpecular = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bMetallic = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bNormal = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bPixelDepthOffset = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bWorldPositionOffset = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bAmbientOcclusion = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bAnisotropy = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bRefraction = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bTangent = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bDisplacement = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bSubsurfaceColor = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	bool bSurfaceThickness = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	TEnumAsByte<EBlendMode> DefaultBlendMode = BLEND_Opaque;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channel")
	EDMMaterialShadingModel DefaultShadingModel = EDMMaterialShadingModel::Unlit;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Usage",
		meta = (Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsUsage"))
	int32 UsageFlag = 0;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "General",
		meta = (Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsGeneral"))
	uint8 GeneralFlags = 0;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Lighting",
		meta = (Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsLighting"))
	uint8 LightingFlags = 0;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Translucency",
		meta = (Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsTranslucency"))
	uint8 TranslucencyFlags = 0;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Motion",
		meta = (Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsMotion"))
	uint8 MotionFlags = 0;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Forward Renderer",
		meta = (Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsForwardRenderer"))
	uint8 ForwardRendererFlags = 0;

	bool IsPropertyEnabled(EDMMaterialPropertyType InProperty) const;

	/*
	 * Deprecated
	 */
	UE_DEPRECATED(5.8, "Moved to MotionFlags")
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "Moved to MotionFlags"))
	bool bDefaultAnimated_DEPRECATED = false;

	UE_DEPRECATED(5.8, "Moved to GeneralFlags")
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "Moved to GeneralFlags"))
	bool bDefaultTwoSided_DEPRECATED = true;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDMMaterialChannelListPreset() = default;
	FDMMaterialChannelListPreset(const FDMMaterialChannelListPreset&) = default;
	FDMMaterialChannelListPreset(FDMMaterialChannelListPreset&&) = default;
	FDMMaterialChannelListPreset& operator=(const FDMMaterialChannelListPreset&) = default;
	FDMMaterialChannelListPreset& operator=(FDMMaterialChannelListPreset&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

UENUM(BlueprintType)
enum class EDMMaterialPreviewMesh : uint8
{
	Plane,
	Cube,
	Sphere,
	Cylinder,
	Custom
};

UENUM(BlueprintType)
enum class EDMMaterialEditorLayout : uint8
{
	Top,
	TopSlim,
	Left,

	First = Top,
	Last = Left
};

UENUM(BlueprintType)
enum class EDMLiveEditMode : uint8
{
	Disabled,
	LiveEditOff,
	LiveEditOn
};

USTRUCT()
struct FDMContentBrowserThumbnailSettings
{
	GENERATED_BODY()

	UPROPERTY(Config, EditAnywhere, Category = "Preview")
	EDMMaterialPreviewMesh PreviewMesh = EDMMaterialPreviewMesh::Custom;

	UPROPERTY(Config, EditAnywhere, Category = "Preview", meta = (EditCondition = "PreviewMesh == EDMMaterialPreviewMesh::Custom", EditConditionHides))
	float CustomMeshOrbitPitch = -30.f;

	UPROPERTY(Config, EditAnywhere, Category = "Preview", meta = (EditCondition = "PreviewMesh == EDMMaterialPreviewMesh::Custom", EditConditionHides))
	float CustomMeshOrbitYaw = 152.f;

	UPROPERTY(Config, EditAnywhere, Category = "Preview", meta = (EditCondition = "PreviewMesh == EDMMaterialPreviewMesh::Custom", EditConditionHides))
	float CustomMeshZoom = -409.f;
};

/**
 * Material Designer Settings
 */
UCLASS(Config=EditorPerProjectUserSettings, meta = (DisplayName = "Material Designer"))
class UDynamicMaterialEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	friend class SDMToolBar;

public:
	UDynamicMaterialEditorSettings();
	virtual ~UDynamicMaterialEditorSettings() override = default;

	DLLEXPORT static UDynamicMaterialEditorSettings* Get();

	/** Changes the currently active material in the designer following actor/object selection. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Editor")
	bool bFollowSelection;

	/**
	 * If true, a "Create with Material Designer" button will show on properties and material lists. The edit
	 * buttons will always appear.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Editor")
	bool bAddDetailsPanelButton;

protected:
	/**
	 * When properties on the preview material are changed, if they are parameter-based then automatically
	 * also set those values on the source material (in the world or asset). This value is only used when Live Edit
	 * mode is disabled.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Editor")
	bool bAutomaticallyCopyParametersToSourceMaterial;

	/**
	 * When a structural material change is made, whether to automatically recompile the preview material to show this change.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Editor", meta = (EditCondition = "LiveEditMode == EDMLiveEditMode::Disabled"))
	bool bAutomaticallyCompilePreviewMaterial;

	/**
	 * When the preview material is recompiled, whether to automatically also apply those changes to the source material.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Editor", meta = (EditCondition = "LiveEditMode == EDMLiveEditMode::Disabled"))
	bool bAutomaticallyApplyToSourceOnPreviewCompile;

public:
	/**
	 * When enabled, reduces compile options to a single mode called Live Edit.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Editor")
	EDMLiveEditMode LiveEditMode;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool ShouldAutomaticallyCopyParametersToSourceMaterial() const
	{
		if (ShouldAutomaticallyApplyToSourceOnPreviewCompile())
		{
			return true;
		}

		return bAutomaticallyCopyParametersToSourceMaterial;
	}

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool ShouldAutomaticallyCompilePreviewMaterial() const
	{
		switch (LiveEditMode)
		{
			case EDMLiveEditMode::LiveEditOff:
				return false;

			case EDMLiveEditMode::LiveEditOn:
				return true;

			default:
				return bAutomaticallyCompilePreviewMaterial;
		}
	}

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool ShouldAutomaticallyApplyToSourceOnPreviewCompile() const
	{
		switch (LiveEditMode)
		{
			case EDMLiveEditMode::LiveEditOff:
				return false;

			case EDMLiveEditMode::LiveEditOn:
				return true;

			default:
				return bAutomaticallyApplyToSourceOnPreviewCompile;
		}
	}

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout")
	EDMMaterialEditorLayout Layout;

	/** Adjusts the vertical size of the material layer view. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout", meta = (
		ClampMin = "0.05", UIMin = "0.05", ClampMax = "0.95", UIMax = "0.95"))
	float SplitterLocation;

	/** Adjusts the vertical size of the material layer view. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout", meta = (
		ClampMin = "0.05", UIMin = "0.05", ClampMax = "0.95", UIMax = "0.95"))
	float PreviewSplitterLocation;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout")
	bool bUVVisualizerVisible;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout")
	bool bUseFullChannelNamesInTopSlimLayout;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout")
	float StagePreviewSize;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Layout", meta = (DisplayName = "Channel Preview Size"))
	float PropertyPreviewSize;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Thumnnails")
	double ThumbnailSize;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Preview")
	EDMMaterialPreviewMesh PreviewMesh;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Preview")
	TSoftObjectPtr<UStaticMesh> CustomPreviewMesh;

	UPROPERTY(Config, EditAnywhere, Category = "Preview")
	FDMContentBrowserThumbnailSettings ContentBrowserThumbnail;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Preview")
	bool bShowPreviewBackground;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Preview")
	bool bPreviewImagesUseTextureUVs;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Preview")
	TSoftObjectPtr<UTexture> DefaultMask;

	/**
	 * Overrides the default values given to slots created in the given material property.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channels")
	TMap<EDMMaterialPropertyType, FDMDefaultMaterialPropertySlotValue> DefaultSlotValueOverrides;

	/*
	 * Add paths to search for custom effects.
	 *
	 * Format examples:
	 * - /Game/Some/Path
	 * - /Plugin/Some/Path
	 *
	 * The assets must be in a sub-folder of the base path. The sub-folder
	 * will be used as the category name.
	 *
	 * Asset Examples:
	 * - /Game/Some/Path/UV/Asset.Asset -> Category: UV
	 * - /Plugin/Some/Path/Color/OtherAsset.OtherAsset -> Category: Color
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Effects")
	TArray<FName> CustomEffectsFolders;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Channels", meta = (TitleProperty = Name))
	TArray<FDMMaterialChannelListPreset> MaterialChannelPresets;

	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = "Material Functions")
	bool bUseLinearColorForVectors;

	UPROPERTY(Config)
	bool bValidatedPresets = false;

	UPROPERTY(Config)
	bool bConvertedVarsToFlags = false;

	/** This variable is accessed in multiple places, so this is a quick accessor. */
	static bool IsUseLinearColorForVectorsEnabled();

	void OpenEditorSettingsWindow() const;

	void ResetAllLayoutSettings();

	TArray<FDMMaterialEffectList> GetEffectList() const;

	const FDMDefaultMaterialPropertySlotValue& GetDefaultSlotValue(EDMMaterialPropertyType InProperty) const;

	const FDMMaterialChannelListPreset* GetPresetByName(FName InName) const;

	FOnFinishedChangingProperties::RegistrationType& GetOnSettingsChanged();

	//~ Begin UObject
	virtual void PostInitProperties() override;
	virtual void PreEditChange(FEditPropertyChain& InPropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

private:
	TArray<FName> PreEditPresetNames;
	FOnFinishedChangingProperties OnSettingsChanged;

	void EnsureUniqueChannelPresetNames();

	void ConvertDeprecatedVarsToFlags();
};
