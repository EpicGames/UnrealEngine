// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialEditorStyle.h"

#include "DetailLayoutBuilder.h"
#include "DMDefs.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"

namespace UE::DynamicMaterialEditor::Private
{
	FLinearColor ReplaceColorAlpha(const FLinearColor& InColor, const float InNewAlpha)
	{
		FLinearColor OutColor(InColor);
		OutColor.A = InNewAlpha;
		return OutColor;
	}
}

FDynamicMaterialEditorStyle::FDynamicMaterialEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	SetupGeneralStyles();
	SetupStageStyles();
	SetupLayerViewStyles();
	SetupLayerViewItemHandleStyles();
	SetupEffectsViewStyles();
	SetupTextStyles();
	SetupComponentIcons();

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDynamicMaterialEditorStyle::~FDynamicMaterialEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

void FDynamicMaterialEditorStyle::SetupGeneralStyles()
{
	using namespace UE::DynamicMaterialEditor::Private;

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/DynamicMaterial"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate")); // This is the engine's content root.

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Color Styles
	const FLinearColor EngineSelectPressColor = FStyleColors::PrimaryPress.GetSpecifiedColor();

	const FLinearColor SelectColor = ReplaceColorAlpha(FStyleColors::Select.GetSpecifiedColor(), 0.9f);
	const FLinearColor SelectHoverColor = FStyleColors::Select.GetSpecifiedColor();
	const FLinearColor SelectPressColor = EngineSelectPressColor;

	Set("Color.Select", SelectColor);
	Set("Color.Select.Hover", SelectHoverColor);
	Set("Color.Select.Press", SelectPressColor);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Brush Styles
	Set("Icons.Menu.Dropdown", new IMAGE_BRUSH_SVG("Icons/MenuDropdown", CoreStyleConstants::Icon16x16));

	Set("Icons.Material.DefaultLit", new IMAGE_BRUSH("Icons/EditorIcons/MaterialTypeDefaultLit", CoreStyleConstants::Icon32x32));
	Set("Icons.Material.Unlit", new IMAGE_BRUSH("Icons/EditorIcons/MaterialTypeUnlit", CoreStyleConstants::Icon32x32));

	Set("Icons.Lock", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Lock", CoreStyleConstants::Icon16x16));
	Set("Icons.Unlock", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Unlock", CoreStyleConstants::Icon16x16));

	Set("Icons.Remove", new IMAGE_BRUSH("Icons/EditorIcons/Remove_16px", CoreStyleConstants::Icon16x16));

	Set("Icons.Normalize", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Normalize", CoreStyleConstants::Icon16x16));

	Set("Icons.Stage.EnabledButton", new IMAGE_BRUSH("Icons/EditorIcons/WhiteBall", CoreStyleConstants::Icon8x8));
	Set("Icons.Stage.BaseToggle", new IMAGE_BRUSH("Icons/EditorIcons/BaseToggle_16x", CoreStyleConstants::Icon16x16));
	Set("Icons.Stage.MaskToggle", new IMAGE_BRUSH("Icons/EditorIcons/MaskToggle_16x", CoreStyleConstants::Icon16x16));
	Set("Icons.Stage.Enabled", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Enable", CoreStyleConstants::Icon24x24));
	Set("Icons.Stage.Disabled", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Disable", CoreStyleConstants::Icon24x24));

	Set("Icons.Stage.ChainLinked", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainLinked", CoreStyleConstants::Icon16x16));
	Set("Icons.Stage.ChainUnlinked", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainUnlinked", CoreStyleConstants::Icon16x16));
	Set("Icons.Stage.ChainLinked.Horizontal", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainLinked_Horizontal", CoreStyleConstants::Icon24x24));
	Set("Icons.Stage.ChainUnlinked.Horizontal", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainUnlinked_Horizontal", CoreStyleConstants::Icon24x24));
	Set("Icons.Stage.ChainLinked.Vertical", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainLinked_Vertical", CoreStyleConstants::Icon24x24));
	Set("Icons.Stage.ChainUnlinked.Vertical", new IMAGE_BRUSH_SVG("Icons/EditorIcons/ChainUnlinked_Vertical", CoreStyleConstants::Icon24x24));

	Set("ImageBorder", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, 10.0f,
		FStyleColors::Panel.GetSpecifiedColor(), 2.0f));

	Set("Border.SinglePixel", new BORDER_BRUSH(TEXT("Images/Borders/Border_SinglePixel"), FMargin(1.0f / 4.0f)));
	Set("Border.LeftTopRight", new BORDER_BRUSH(TEXT("Images/Borders/Border_LeftTopRight"), FMargin(1.0f / 4.0f, 1.0f / 2.0f)));
	Set("Border.LeftBottomRight", new BORDER_BRUSH(TEXT("Images/Borders/Border_LeftBottomRight"), FMargin(1.0f / 4.0f, 1.0f / 2.0f)));
	Set("Border.TopLeftBottom", new BORDER_BRUSH(TEXT("Images/Borders/Border_TopLeftBottom"), FMargin(1.0f / 2.0f, 1.0f / 4.0f)));
	Set("Border.TopRightBottom", new BORDER_BRUSH(TEXT("Images/Borders/Border_TopRightBottom"), FMargin(1.0f / 2.0f, 1.0f / 4.0f)));
	Set("Border.Top", new BORDER_BRUSH(TEXT("Images/Borders/Border_Top"), FMargin(0.0f, 1.0f / 2.0f)));
	Set("Border.Bottom", new BORDER_BRUSH(TEXT("Images/Borders/Border_Bottom"), FMargin(0.0f, 1.0f / 2.0f)));
	Set("Border.Left", new BORDER_BRUSH(TEXT("Images/Borders/Border_Left"), FMargin(1.0f / 2.0f, 0.0f)));
	Set("Border.Right", new BORDER_BRUSH(TEXT("Images/Borders/Border_Right"), FMargin(1.0f / 2.0f, 0.0f)));

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Button Styles
	Set("HoverHintOnly", FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("HoverHintOnly.Bordered", FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 4.0f, FLinearColor(1, 1, 1, 0.25f), 1.0f))
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f, FLinearColor(1, 1, 1, 0.4f), 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f, FLinearColor(1, 1, 1, 0.5f), 1.0f))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("HoverHintOnly.Bordered.Dark", FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 4.0f, FStyleColors::InputOutline.GetSpecifiedColor(), 1.0f))
		.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f, FLinearColor(1, 1, 1, 0.4f), 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f, FLinearColor(1, 1, 1, 0.5f), 1.0f))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("DulledSectionButton", FCheckBoxStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FCheckBoxStyle>("FilterBar.BasicFilterButton"))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor() * FLinearColor(0.5f, 0.5f, 0.5f, 1.f), 4.0f, FStyleColors::Input, 1.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 4.0f, FStyleColors::Input, 1.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::AccentGreen, 4.0f, FStyleColors::Input, 1.0f)));
}

void FDynamicMaterialEditorStyle::SetupStageStyles()
{
	using namespace UE::DynamicMaterialEditor::Private;

	constexpr float StageCornerRadius = 6.0f;
	constexpr float StageBorderWidth = 2.0f;
	constexpr float NonHoverAlpha = 1.f;
	constexpr float HoverAlpha = 0.75f;

	const FLinearColor EnabledColor = FStyleColors::Foreground.GetSpecifiedColor();
	const FLinearColor EnabledSelectedColor = FStyleColors::Primary.GetSpecifiedColor();
	const FLinearColor DisabledColor = FStyleColors::AccentRed.GetSpecifiedColor() * FLinearColor(0.5f, 0.5f, 0.5f, 1.f);
	const FLinearColor DisabledSelectedColor = FStyleColors::AccentRed.GetSpecifiedColor();

	Set("Stage.Inactive", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		FStyleColors::Panel.GetSpecifiedColor(), StageBorderWidth));

	Set("Stage.Enabled", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(EnabledColor, NonHoverAlpha), StageBorderWidth));
	Set("Stage.Enabled.Hover", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(EnabledColor, HoverAlpha), StageBorderWidth));
	Set("Stage.Enabled.Select", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(EnabledSelectedColor, NonHoverAlpha), StageBorderWidth));
	Set("Stage.Enabled.Select.Hover", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(EnabledSelectedColor, HoverAlpha), StageBorderWidth));

	Set("Stage.Disabled", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(DisabledColor, NonHoverAlpha), StageBorderWidth));
	Set("Stage.Disabled.Hover", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(DisabledColor, HoverAlpha), StageBorderWidth));
	Set("Stage.Disabled.Select", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(DisabledSelectedColor, NonHoverAlpha), StageBorderWidth));
	Set("Stage.Disabled.Select.Hover", new FSlateRoundedBoxBrush(
		FLinearColor::Transparent, StageCornerRadius,
		ReplaceColorAlpha(DisabledSelectedColor, HoverAlpha), StageBorderWidth));
}

void FDynamicMaterialEditorStyle::SetupLayerViewStyles()
{
	using namespace UE::DynamicMaterialEditor::Private;

	Set("LayerView.Background", new FSlateRoundedBoxBrush(
		FStyleColors::Panel.GetSpecifiedColor()/*FLinearColor(0, 0, 0, 0.25f)*/, 6.0f,
		FStyleColors::Header.GetSpecifiedColor()/*FLinearColor(1, 1, 1, 0.2f)*/, 1.0f));

	/**
	 * SListView and and FTableViewStyle have no support for adding padding between the background brush
	 * and the SListView widget, so we are not using this style for the SDMMaterialSlotLayerView. Instead, we add a
	 * SBorder around the SDMBLayerView and style that.
	 */
	Set("LayerView", FTableViewStyle()
		.SetBackgroundBrush(*GetBrush("LayerView.Background"))
	);

	constexpr float LayerViewItemCornerRadius = 5.0f;
	constexpr float LayerViewItemBorderWidth = 1.0f;

	const FLinearColor LayerViewItemFillColor = FLinearColor::Transparent;
	constexpr FLinearColor LayerViewItemBorderColor = FLinearColor(1, 1, 1, 0.15f);

	const FLinearColor LayerItemHoverFillColor = FStyleColors::Recessed.GetSpecifiedColor();
	constexpr FLinearColor LayerItemHoverBorderColor = FLinearColor(1, 1, 1, 0.2f);

	const FLinearColor LayerItemSelectFillColor = FStyleColors::Header.GetSpecifiedColor();
	const FLinearColor LayerItemSelectBorderColor = ReplaceColorAlpha(FStyleColors::Select.GetSpecifiedColor(), 0.9f);

	Set("LayerView.Row.Item", new FSlateRoundedBoxBrush(
		LayerViewItemFillColor, LayerViewItemCornerRadius,
		LayerViewItemBorderColor, LayerViewItemBorderWidth));

	Set("LayerView.Row.Hovered", new FSlateRoundedBoxBrush(
		LayerItemHoverFillColor, LayerViewItemCornerRadius,
		LayerItemHoverBorderColor, LayerViewItemBorderWidth));

	Set("LayerView.Row.Selected", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));

	Set("LayerView.Row.ActiveBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Set("LayerView.Row.ActiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Set("LayerView.Row.InactiveBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Set("LayerView.Row.InactiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));

	Set("LayerView.Row", FTableRowStyle()
		.SetTextColor(FStyleColors::Foreground)
		.SetSelectedTextColor(FStyleColors::ForegroundHover)
		.SetEvenRowBackgroundBrush(*GetBrush(TEXT("LayerView.Row.Item")))
		.SetEvenRowBackgroundHoveredBrush(*GetBrush(TEXT("LayerView.Row.Hovered")))
		.SetOddRowBackgroundBrush(*GetBrush(TEXT("LayerView.Row.Item")))
		.SetOddRowBackgroundHoveredBrush(*GetBrush(TEXT("LayerView.Row.Hovered")))
		.SetSelectorFocusedBrush(*GetBrush(TEXT("LayerView.Row.Selected")))
		.SetActiveBrush(*GetBrush(TEXT("LayerView.Row.ActiveBrush")))
		.SetActiveHoveredBrush(*GetBrush(TEXT("LayerView.Row.ActiveHoveredBrush")))
		.SetInactiveBrush(*GetBrush(TEXT("LayerView.Row.InactiveBrush")))
		.SetInactiveHoveredBrush(*GetBrush(TEXT("LayerView.Row.InactiveHoveredBrush")))
		.SetSelectorFocusedBrush(BORDER_BRUSH("Images/DropIndicators/DropZoneIndicator_Onto", FMargin(4.f / 16.f), GetColor(TEXT("Color.Select.Hover"))))
		.SetDropIndicator_Onto(BOX_BRUSH("Images/DropIndicators/LayerView_DropIndicator_Onto", FMargin(4.0f / 16.0f), GetColor(TEXT("Color.Select.Hover"))))
		.SetDropIndicator_Above(BOX_BRUSH("Images/DropIndicators/LayerView_DropIndicator_Above", FMargin(4.0f / 16.0f, 4.0f / 16.0f, 0.f, 0.f), GetColor(TEXT("Color.Select.Hover"))))
		.SetDropIndicator_Below(BOX_BRUSH("Images/DropIndicators/LayerView_DropIndicator_Below", FMargin(4.0f / 16.0f, 0.f, 0.f, 4.0f / 16.0f), GetColor(TEXT("Color.Select.Hover"))))
	);
}

void FDynamicMaterialEditorStyle::SetupLayerViewItemHandleStyles()
{
	constexpr FLinearColor RowHandleFillColor = FLinearColor(1, 1, 1, 0.3f);
	const FLinearColor RowHandleHoverFillColor = FLinearColor(1, 1, 1, 0.4f);
	const FLinearColor RowHandleBorderColor = FLinearColor::Transparent;
	constexpr float RowHandleCornerRadius = 6.0f;
	constexpr float RowHandleBorderWidth = 1.0f;

	Set("LayerView.Row.Handle.Left", new FSlateRoundedBoxBrush(
		RowHandleFillColor, FVector4(RowHandleCornerRadius, 0.0f, 0.0f, RowHandleCornerRadius),
		RowHandleBorderColor, RowHandleBorderWidth));
	Set("LayerView.Row.Handle.Top", new FSlateRoundedBoxBrush(
		RowHandleFillColor, FVector4(RowHandleCornerRadius, RowHandleCornerRadius, 0.0f, 0.0f),
		RowHandleBorderColor, RowHandleBorderWidth));
	Set("LayerView.Row.Handle.Right", new FSlateRoundedBoxBrush(
		RowHandleFillColor, FVector4(0.0f, RowHandleCornerRadius, RowHandleCornerRadius, 0.0f),
		RowHandleBorderColor, RowHandleBorderWidth));
	Set("LayerView.Row.Handle.Bottom", new FSlateRoundedBoxBrush(
		RowHandleFillColor, FVector4(0.0f, 0.0f, RowHandleCornerRadius, RowHandleCornerRadius),
		RowHandleBorderColor, RowHandleBorderWidth));

	Set("LayerView.Row.Handle.Left.Hover", new FSlateRoundedBoxBrush(
		RowHandleHoverFillColor, FVector4(RowHandleCornerRadius, 0.0f, 0.0f, RowHandleCornerRadius),
		RowHandleBorderColor, RowHandleBorderWidth));
	Set("LayerView.Row.Handle.Top.Hover", new FSlateRoundedBoxBrush(
		RowHandleHoverFillColor, FVector4(RowHandleCornerRadius, RowHandleCornerRadius, 0.0f, 0.0f),
		RowHandleBorderColor, RowHandleBorderWidth));
	Set("LayerView.Row.Handle.Right.Hover", new FSlateRoundedBoxBrush(
		RowHandleHoverFillColor, FVector4(0.0f, RowHandleCornerRadius, RowHandleCornerRadius, 0.0f),
		RowHandleBorderColor, RowHandleBorderWidth));
	Set("LayerView.Row.Handle.Bottom.Hover", new FSlateRoundedBoxBrush(
		RowHandleHoverFillColor, FVector4(0.0f, 0.0f, RowHandleCornerRadius, RowHandleCornerRadius),
		RowHandleBorderColor, RowHandleBorderWidth));

	Set("LayerView.Row.Handle.Left.Select", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select")), 0.0f,
		RowHandleBorderColor, 0.0f));
	Set("LayerView.Row.Handle.Top.Select", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select")), 0.0f,
		RowHandleBorderColor, 0.0f));
	Set("LayerView.Row.Handle.Right.Select", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select")), 0.0f,
		RowHandleBorderColor, 0.0f));
	Set("LayerView.Row.Handle.Bottom.Select", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select")), 0.0f,
		RowHandleBorderColor, 0.0f));

	Set("LayerView.Row.Handle.Left.Select.Hover", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select.Hover")), 0.0f,
		RowHandleBorderColor, 0.0f));
	Set("LayerView.Row.Handle.Top.Select.Hover", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select.Hover")), 0.0f,
		RowHandleBorderColor, 0.0f));
	Set("LayerView.Row.Handle.Right.Select.Hover", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select.Hover")), 0.0f,
		RowHandleBorderColor, 0.0f));
	Set("LayerView.Row.Handle.Bottom.Select.Hover", new FSlateRoundedBoxBrush(
		GetColor(TEXT("Color.Select.Hover")), 0.0f,
		RowHandleBorderColor, 0.0f));
}

void FDynamicMaterialEditorStyle::SetupEffectsViewStyles()
{
	using namespace UE::DynamicMaterialEditor::Private;

	const float LayerViewItemCornerRadius = 0.0f;
	const float LayerViewItemBorderWidth = 1.0f;

	const FLinearColor LayerViewItemFillColor = FLinearColor::Transparent;
	constexpr FLinearColor LayerViewItemBorderColor = FLinearColor(1, 1, 1, 0.15f);

	const FLinearColor LayerItemHoverFillColor = FStyleColors::Recessed.GetSpecifiedColor();
	constexpr FLinearColor LayerItemHoverBorderColor = FLinearColor(1, 1, 1, 0.2f);

	const FLinearColor LayerItemSelectFillColor = FStyleColors::Header.GetSpecifiedColor();
	const FLinearColor LayerItemSelectBorderColor = ReplaceColorAlpha(FStyleColors::Select.GetSpecifiedColor(), 0.9f);

	Set("EffectsView.Row.Item", new FSlateRoundedBoxBrush(
		LayerViewItemFillColor, LayerViewItemCornerRadius,
		LayerViewItemBorderColor, LayerViewItemBorderWidth));

	Set("EffectsView.Row.Hovered", new FSlateRoundedBoxBrush(
		LayerItemHoverFillColor, LayerViewItemCornerRadius,
		LayerItemHoverBorderColor, LayerViewItemBorderWidth));

	Set("EffectsView.Row.Selected", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));

	Set("EffectsView.Row.ActiveBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Set("EffectsView.Row.ActiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Set("EffectsView.Row.InactiveBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));
	Set("EffectsView.Row.InactiveHoveredBrush", new FSlateRoundedBoxBrush(
		LayerItemSelectFillColor, LayerViewItemCornerRadius,
		LayerItemSelectBorderColor, LayerViewItemBorderWidth));

	Set("EffectsView.Row", FTableRowStyle()
		.SetTextColor(FStyleColors::Foreground)
		.SetSelectedTextColor(FStyleColors::ForegroundHover)
		.SetEvenRowBackgroundBrush(*GetBrush(TEXT("EffectsView.Row.Item")))
		.SetEvenRowBackgroundHoveredBrush(*GetBrush(TEXT("EffectsView.Row.Hovered")))
		.SetOddRowBackgroundBrush(*GetBrush(TEXT("EffectsView.Row.Item")))
		.SetOddRowBackgroundHoveredBrush(*GetBrush(TEXT("EffectsView.Row.Hovered")))
		.SetSelectorFocusedBrush(*GetBrush(TEXT("EffectsView.Row.Selected")))
		.SetActiveBrush(*GetBrush(TEXT("EffectsView.Row.ActiveBrush")))
		.SetActiveHoveredBrush(*GetBrush(TEXT("EffectsView.Row.ActiveHoveredBrush")))
		.SetInactiveBrush(*GetBrush(TEXT("EffectsView.Row.InactiveBrush")))
		.SetInactiveHoveredBrush(*GetBrush(TEXT("EffectsView.Row.InactiveHoveredBrush")))
		.SetSelectorFocusedBrush(BORDER_BRUSH("Images/DropIndicators/DropZoneIndicator_Onto", FMargin(4.f / 16.f), GetColor(TEXT("Color.Select.Hover"))))
		.SetDropIndicator_Onto(BOX_BRUSH("Images/DropIndicators/LayerView_DropIndicator_Onto", FMargin(4.0f / 16.0f), GetColor(TEXT("Color.Select.Hover"))))
		.SetDropIndicator_Above(BOX_BRUSH("Images/DropIndicators/LayerView_DropIndicator_Above", FMargin(4.0f / 16.0f, 4.0f / 16.0f, 0.f, 0.f), GetColor(TEXT("Color.Select.Hover"))))
		.SetDropIndicator_Below(BOX_BRUSH("Images/DropIndicators/LayerView_DropIndicator_Below", FMargin(4.0f / 16.0f, 0.f, 0.f, 4.0f / 16.0f), GetColor(TEXT("Color.Select.Hover"))))
	);

	Set("EffectsView.Row.Fx.Closed", new IMAGE_BRUSH_SVG("Icons/Fx_Closed", CoreStyleConstants::Icon24x24));
	Set("EffectsView.Row.Fx.Opened", new IMAGE_BRUSH_SVG("Icons/Fx_Opened", CoreStyleConstants::Icon24x24));
	Set("EffectsView.Row.Fx", new IMAGE_BRUSH_SVG("Icons/Fx", CoreStyleConstants::Icon24x24));
}

void FDynamicMaterialEditorStyle::SetupTextStyles()
{
	const FTextBlockStyle NormalTextStyle(FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText")));

	constexpr FLinearColor LayerViewItemTextShadowColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.5f);

	FFontOutlineSettings HandleFontOutline;
	HandleFontOutline.OutlineColor = LayerViewItemTextShadowColor;
	HandleFontOutline.OutlineSize = 1;

	FTextBlockStyle SmallTextStyle(NormalTextStyle);
	SmallTextStyle.SetFont(DEFAULT_FONT("Regular", 8));
	Set("SmallFont", SmallTextStyle);

	FTextBlockStyle RegularTextStyle(NormalTextStyle);
	RegularTextStyle.SetFont(DEFAULT_FONT("Regular", 10));
	Set("RegularFont", RegularTextStyle);

	FTextBlockStyle BoldTextStyle(NormalTextStyle);
	BoldTextStyle.SetFont(DEFAULT_FONT("Bold", 10));
	Set("BoldFont", BoldTextStyle);

	Set("ActorName", RegularTextStyle);

	FTextBlockStyle ActorNameBigTextStyle(NormalTextStyle);
	ActorNameBigTextStyle.SetFont(DEFAULT_FONT("Regular", 14));
	Set("ActorNameBig", ActorNameBigTextStyle);

	FTextBlockStyle ComponentNameBigTextStyle(NormalTextStyle);
	ComponentNameBigTextStyle.SetFont(DEFAULT_FONT("Regular", 12));
	Set("ComponentNameBig", ComponentNameBigTextStyle);

	FTextBlockStyle SlotLayerInfoTextStyle(NormalTextStyle);
	SlotLayerInfoTextStyle.SetFont(DEFAULT_FONT("Italic", 8));
	Set("SlotLayerInfo", SlotLayerInfoTextStyle);

	FSlateFontInfo LayerViewItemFont = DEFAULT_FONT("Bold", 12);
	LayerViewItemFont.OutlineSettings = HandleFontOutline;
	Set("LayerView.Row.Font", LayerViewItemFont);

	Set("LayerView.Row.HandleFont", RegularTextStyle);

	FTextBlockStyle LayerViewItemTextStyle(NormalTextStyle);
	LayerViewItemTextStyle.SetShadowOffset(FVector2D(1.0f, 1.0f));
	LayerViewItemTextStyle.SetColorAndOpacity(LayerViewItemTextShadowColor);

	Set("LayerView.Row.HeaderText",
		FTextBlockStyle(LayerViewItemTextStyle)
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetFont(LayerViewItemFont));

	Set("LayerView.Row.HeaderText.Small",
		FTextBlockStyle(LayerViewItemTextStyle)
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetFont(RegularTextStyle.Font));

	FTextBlockStyle StagePropertyDetailsTextStyle(NormalTextStyle);
	StagePropertyDetailsTextStyle.SetFont(DEFAULT_FONT("Regular", 12));
	Set("Font.Stage.Details", StagePropertyDetailsTextStyle);

	Set("Font.Stage.Details.Bold", BoldTextStyle);

	FTextBlockStyle StagePropertyDetailsSmallTextStyle(NormalTextStyle);
	StagePropertyDetailsSmallTextStyle.SetFont(IDetailLayoutBuilder::GetDetailFont());
	Set("Font.Stage.Details.Small", StagePropertyDetailsSmallTextStyle);

	FTextBlockStyle StagePropertyDetailsSmallBoldTextStyle(NormalTextStyle);
	StagePropertyDetailsSmallBoldTextStyle.SetFont(IDetailLayoutBuilder::GetDetailFontBold());
	Set("Font.Stage.Details.Small.Bold", StagePropertyDetailsSmallBoldTextStyle);

	Set("Font.Stage.Details.Small.Bold", FTextBlockStyle(NormalTextStyle)
		.SetFont(IDetailLayoutBuilder::GetDetailFontBold()));

	FEditableTextBoxStyle InlineEditableTextBoxStyle;
	InlineEditableTextBoxStyle.SetBackgroundColor(FStyleColors::Transparent);
	Set("InlineEditableTextBoxStyle", InlineEditableTextBoxStyle);
}

void FDynamicMaterialEditorStyle::SetupComponentIcons()
{
	Set(TEXT("ClassIcon.DMMaterialComponent"), new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));

	Set(TEXT("ClassIcon.DMMaterialValueBool"),         new IMAGE_BRUSH("Icons/ValueTypes/Bool", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueColorAtlas"),   new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueFloat1"),       new IMAGE_BRUSH("Icons/ValueTypes/Float1", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueFloat2"),       new IMAGE_BRUSH("Icons/ValueTypes/Float2", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueFloat3RGB"),    new FSlateImageBrush(FPaths::EngineContentDir() + TEXT("Slate/Common/ColorPicker_Mode_16x.png"), CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueFloat3RPY"),    new CORE_IMAGE_BRUSH("Icons/icon_ClockwiseRotation_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueFloat3XYZ"),    new CORE_IMAGE_BRUSH("Icons/Mobility/Movable_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueFloat4"),       new FSlateImageBrush(FPaths::EngineContentDir() + TEXT("Slate/Common/ColorPicker_Mode_16x.png"), CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueRenderTarget"), new CORE_IMAGE_BRUSH("Icons/AssetIcons/TextureRenderTarget2D_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialValueTexture"),      new IMAGE_BRUSH("Icons/ValueTypes/Texture", CoreStyleConstants::Icon16x16));

	Set(TEXT("ClassIcon.DMRenderTargetRenderer"),           new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMRenderTargetWidgetRendererBase"), new CORE_IMAGE_BRUSH("Icons/AssetIcons/WidgetBlueprint_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMRenderTargetTextRenderer"),       new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMRenderTargetUMGWidgetRenderer"),  new CORE_IMAGE_BRUSH("Icons/AssetIcons/WidgetBlueprint_16x", CoreStyleConstants::Icon16x16));

	Set(TEXT("ClassIcon.DMMaterialStageGradientLinear"), new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialStageGradientRadial"), new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));

	Set(TEXT("ClassIcon.DMMaterialStageExpressionSceneTexture"),           new CORE_IMAGE_BRUSH("Icons/AssetIcons/PostProcessVolume_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialStageExpressionTextureSampleBase"),      new IMAGE_BRUSH("Icons/ValueTypes/Texture", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialStageExpressionTextureSampleEdgeColor"), new IMAGE_BRUSH("Icons/ClassIcons/TextureEdgeColor", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialStageExpressionWorldPositionNoise"),     new CORE_IMAGE_BRUSH("Icons/icon_help_16x", CoreStyleConstants::Icon16x16));

	Set(TEXT("ClassIcon.DMMaterialStageFunction"),      new CORE_IMAGE_BRUSH("Icons/AssetIcons/MaterialFunction_16x", CoreStyleConstants::Icon16x16));
	Set(TEXT("ClassIcon.DMMaterialStageInputFunction"), new CORE_IMAGE_BRUSH("Icons/AssetIcons/MaterialFunction_16x", CoreStyleConstants::Icon16x16));
}
