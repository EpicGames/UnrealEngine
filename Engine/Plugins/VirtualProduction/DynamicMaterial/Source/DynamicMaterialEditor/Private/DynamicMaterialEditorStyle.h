// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FName;
enum class EDMValueType : uint8;

class FDynamicMaterialEditorStyle final : public FSlateStyleSet
{
public:
	static FDynamicMaterialEditorStyle& Get()
	{
		static FDynamicMaterialEditorStyle Instance;
		return Instance;
	}

	FDynamicMaterialEditorStyle();
	virtual ~FDynamicMaterialEditorStyle() override;

private:
	void SetupGeneralStyles();
	void SetupStageStyles();
	void SetupLayerViewStyles();
	void SetupLayerViewItemHandleStyles();
	void SetupEffectsViewStyles();
	void SetupTextStyles();
	void SetupComponentIcons();
};
