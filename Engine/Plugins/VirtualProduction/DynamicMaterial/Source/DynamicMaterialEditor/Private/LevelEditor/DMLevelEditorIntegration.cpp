// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditor/DMLevelEditorIntegration.h"

#include "LevelEditor.h"
#include "LevelEditor/DMLevelEditorIntegrationInstance.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UI/Widgets/SDMMaterialDesigner.h"
#include "Utils/DMBuildRequestSubsystem.h"

namespace UE::DynamicMaterialEditor::Private
{
	FDelegateHandle LevelEditorCreatedHandle;
	FDelegateHandle LevelEditorMapChangeHandle;

	FLevelEditorModule& GetLevelEditorModule()
	{
		return FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	}

	FLevelEditorModule* GetLevelEditorModulePtr()
	{
		return FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
	}

	FLevelEditorModule& LoadLevelEditorModuleChecked()
	{
		return FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	}	
}

void FDMLevelEditorIntegration::Initialize()
{
	using namespace UE::DynamicMaterialEditor::Private;

	FLevelEditorModule& LevelEditorModule = LoadLevelEditorModuleChecked();

	LevelEditorCreatedHandle = LevelEditorModule.OnLevelEditorCreated().AddLambda(
		[](TSharedPtr<ILevelEditor> InLevelEditor)
		{
			if (InLevelEditor.IsValid())
			{
				FDMLevelEditorIntegrationInstance::AddIntegration(InLevelEditor.ToSharedRef());
			}
		}
	);

	LevelEditorMapChangeHandle = LevelEditorModule.OnMapChanged().AddLambda(
		[](UWorld* InWorld, EMapChangeType InMapChangeType)
		{
			switch (InMapChangeType)
			{
				case EMapChangeType::TearDownWorld:
					OnMapTearDown(InWorld);
					break;

				case EMapChangeType::LoadMap:
					OnMapLoad(InWorld);
					break;
			}
		}
	);
}

void FDMLevelEditorIntegration::Shutdown()
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (LevelEditorCreatedHandle.IsValid())
	{
		if (FLevelEditorModule* ModulePtr = GetLevelEditorModulePtr())
		{
			ModulePtr->OnLevelEditorCreated().Remove(LevelEditorCreatedHandle);
			LevelEditorCreatedHandle.Reset();

			ModulePtr->OnMapChanged().Remove(LevelEditorMapChangeHandle);
			LevelEditorMapChangeHandle.Reset();
		}
	}

	FDMLevelEditorIntegrationInstance::RemoveIntegrations();
}

TSharedPtr<SDMMaterialDesigner> FDMLevelEditorIntegration::GetMaterialDesignerForWorld(UWorld* InWorld)
{
	// If we have an invalid world, return the first level editor integration (for assets)... if possible
	if (!IsValid(InWorld))
	{
		using namespace UE::DynamicMaterialEditor::Private;

		if (TSharedPtr<ILevelEditor> FirstLevelEditor = GetLevelEditorModule().GetFirstLevelEditor())
		{
			InWorld = FirstLevelEditor->GetWorld();
		}

		if (!IsValid(InWorld))
		{
			return nullptr;
		}
	}

	if (const FDMLevelEditorIntegrationInstance* Integration = FDMLevelEditorIntegrationInstance::GetIntegrationForWorld(InWorld))
	{
		return Integration->GetMaterialDesigner();
	}

	return nullptr;
}

TSharedPtr<SDockTab> FDMLevelEditorIntegration::InvokeTabForWorld(UWorld* InWorld)
{
	// If we have an invalid world, return the first level editor integration (for assets)... if possible
	if (!IsValid(InWorld))
	{
		using namespace UE::DynamicMaterialEditor::Private;

		if (TSharedPtr<ILevelEditor> FirstLevelEditor = GetLevelEditorModule().GetFirstLevelEditor())
		{
			InWorld = FirstLevelEditor->GetWorld();
		}

		if (!IsValid(InWorld))
		{
			return nullptr;
		}
	}

	if (const FDMLevelEditorIntegrationInstance* Integration = FDMLevelEditorIntegrationInstance::GetIntegrationForWorld(InWorld))
	{
		return Integration->InvokeTab();
	}

	return nullptr;
}

void FDMLevelEditorIntegration::OnMapTearDown(UWorld* InWorld)
{
	if (UDMBuildRequestSubsystem* BuildRequestSubsystem = UDMBuildRequestSubsystem::Get())
	{
		BuildRequestSubsystem->RemoveBuildRequestForOuter(InWorld);
	}

	FDMLevelEditorIntegrationInstance* Instance = FDMLevelEditorIntegrationInstance::GetMutableIntegrationForWorld(InWorld);

	if (!Instance)
	{
		return;
	}

	Instance->SetLastAssetOpenPartialPath("");

	TSharedPtr<SDMMaterialDesigner> Designer = Instance->GetMaterialDesigner();

	if (!Designer.IsValid())
	{
		return;
	}

	UDynamicMaterialModelBase* MaterialModelBase = Designer->GetOriginalMaterialModelBase();
	Designer->Empty();

	if (MaterialModelBase)
	{
		const FString WorldPath = InWorld->GetPathName();
		const int32 WorldPathLength = WorldPath.Len();
		const FString ModelPath = MaterialModelBase->GetPathName();

		if (ModelPath.Len() > WorldPathLength && ModelPath.StartsWith(WorldPath))
		{
			switch (ModelPath[WorldPathLength])
			{
				case '.':
				case '/':
				case ':':
					Instance->SetLastAssetOpenPartialPath(ModelPath.RightChop(WorldPath.Len()));
					break;
			}
		}
	}
}

void FDMLevelEditorIntegration::OnMapLoad(UWorld* InWorld)
{
	FDMLevelEditorIntegrationInstance* Instance = FDMLevelEditorIntegrationInstance::GetMutableIntegrationForWorld(InWorld);

	if (!Instance)
	{
		return;
	}

	const FString PartialAssetPath = Instance->GetLastOpenAssetPartialPath();
	Instance->SetLastAssetOpenPartialPath("");

	if (PartialAssetPath.IsEmpty())
	{
		return;
	}

	TSharedPtr<SDMMaterialDesigner> Designer = Instance->GetMaterialDesigner();

	if (!Designer.IsValid())
	{
		return;
	}

	const FString ModelPath = InWorld->GetPathName() + PartialAssetPath;

	if (UObject* Object = FindObject<UObject>(nullptr, *ModelPath, EFindObjectFlags::None))
	{
		if (UDynamicMaterialModelBase* MaterialModel = Cast<UDynamicMaterialModelBase>(Object))
		{
			Designer->OpenMaterialModelBase(MaterialModel);

			if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
			{
				if (UDMBuildRequestSubsystem* BuildRequestSubsystem = UDMBuildRequestSubsystem::Get())
				{
					BuildRequestSubsystem->AddBuildRequest(EditorOnlyData, /* Dirty Assets */ false);
				}				
			}
		}
	}
}
