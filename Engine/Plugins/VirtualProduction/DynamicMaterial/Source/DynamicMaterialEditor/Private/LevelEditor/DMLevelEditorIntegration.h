// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class SDMMaterialDesigner;
class SDockTab;
class UWorld;

class FDMLevelEditorIntegration
{
public:
	static void Initialize();

	static void Shutdown();

	static TSharedPtr<SDMMaterialDesigner> GetMaterialDesignerForWorld(UWorld* InWorld);

	static TSharedPtr<SDockTab> InvokeTabForWorld(UWorld* InWorld);

private:
	static void OnMapTearDown(UWorld* InWorld);

	static void OnMapLoad(UWorld* InWorld);
};
