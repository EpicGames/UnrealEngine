// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditor/DMLevelEditorIntegrationInstance.h"

#include "Algo/Find.h"
#include "Components/PrimitiveComponent.h"
#include "DynamicMaterialEditorModule.h"
#include "EditorModeManager.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "ILevelEditor.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Selection.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Projection.h"
#include "UI/Widgets/SDMMaterialDesigner.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "DMLevelEditorIntegration"

TArray<FDMLevelEditorIntegrationInstance, TInlineAllocator<1>> FDMLevelEditorIntegrationInstance::Instances;

const FDMLevelEditorIntegrationInstance* FDMLevelEditorIntegrationInstance::AddIntegration(const TSharedRef<ILevelEditor>& InLevelEditor)
{
	ValidateInstances();

	for (const FDMLevelEditorIntegrationInstance& Instance : Instances)
	{
		if (Instance.LevelEditorWeak == InLevelEditor)
		{
			return &Instance;
		}
	}

	// Create a new instance directly in the Instances array.
	new(Instances) FDMLevelEditorIntegrationInstance(InLevelEditor);

	return &Instances.Last();
}

void FDMLevelEditorIntegrationInstance::RemoveIntegrations()
{
	Instances.Empty();
}

const FDMLevelEditorIntegrationInstance* FDMLevelEditorIntegrationInstance::GetIntegrationForWorld(UWorld* InWorld)
{
	return GetMutableIntegrationForWorld(InWorld);
}

FDMLevelEditorIntegrationInstance::~FDMLevelEditorIntegrationInstance()
{
	UnregisterSelectionChange();
	UnregisterWithTabManager();
}

TSharedPtr<SDMMaterialDesigner> FDMLevelEditorIntegrationInstance::GetMaterialDesigner() const
{
	return MaterialDesignerWeak.Pin();
}

TSharedPtr<SDockTab> FDMLevelEditorIntegrationInstance::InvokeTab() const
{
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(FDynamicMaterialEditorModule::TabId);

	if (!Tab.IsValid())
	{
		Tab = TabManager->TryInvokeTab(FDynamicMaterialEditorModule::TabId);
	}

	if (Tab.IsValid())
	{
		Tab->ActivateInParent(ETabActivationCause::SetDirectly);
		Tab->DrawAttention();
	}

	return Tab;
}

const FString& FDMLevelEditorIntegrationInstance::GetLastOpenAssetPartialPath() const
{
	return LastOpenAssetPartialPath;
}

void FDMLevelEditorIntegrationInstance::SetLastAssetOpenPartialPath(const FString& InPath)
{
	LastOpenAssetPartialPath = InPath;
}

void FDMLevelEditorIntegrationInstance::ValidateInstances()
{
	for (int32 Index = 0; Index < Instances.Num(); ++Index)
	{
		if (!Instances[Index].LevelEditorWeak.IsValid())
		{
			Instances.RemoveAt(Index);
			--Index;
		}
	}
}

FDMLevelEditorIntegrationInstance* FDMLevelEditorIntegrationInstance::GetMutableIntegrationForWorld(UWorld* InWorld)
{
	if (!IsValid(InWorld))
	{
		return nullptr;
	}

	ValidateInstances();

	for (FDMLevelEditorIntegrationInstance& Instance : Instances)
	{
		// Always return the first level editor integration for null words - they are assets.
		if (!InWorld)
		{
			return &Instance;
		}

		if (TSharedPtr<ILevelEditor> LevelEditor = Instance.LevelEditorWeak.Pin())
		{
			if (LevelEditor->GetWorld() == InWorld)
			{
				return &Instance;
			}
		}
	}

	return nullptr;
}

FDMLevelEditorIntegrationInstance::FDMLevelEditorIntegrationInstance(const TSharedRef<ILevelEditor>& InLevelEditor)
{
	LevelEditorWeak = InLevelEditor;

	RegisterSelectionChange();
	RegisterWithTabManager();
}

void FDMLevelEditorIntegrationInstance::RegisterSelectionChange()
{
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return;
	}

	if (USelection* Selection = LevelEditor->GetEditorModeManager().GetSelectedActors())
	{
		if (UTypedElementSelectionSet* ActorSelectionSet = Selection->GetElementSelectionSet())
		{
			ActorSelectionSet->OnChanged().AddRaw(this, &FDMLevelEditorIntegrationInstance::OnActorSelectionChanged);
			ActorSelectionSetWeak = ActorSelectionSet;
		}
	}

	if (USelection* Selection = LevelEditor->GetEditorModeManager().GetSelectedObjects())
	{
		if (UTypedElementSelectionSet* ObjectSelectionSet = Selection->GetElementSelectionSet())
		{
			ObjectSelectionSet->OnChanged().AddRaw(this, &FDMLevelEditorIntegrationInstance::OnObjectSelectionChanged);
			ObjectSelectionSetWeak = ObjectSelectionSet;
		}
	}
}

void FDMLevelEditorIntegrationInstance::UnregisterSelectionChange()
{
	UTypedElementSelectionSet* ActorSelectionSet = ActorSelectionSetWeak.Get();

	if (ActorSelectionSet)
	{
		ActorSelectionSet->OnChanged().RemoveAll(this);
	}

	UTypedElementSelectionSet* ObjectSelectionSet = ObjectSelectionSetWeak.Get();

	if (ObjectSelectionSet)
	{
		ObjectSelectionSet->OnChanged().RemoveAll(this);
	}
}

void FDMLevelEditorIntegrationInstance::RegisterWithTabManager()
{
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return;
	}

	TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return;
	}

	static const FText TabText = LOCTEXT("MaterialDesignerTabName", "Material Designer");

	TSharedRef<FWorkspaceItem> Category = WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory();

	// Group Material Designer under Motion Design > Motion Design Tools if the plugin is enabled.
	constexpr auto GroupProjection = Projection(&TSharedRef<FWorkspaceItem>::Get, &FWorkspaceItem::GetFName);
	if (const TSharedRef<FWorkspaceItem>* RootGroup = Algo::FindBy(Category->GetChildItems(), FName("MotionDesignGroup"), GroupProjection))
	{
		if (const TSharedRef<FWorkspaceItem>* ToolsGroup = Algo::FindBy((*RootGroup)->GetChildItems(), FName("MotionDesignToolsGroup"), GroupProjection))
		{
			Category = *ToolsGroup;
		}
	}

	TabManager->RegisterTabSpawner(
		FDynamicMaterialEditorModule::TabId,
		FOnSpawnTab::CreateLambda(
			[this](const FSpawnTabArgs&)
			{
				TSharedRef<SDMMaterialDesigner> NewMaterialDesigner = StaticCastSharedRef<SDMMaterialDesigner>(
					FDynamicMaterialEditorModule::CreateEditor(nullptr, nullptr)
				);
				MaterialDesignerWeak = NewMaterialDesigner;

				if (AActor* LastSelectedActor = LastSelectedActorWeak.Get())
				{
					NewMaterialDesigner->OnActorSelected(LastSelectedActor);
				}

				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Content()
					[
						NewMaterialDesigner
					];
			}
		)
	)
	.SetDisplayNameAttribute(TabText)
	.SetDisplayName(TabText)
	.SetGroup(Category)
	.SetIcon(FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()));
}

void FDMLevelEditorIntegrationInstance::UnregisterWithTabManager()
{
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return;
	}

	TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return;
	}

	TabManager->UnregisterTabSpawner(FDynamicMaterialEditorModule::TabId);
}

void FDMLevelEditorIntegrationInstance::OnActorSelectionChanged(const UTypedElementSelectionSet* InSelectionSet)
{
	AActor* NewSelectedActor = nullptr;

	for (AActor* Actor : InSelectionSet->GetSelectedObjects<AActor>())
	{
		// Only do this if we have a single selected actor.
		if (NewSelectedActor != nullptr)
		{
			return;
		}

		NewSelectedActor = Actor;
	}

	if (NewSelectedActor)
	{
		OnActorSelected(NewSelectedActor);
	}
}

void FDMLevelEditorIntegrationInstance::OnActorSelected(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return;
	}

	LastSelectedActorWeak = InActor;

	if (TSharedPtr<SDMMaterialDesigner> MaterialDesigner = GetMaterialDesigner())
	{
		MaterialDesigner->OnActorSelected(InActor);
	}
}

void FDMLevelEditorIntegrationInstance::OnObjectSelectionChanged(const UTypedElementSelectionSet* InSelectionSet)
{
	TSharedPtr<SDMMaterialDesigner> MaterialDesigner = GetMaterialDesigner();

	if (!MaterialDesigner.IsValid())
	{
		return;
	}

	UDynamicMaterialModelBase* NewSelectedMaterialModelBase = nullptr;

	for (UDynamicMaterialModelBase* MaterialModelBase : InSelectionSet->GetSelectedObjects<UDynamicMaterialModelBase>())
	{
		// Only do this if we have a single selected instance.
		if (NewSelectedMaterialModelBase != nullptr)
		{
			return;
		}

		NewSelectedMaterialModelBase = MaterialModelBase;
	}

	if (NewSelectedMaterialModelBase)
	{
		MaterialDesigner->OnMaterialModelBaseSelected(NewSelectedMaterialModelBase);
		return;
	}

	UDynamicMaterialInstance* NewSelectedMaterialInstance = nullptr;

	for (UDynamicMaterialInstance* MaterialInstance : InSelectionSet->GetSelectedObjects<UDynamicMaterialInstance>())
	{
		// Only do this if we have a single selected model.
		if (NewSelectedMaterialInstance != nullptr)
		{
			return;
		}

		NewSelectedMaterialInstance = MaterialInstance;
	}
	
	if (NewSelectedMaterialInstance)
	{
		MaterialDesigner->OnMaterialModelBaseSelected(NewSelectedMaterialInstance->GetMaterialModelBase());
	}
}

#undef LOCTEXT_NAMESPACE
