// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class FString;
class ILevelEditor;
class SDMMaterialDesigner;
class SDockTab;
class UTypedElementSelectionSet;
class UWorld;

struct FDMLevelEditorIntegrationInstance
{
public:
	static const FDMLevelEditorIntegrationInstance* AddIntegration(const TSharedRef<ILevelEditor>& InLevelEditor);

	static void RemoveIntegrations();

	static const FDMLevelEditorIntegrationInstance* GetIntegrationForWorld(UWorld* InWorld);

	static FDMLevelEditorIntegrationInstance* GetMutableIntegrationForWorld(UWorld* InWorld);

	~FDMLevelEditorIntegrationInstance();

	TSharedPtr<SDMMaterialDesigner> GetMaterialDesigner() const;

	TSharedPtr<SDockTab> InvokeTab() const;

	const FString& GetLastOpenAssetPartialPath() const;

	void SetLastAssetOpenPartialPath(const FString& InPath);

private:
	static TArray<FDMLevelEditorIntegrationInstance, TInlineAllocator<1>> Instances;

	static void ValidateInstances();

	TWeakPtr<ILevelEditor> LevelEditorWeak;
	TWeakObjectPtr<UTypedElementSelectionSet> ActorSelectionSetWeak;
	TWeakObjectPtr<UTypedElementSelectionSet> ObjectSelectionSetWeak;
	TWeakPtr<SDMMaterialDesigner> MaterialDesignerWeak;
	FString LastOpenAssetPartialPath;
	TWeakObjectPtr<AActor> LastSelectedActorWeak;

	FDMLevelEditorIntegrationInstance(const TSharedRef<ILevelEditor>& InLevelEditor);

	void RegisterSelectionChange();

	void UnregisterSelectionChange();

	void RegisterWithTabManager();

	void UnregisterWithTabManager();

	void OnActorSelectionChanged(const UTypedElementSelectionSet* InSelectionSet);

	void OnActorSelected(AActor* InActor);

	void OnObjectSelectionChanged(const UTypedElementSelectionSet* InSelectionSet);
};
