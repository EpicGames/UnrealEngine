// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DynamicMaterialInstance.h"
#include "AssetToolsModule.h"
#include "DynamicMaterialEditorSettings.h"
#include "IAssetTools.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DynamicMaterialInstance)

#define LOCTEXT_NAMESPACE "AssetDefinition_DynamicMaterialInstance"

FText UAssetDefinition_DynamicMaterialInstance::GetAssetDisplayName() const
{
	return LOCTEXT("MaterialDesigner", "Material Designer");
}

FText UAssetDefinition_DynamicMaterialInstance::GetAssetDisplayName(const FAssetData& InAssetData) const
{
	const FString ModelTypeTag = UDynamicMaterialInstance::GetMaterialTypeTag(InAssetData);

	if (ModelTypeTag == UDynamicMaterialInstance::ModelTypeTag_Material)
	{
		return LOCTEXT("MaterialDesignerMaterial", "MD Material");
	}

	if (ModelTypeTag == UDynamicMaterialInstance::ModelTypeTag_Instance)
	{
		return LOCTEXT("MaterialDesignerInstance", "MD Instance");
	}

	return GetAssetDisplayName();
}

TSoftClassPtr<> UAssetDefinition_DynamicMaterialInstance::GetAssetClass() const
{
	return UDynamicMaterialInstance::StaticClass();
}

FLinearColor UAssetDefinition_DynamicMaterialInstance::GetAssetColor() const
{
	return FLinearColor(FColor(64, 192, 64));
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DynamicMaterialInstance::GetAssetCategories() const
{
	static TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Material};
	return Categories;
}

UThumbnailInfo* UAssetDefinition_DynamicMaterialInstance::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>();

	if (!Settings)
	{
		return nullptr;
	}

	UDynamicMaterialInstance* MaterialInstance = Cast<UDynamicMaterialInstance>(InAsset.GetAsset());

	if (!MaterialInstance)
	{
		return nullptr;
	}

	return UE::Editor::FindOrCreateThumbnailInfo<USceneThumbnailInfoWithPrimitive>(MaterialInstance);
}

EAssetCommandResult UAssetDefinition_DynamicMaterialInstance::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	TArray<UObject*> MaterialModels;

	for (UObject* Object : InOpenArgs.LoadObjects<UDynamicMaterialInstance>())
	{
		UDynamicMaterialInstance* Instance = Cast<UDynamicMaterialInstance>(Object);

		if (!Instance)
		{
			continue;
		}

		UDynamicMaterialModelBase* MaterialModelBase = Instance->GetMaterialModelBase();

		if (!MaterialModelBase)
		{
			continue;
		}

		MaterialModels.Add(MaterialModelBase);
	}

	if (MaterialModels.IsEmpty())
	{
		return EAssetCommandResult::Unhandled;
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	AssetTools.OpenEditorForAssets(MaterialModels);

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
