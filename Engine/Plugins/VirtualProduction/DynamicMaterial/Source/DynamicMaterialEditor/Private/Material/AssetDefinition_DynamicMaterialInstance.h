// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_DynamicMaterialInstance.generated.h"

class UAssetDefinition_DynamicMaterialInstanceInterface;

UCLASS()
class UAssetDefinition_DynamicMaterialInstance : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDisplayName(const FAssetData& InAssetData) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAsset) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& InOpenArgs) const override;
	//~ End UAssetDefinition End
};
