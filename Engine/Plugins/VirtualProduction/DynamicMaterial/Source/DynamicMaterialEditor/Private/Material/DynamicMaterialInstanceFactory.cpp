// Copyright Epic Games, Inc. All Rights Reserved.

#include "Material/DynamicMaterialInstanceFactory.h"

#include "DMDefs.h"
#include "EngineAnalytics.h"
#include "GameFramework/Actor.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelFactory.h"
#include "Model/IDynamicMaterialModelEditorOnlyDataInterface.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialInstanceFactory)

#define LOCTEXT_NAMESPACE "MaterialDesignerInstanceFactory"

UDynamicMaterialInstanceFactory::UDynamicMaterialInstanceFactory()
{
	SupportedClass = UDynamicMaterialInstance::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;
	bText = false;
}

UObject* UDynamicMaterialInstanceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, 
	EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	check(InClass->IsChildOf(UDynamicMaterialInstance::StaticClass()));

	if (InName.IsNone())
	{
		InName = MakeUniqueObjectName(InParent, UDynamicMaterialInstance::StaticClass(), TEXT("MaterialDesigner"));
	}

	UDynamicMaterialInstance* NewInstance = NewObject<UDynamicMaterialInstance>(InParent, InClass, InName, InFlags | RF_Transactional);
	check(NewInstance);

	UDynamicMaterialModelFactory* EditorFactory = NewObject<UDynamicMaterialModelFactory>();
	check(EditorFactory);

	UDynamicMaterialModelBase* ModelBase = Cast<UDynamicMaterialModelBase>(InContext);

	if (!ModelBase)
	{
		ModelBase = Cast<UDynamicMaterialModel>(EditorFactory->FactoryCreateNew(
			UDynamicMaterialModel::StaticClass(), NewInstance, NAME_None, RF_Transactional | RF_Public, nullptr, GWarn));
		check(ModelBase);
	}

	const FDMInitializationGuard InitGuard;

	NewInstance->SetMaterialModel(ModelBase);

	ModelBase->SetDynamicMaterialInstance(NewInstance);

	if (UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(ModelBase))
	{
		if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = MaterialModel->GetEditorOnlyData())
		{
			ModelEditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Async);
		}
	}

	NewInstance->InitializeMIDPublic();

	if (InParent)
	{
		if (AActor* Actor = InParent->GetTypedOuter<AActor>())
		{
			if (Actor->bIsEditorPreviewActor)
			{
				// If it is a preview actor do not trigger analytics or open in editor.
				return NewInstance;
			}
		}
	}

	if (FEngineAnalytics::IsAvailable())
	{
		static const FString AssetType = TEXT("Asset");
		static const FString SubobjectType = TEXT("Subobject");

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Action"), TEXT("MaterialCreated")));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ActionDetails"), NewInstance->IsAsset() ? AssetType : SubobjectType));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), Attributes);
	}

	return NewInstance;
}

FText UDynamicMaterialInstanceFactory::GetDisplayName() const
{
	return LOCTEXT("MaterialDesignerInstance", "Material Designer Material");
}

FText UDynamicMaterialInstanceFactory::GetToolTip() const
{
	return LOCTEXT("MaterialDesignerInstanceTooltip", "The Material Designer Material is a combination of a Material Instance Dyanmic and a Material Designer Model.");
}

#undef LOCTEXT_NAMESPACE
