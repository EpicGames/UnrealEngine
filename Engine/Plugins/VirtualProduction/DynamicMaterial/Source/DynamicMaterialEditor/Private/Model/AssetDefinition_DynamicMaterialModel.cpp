// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/AssetDefinition_DynamicMaterialModel.h"
#include "DynamicMaterialEditorModule.h"
#include "Engine/World.h"
#include "Model/DynamicMaterialModel.h"
#include "Toolkits/IToolkitHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DynamicMaterialModel)

#define LOCTEXT_NAMESPACE "AssetDefinition_DynamicMaterialModel"

FText UAssetDefinition_DynamicMaterialModel::GetAssetDisplayName() const
{
	return LOCTEXT("DynamicMaterialModel", "Dynamic Material Model");
}

FText UAssetDefinition_DynamicMaterialModel::GetAssetDisplayName(const FAssetData& InAssetData) const
{
	return GetAssetDisplayName();
}

TSoftClassPtr<> UAssetDefinition_DynamicMaterialModel::GetAssetClass() const
{
	return UDynamicMaterialModel::StaticClass();
}

FLinearColor UAssetDefinition_DynamicMaterialModel::GetAssetColor() const
{
	return FLinearColor(FColor(96, 192, 96));
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DynamicMaterialModel::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Material};
	return Categories;
}

EAssetCommandResult UAssetDefinition_DynamicMaterialModel::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	for (UObject* Object : InOpenArgs.LoadObjects<UDynamicMaterialModel>())
	{
		UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(Object);

		if (!MaterialModel)
		{
			continue;
		}

		UWorld* World = MaterialModel->GetWorld();

		if (!World && InOpenArgs.ToolkitHost.IsValid())
		{
			World = InOpenArgs.ToolkitHost->GetWorld();
		}

		FDynamicMaterialEditorModule::Get().OpenMaterialModel(
			MaterialModel,
			World,
			/* Invoke Tab */ true
		);

		// Only process the first selected object.
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
