// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/AssetDefinition_DynamicMaterialModelDynamic.h"
#include "DynamicMaterialEditorModule.h"
#include "Engine/World.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Toolkits/IToolkitHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DynamicMaterialModelDynamic)

#define LOCTEXT_NAMESPACE "AssetDefinition_DynamicMaterialModelDynamic"

FText UAssetDefinition_DynamicMaterialModelDynamic::GetAssetDisplayName() const
{
	return LOCTEXT("DynamicMaterialModel", "Dynamic Material Model Instance");
}

FText UAssetDefinition_DynamicMaterialModelDynamic::GetAssetDisplayName(const FAssetData& InAssetData) const
{
	return GetAssetDisplayName();
}

TSoftClassPtr<> UAssetDefinition_DynamicMaterialModelDynamic::GetAssetClass() const
{
	return UDynamicMaterialModelDynamic::StaticClass();
}

FLinearColor UAssetDefinition_DynamicMaterialModelDynamic::GetAssetColor() const
{
	// UDynamicMaterialModel color + 40
	return FLinearColor(FColor(136, 232, 136));
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DynamicMaterialModelDynamic::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Material};
	return Categories;
}

EAssetCommandResult UAssetDefinition_DynamicMaterialModelDynamic::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	for (UObject* Object : InOpenArgs.LoadObjects<UDynamicMaterialModelDynamic>())
	{
		UDynamicMaterialModelDynamic* MaterialModel = Cast<UDynamicMaterialModelDynamic>(Object);

		if (!MaterialModel)
		{
			continue;
		}

		UWorld* World = MaterialModel->GetWorld();

		if (!World && InOpenArgs.ToolkitHost.IsValid())
		{
			World = InOpenArgs.ToolkitHost->GetWorld();
		}

		FDynamicMaterialEditorModule::Get().OpenMaterialModel(
			MaterialModel,
			World,
			/* Invoke Tab */ true
		);

		// Only process the first selected object.
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
