// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DMMaterialBuildState.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageSource.h"
#include "Components/DMMaterialValue.h"
#include "DMDefs.h"
#include "DynamicMaterialEditorSettings.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Model/DMMaterialNodeArranger.h"
#include "Model/DMMaterialBuildUtils.h"

namespace UE::DynamicMaterialEditor::Private
{
	static const TArray<UMaterialExpression*> EmptyExpressionSet = {};
	static const TArray<UDMMaterialStageSource*> EmptySourceSet = {};
	static const TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>> EmptySlotPropertyMap = {};
}

FDMMaterialBuildState::FDMMaterialBuildState(UMaterial* InDynamicMaterial, UDynamicMaterialModel* InMaterialModel, bool bInDirtyAssets)
	: DynamicMaterial(InDynamicMaterial)
	, MaterialModel(InMaterialModel)
	, bDirtyAssets(bInDirtyAssets)
	, bIgnoreUVs(false)
	, Utils(MakeShared<FDMMaterialBuildUtils>(*this))
{
	check(InDynamicMaterial);
	check(InMaterialModel);

	if (GUndo)
	{
		DynamicMaterial->Modify();
	}

	DynamicMaterial->GetEditorOnlyData()->ExpressionCollection.Empty();
	DynamicMaterial->EditorParameters.Empty();
}

FDMMaterialBuildState::~FDMMaterialBuildState()
{
	FDMMaterialNodeArranger(DynamicMaterial).ArrangeNodes();

	BuildMaterial();
}

UMaterial* FDMMaterialBuildState::GetDynamicMaterial() const
{
	return DynamicMaterial;
}

UDynamicMaterialModel* FDMMaterialBuildState::GetMaterialModel() const
{
	return MaterialModel;
}

const UDMMaterialProperty* FDMMaterialBuildState::GetCurrentMaterialProperty() const
{
	return CurrentProperty;
}

void FDMMaterialBuildState::SetCurrentMaterialProperty(const UDMMaterialProperty* InProperty)
{
	CurrentProperty = InProperty;
}

void FDMMaterialBuildState::SetIgnoreUVs()
{
	bIgnoreUVs = true;
}

void FDMMaterialBuildState::SetPreviewObject(UObject* InObject)
{
	PreviewObject = InObject;
	DynamicMaterial->MaterialDomain = EMaterialDomain::MD_UI;

	if (UDynamicMaterialEditorSettings::Get()->bPreviewImagesUseTextureUVs == false)
	{
		SetIgnoreUVs();
	}
}

IDMMaterialBuildUtilsInterface& FDMMaterialBuildState::GetBuildUtils() const
{
	return *Utils;
}

FExpressionInput* FDMMaterialBuildState::GetMaterialProperty(EDMMaterialPropertyType InProperty) const
{
	UMaterialEditorOnlyData* EditorOnlyData = DynamicMaterial->GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return nullptr;
	}

	switch (InProperty)
	{
		case EDMMaterialPropertyType::AmbientOcclusion:
			return &(EditorOnlyData->AmbientOcclusion);

		case EDMMaterialPropertyType::Anisotropy:
			return &(EditorOnlyData->Anisotropy);

		case EDMMaterialPropertyType::BaseColor:
			return &(EditorOnlyData->BaseColor);

		case EDMMaterialPropertyType::EmissiveColor:
			return &(EditorOnlyData->EmissiveColor);

		case EDMMaterialPropertyType::Metallic:
			return &(EditorOnlyData->Metallic);

		case EDMMaterialPropertyType::Normal:
			return &(EditorOnlyData->Normal);

		case EDMMaterialPropertyType::Opacity:
			return &(EditorOnlyData->Opacity);

		case EDMMaterialPropertyType::OpacityMask:
			return &(EditorOnlyData->OpacityMask);

		case EDMMaterialPropertyType::PixelDepthOffset:
			return &(EditorOnlyData->PixelDepthOffset);

		case EDMMaterialPropertyType::Refraction:
			return &(EditorOnlyData->Refraction);

		case EDMMaterialPropertyType::Roughness:
			return &(EditorOnlyData->Roughness);

		case EDMMaterialPropertyType::Specular:
			return &(EditorOnlyData->Specular);

		case EDMMaterialPropertyType::Tangent:
			return &(EditorOnlyData->Tangent);

		case EDMMaterialPropertyType::WorldPositionOffset:
			return &(EditorOnlyData->WorldPositionOffset);

		case EDMMaterialPropertyType::Displacement:
			return &(EditorOnlyData->Displacement);

		case EDMMaterialPropertyType::SubsurfaceColor:
			return &(EditorOnlyData->SubsurfaceColor);

		case EDMMaterialPropertyType::SurfaceThickness:
			return &(EditorOnlyData->SurfaceThickness);

		default:
			return nullptr;
	}
}

///////////////////////////////////////
/// Slots

bool FDMMaterialBuildState::HasSlot(const UDMMaterialSlot* InSlot) const
{
	return Slots.Contains(InSlot);
}

const TArray<UMaterialExpression*>& FDMMaterialBuildState::GetSlotExpressions(const UDMMaterialSlot* InSlot) const
{
	if (const TArray<UMaterialExpression*>* SlotExpressions = Slots.Find(InSlot))
	{
		return *SlotExpressions;
	}

	return UE::DynamicMaterialEditor::Private::EmptyExpressionSet;
}

UMaterialExpression* FDMMaterialBuildState::GetLastSlotExpression(const UDMMaterialSlot* InSlot) const
{
	const TArray<UMaterialExpression*>* SlotExpressions = Slots.Find(InSlot);

	if (ensure(SlotExpressions))
	{
		return SlotExpressions->Last();
	}

	return nullptr;
}

void FDMMaterialBuildState::AddSlotExpressions(const UDMMaterialSlot* InSlot, const TArray<UMaterialExpression*>& InSlotExpressions)
{
	if (ensure(!HasSlot(InSlot)))
	{
		if (ensure(!InSlotExpressions.IsEmpty()))
		{
			Slots.Emplace(InSlot, InSlotExpressions);
		}
		else
		{
			Slots.Emplace(InSlot, {Utils->CreateDefaultExpression()});
		}
	}
}

bool FDMMaterialBuildState::HasSlotProperties(const UDMMaterialSlot* InSlot) const
{
	if (ensure(HasSlot(InSlot)))
	{
		return SlotProperties.Contains(InSlot);
	}

	return false;
}

void FDMMaterialBuildState::AddSlotPropertyExpressions(const UDMMaterialSlot* InSlot, const TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>>& InSlotPropertyExpressions)
{
	if (ensure(!HasSlotProperties(InSlot)))
	{
		ensure(!InSlotPropertyExpressions.IsEmpty());
		SlotProperties.Emplace(InSlot, InSlotPropertyExpressions);
	}
}

const TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>>& FDMMaterialBuildState::GetSlotPropertyExpressions(const UDMMaterialSlot* InSlot)
{
	if (ensure(HasSlotProperties(InSlot)))
	{
		return SlotProperties[InSlot];
	}

	return UE::DynamicMaterialEditor::Private::EmptySlotPropertyMap;
}

UMaterialExpression* FDMMaterialBuildState::GetLastSlotPropertyExpression(const UDMMaterialSlot* InSlot, EDMMaterialPropertyType InMaterialProperty) const
{
	const TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>>* MapPtr = SlotProperties.Find(InSlot);

	if (ensure(MapPtr))
	{
		const TArray<UMaterialExpression*>* ListPtr = MapPtr->Find(InMaterialProperty);

		if (ListPtr && !ListPtr->IsEmpty())
		{
			return ListPtr->Last();
		}
	}

	return nullptr;
}

TArray<const UDMMaterialSlot*> FDMMaterialBuildState::GetSlots() const
{
	TArray<const UDMMaterialSlot*> Keys;
	Slots.GetKeys(Keys);
	return Keys;
}

const TMap<const UDMMaterialSlot*, TArray<UMaterialExpression*>>& FDMMaterialBuildState::GetSlotMap() const
{
	return Slots;
}

///////////////////////////////////////
/// Layers

bool FDMMaterialBuildState::HasLayer(const UDMMaterialLayerObject* InLayer) const
{
	return Layers.Contains(InLayer);
}

const TArray<UMaterialExpression*>& FDMMaterialBuildState::GetLayerExpressions(const UDMMaterialLayerObject* InLayer) const
{
	if (const TArray<UMaterialExpression*>* LayerExpressions = Layers.Find(InLayer))
	{
		return *LayerExpressions;
	}

	return UE::DynamicMaterialEditor::Private::EmptyExpressionSet;
}

UMaterialExpression* FDMMaterialBuildState::GetLastLayerExpression(const UDMMaterialLayerObject* InLayer) const
{
	const TArray<UMaterialExpression*>* LayerExpressions = Layers.Find(InLayer);

	if (ensure(LayerExpressions))
	{
		return LayerExpressions->Last();
	}

	return nullptr;
}

void FDMMaterialBuildState::AddLayerExpressions(const UDMMaterialLayerObject* InLayer, const TArray<UMaterialExpression*>& InLayerExpressions)
{
	if (ensure(!HasLayer(InLayer)))
	{
		if (ensure(!InLayerExpressions.IsEmpty()))
		{
			Layers.Emplace(InLayer, InLayerExpressions);
		}
		else
		{
			Layers.Emplace(InLayer, {Utils->CreateDefaultExpression()});
		}
	}
}

TArray<const UDMMaterialLayerObject*> FDMMaterialBuildState::GetLayers() const
{
	TArray<const UDMMaterialLayerObject*> Keys;
	Layers.GetKeys(Keys);
	return Keys;
}

const TMap<const UDMMaterialLayerObject*, TArray<UMaterialExpression*>>& FDMMaterialBuildState::GetLayerMap() const
{
	return Layers;
}

///////////////////////////////////////
/// Stages

bool FDMMaterialBuildState::HasStage(const UDMMaterialStage* InStage) const
{
	return Stages.Contains(InStage);
}

const TArray<UMaterialExpression*>& FDMMaterialBuildState::GetStageExpressions(const UDMMaterialStage* InStage) const
{
	if (const TArray<UMaterialExpression*>* StageExpressions = Stages.Find(InStage))
	{
		return *StageExpressions;
	}

	return UE::DynamicMaterialEditor::Private::EmptyExpressionSet;
}

UMaterialExpression* FDMMaterialBuildState::GetLastStageExpression(const UDMMaterialStage* InStage) const
{
	const TArray<UMaterialExpression*>* StageExpressions = Stages.Find(InStage);

	if (ensure(StageExpressions))
	{
		return StageExpressions->Last();
	}

	return nullptr;
}

void FDMMaterialBuildState::AddStageExpressions(const UDMMaterialStage* InStage, const TArray<UMaterialExpression*>& InStageExpressions)
{
	if (ensure(!HasStage(InStage)))
	{
		if (ensure(!InStageExpressions.IsEmpty()))
		{
			Stages.Emplace(InStage, InStageExpressions);
		}
		else
		{
			Stages.Emplace(InStage, {Utils->CreateDefaultExpression()});
		}
	}
}

TArray<const UDMMaterialStage*> FDMMaterialBuildState::GetStages() const
{
	TArray<const UDMMaterialStage*> Keys;
	Stages.GetKeys(Keys);
	return Keys;
}

const TMap<const UDMMaterialStage*, TArray<UMaterialExpression*>>& FDMMaterialBuildState::GetStageMap() const
{
	return Stages;
}

///////////////////////////////////////
/// Stage Sources

bool FDMMaterialBuildState::HasStageSource(const UDMMaterialStageSource* InStageSource) const
{
	return StageSources.Contains(InStageSource);
}

const TArray<UMaterialExpression*>& FDMMaterialBuildState::GetStageSourceExpressions(const UDMMaterialStageSource* InStageSource) const
{
	if (const TArray<UMaterialExpression*>* StageSourceExpressions = StageSources.Find(InStageSource))
	{
		return *StageSourceExpressions;
	}

	return UE::DynamicMaterialEditor::Private::EmptyExpressionSet;
}

UMaterialExpression* FDMMaterialBuildState::GetLastStageSourceExpression(const UDMMaterialStageSource* InStageSource) const
{
	const TArray<UMaterialExpression*>* StageSourceExpressions = StageSources.Find(InStageSource);

	if (ensure(StageSourceExpressions))
	{
		return StageSourceExpressions->Last();
	}

	return nullptr;
}

void FDMMaterialBuildState::AddStageSourceExpressions(const UDMMaterialStageSource* InStageSource, const TArray<UMaterialExpression*>& InStageSourceExpressions)
{
	if (ensure(!HasStageSource(InStageSource)))
	{
		if (!InStageSourceExpressions.IsEmpty())
		{
			StageSources.Emplace(InStageSource, InStageSourceExpressions);
		}
		else
		{
			StageSources.Emplace(InStageSource, {Utils->CreateDefaultExpression()});
		}
	}
}

TArray<const UDMMaterialStageSource*> FDMMaterialBuildState::GetStageSources() const
{
	TArray<const UDMMaterialStageSource*> Keys;
	StageSources.GetKeys(Keys);
	return Keys;
}

const TMap<const UDMMaterialStageSource*, TArray<UMaterialExpression*>>& FDMMaterialBuildState::GetStageSourceMap() const
{ 
	return StageSources; 
}

///////////////////////////////////////
/// Values

bool FDMMaterialBuildState::HasValue(const UDMMaterialValue* InValue) const
{
	return Values.Contains(InValue);
}

const TArray<UMaterialExpression*>& FDMMaterialBuildState::GetValueExpressions(const UDMMaterialValue* InValue) const
{
	if (ensure(HasValue(InValue)))
	{
		return Values[InValue];
	}

	return UE::DynamicMaterialEditor::Private::EmptyExpressionSet;
}

UMaterialExpression* FDMMaterialBuildState::GetLastValueExpression(const UDMMaterialValue* InValue) const
{
	const TArray<UMaterialExpression*>* ValueExpressions = Values.Find(InValue);

	if (ensure(ValueExpressions))
	{
		return ValueExpressions->Last();
	}

	return nullptr;
}

void FDMMaterialBuildState::AddValueExpressions(const UDMMaterialValue* InValue, const TArray<UMaterialExpression*>& InValueExpressions)
{
	if (ensure(!HasValue(InValue)))
	{
		if (ensure(!InValueExpressions.IsEmpty()))
		{
			Values.Emplace(InValue, InValueExpressions);
		}
		else
		{
			Values.Emplace(InValue, {Utils->CreateDefaultExpression()});
		}
	}
}

TArray<const UDMMaterialValue*> FDMMaterialBuildState::GetValues() const
{
	TArray<const UDMMaterialValue*> Keys;
	Values.GetKeys(Keys);
	return Keys;
}

const TMap<const UDMMaterialValue*, TArray<UMaterialExpression*>>& FDMMaterialBuildState::GetValueMap() const
{
	return Values;
}

///////////////////////////////////////
/// Other expression

void FDMMaterialBuildState::AddOtherExpressions(const TArray<UMaterialExpression*>& InOtherExpressions)
{
	OtherExpressions.Append(InOtherExpressions);
}

const TSet<UMaterialExpression*>& FDMMaterialBuildState::GetOtherExpressions()
{
	return OtherExpressions;
}

///////////////////////////////////////
/// Global expression

UMaterialExpression* FDMMaterialBuildState::GetGlobalExpression(FName InName) const
{
	if (UMaterialExpression* const* GlobalExpressionPtr = GlobalExpressions.Find(InName))
	{
		return *GlobalExpressionPtr;
	}

	return nullptr;
}

void FDMMaterialBuildState::SetGlobalExpression(FName InName, UMaterialExpression* InExpression)
{
	GlobalExpressions.FindOrAdd(InName) = InExpression;
}

void FDMMaterialBuildState::BuildMaterial()
{
	DynamicMaterial->UpdateCachedExpressionData();
	DynamicMaterial->CacheShaders(EMaterialShaderPrecompileMode::Background);
}
