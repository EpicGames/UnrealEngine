// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DMMaterialBuildUtils.h"

#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialStageInput.h"
#include "DynamicMaterialEditorModule.h"
#include "MaterialEditingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Model/DMMaterialBuildState.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

FDMMaterialBuildUtils::FDMMaterialBuildUtils(FDMMaterialBuildState& InBuildState)
	: BuildState(InBuildState)
{
}

UMaterialExpression* FDMMaterialBuildUtils::CreateDefaultExpression() const
{
	UMaterialExpressionConstant* Constant = CreateExpression<UMaterialExpressionConstant>(UE_DM_NodeComment_Default, BuildState.GetDynamicMaterial());
	Constant->R = 0.f;

	return Constant;
}

UMaterialExpression* FDMMaterialBuildUtils::CreateExpression(TSubclassOf<UMaterialExpression> InExpressionClass, const FString& InComment, 
	UObject* InAsset /*= nullptr*/) const
{
	check(BuildState.GetDynamicMaterial());
	check(InExpressionClass.Get());
	check(InExpressionClass.Get() != UMaterialExpression::StaticClass());

	UMaterialExpression* NewExpression = UMaterialEditingLibrary::CreateMaterialExpressionEx(
		BuildState.GetDynamicMaterial(), 
		/* In Material Function */ nullptr,
		InExpressionClass.Get(),
		InAsset, 
		/* PosX */ 0, 
		/* PosY */ 0,
		/* Mark Dirty */ BuildState.ShouldDirtyAssets()
	);

	NewExpression->Desc = InComment;

	BuildState.GetDynamicMaterial()->GetEditorOnlyData()->ExpressionCollection.AddExpression(NewExpression);

	return NewExpression;
}

UMaterialExpression* FDMMaterialBuildUtils::CreateExpressionParameter(TSubclassOf<UMaterialExpression> InExpressionClass, 
	FName InParameterName, EDMMaterialParameterGroup InParameterGroup, const FString& InComment, UObject* InAsset /*= nullptr*/) const
{
	check(BuildState.GetDynamicMaterial());
	check(InExpressionClass.Get());
	check(InExpressionClass.Get() != UMaterialExpression::StaticClass());

	UMaterialExpression* NewExpression = UMaterialEditingLibrary::CreateMaterialExpressionEx(
		BuildState.GetDynamicMaterial(),
		/* In Material Function */ nullptr,
		InExpressionClass.Get(),
		InAsset,
		/* PosX */ 0,
		/* PosY */ 0,
		/* Mark Dirty */ true
	);

	NewExpression->Desc = InComment;
	NewExpression->SetParameterName(InParameterName);

	if (FNameProperty* GroupProperty = CastField<FNameProperty>(NewExpression->GetClass()->FindPropertyByName("Group")))
	{
		int32 PropertyIndex;
		FString PropertyName;

		switch (InParameterGroup)
		{
			case EDMMaterialParameterGroup::Property:
				if (const UDMMaterialProperty* Property = BuildState.GetCurrentMaterialProperty())
				{
					const EDMMaterialPropertyType PropertyType = Property->GetMaterialProperty();
					PropertyIndex = static_cast<int32>(PropertyType);
					PropertyName = UE::DynamicMaterialEditor::Private::GetMaterialPropertyShortDisplayName(PropertyType).ToString();
					break;
				}

				// Error - fall back to hidden group
				if (!BuildState.GetPreviewObject())
				{
					UE_LOGF(LogDynamicMaterialEditor, Error, "Missing property for group type.");
				}

				// Fall through

			case EDMMaterialParameterGroup::NotExposed:
			default:
				PropertyIndex = 99;
				PropertyName = TEXT("Uncategorized");
				break;

			case EDMMaterialParameterGroup::Global:
				PropertyIndex = 0;
				PropertyName = TEXT("Global");
				break;
		}

		GroupProperty->SetValue_InContainer(
			NewExpression, 
			*FString::Printf(
				TEXT("%02d - %s"),
				PropertyIndex,
				*PropertyName
			)
		);
	}

	BuildState.GetDynamicMaterial()->GetEditorOnlyData()->ExpressionCollection.AddExpression(NewExpression);

	TArray<UMaterialExpression*>& ExpressionList = BuildState.GetDynamicMaterial()->EditorParameters.FindOrAdd(InParameterName);
	ExpressionList.Add(NewExpression);

	return NewExpression;
}

TArray<UMaterialExpression*> FDMMaterialBuildUtils::CreateExpressionInputs(const TArray<FDMMaterialStageConnection>& InInputConnectionMap,
	int32 InStageSourceInputIdx, const TArray<UDMMaterialStageInput*>& InStageInputs, int32& OutOutputIndex, int32& OutOutputChannel) const
{
	if (InStageInputs.IsEmpty())
	{
		return CreateExpressionInput(nullptr);
	}

	// TODO Combine inputs into a single output
	if (InStageInputs.Num() == 1)
	{
		TArray<UMaterialExpression*> InputExpressions = CreateExpressionInput(InStageInputs[0]);
		check(InputExpressions.IsEmpty() == false);

		static constexpr int32 InputIdx = FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;
		const FDMMaterialStageConnectorChannel* InputChannel = nullptr;

		for (const FDMMaterialStageConnectorChannel& Channel : InInputConnectionMap[InStageSourceInputIdx].Channels)
		{
			if (Channel.SourceIndex == InputIdx)
			{
				InputChannel = &Channel;
				break;
			}
		}

		if (!InputChannel)
		{
			OutOutputIndex = 0;
			OutOutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
		}
		else
		{
			const TArray<FDMMaterialStageConnector>& OutputConnectors = InStageInputs[0]->GetOutputConnectors();
			check(OutputConnectors.IsValidIndex(InputChannel->OutputIndex));

			OutOutputIndex = OutputConnectors[InputChannel->OutputIndex].Index;
			OutOutputChannel = InputChannel->OutputChannel;
		}

		return InputExpressions;
	}

	TArray<UMaterialExpression*> Expressions;
	TArray<TArray<UMaterialExpression*>> PerInputExpressions;

	// There are a 4 channels in an RGBA input.
	for (UDMMaterialStageInput* StageInput : InStageInputs)
	{
		TArray<UMaterialExpression*> InputExpressions = CreateExpressionInput(StageInput);
		PerInputExpressions.Add(InputExpressions);
		Expressions.Append(InputExpressions);
	}

	struct FDMMaskOutput
	{
		UMaterialExpression* Mask;
		int32 OutputIndex;
	};

	TArray<FDMMaskOutput> Masks;

	for (int32 StageInputIdx = 0; StageInputIdx < PerInputExpressions.Num(); ++StageInputIdx)
	{
		const int32 InputIdx = FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT + StageInputIdx;
		const FDMMaterialStageConnectorChannel* InputChannel = nullptr;

		for (const FDMMaterialStageConnectorChannel& Channel : InInputConnectionMap[InStageSourceInputIdx].Channels)
		{
			if (Channel.SourceIndex == InputIdx)
			{
				InputChannel = &Channel;
				break;
			}
		}

		UMaterialExpression* LastExpression = PerInputExpressions[StageInputIdx].Last();

		if (!InputChannel
			|| (InputChannel->OutputIndex == 0
				&& InputChannel->OutputChannel == FDMMaterialStageConnectorChannel::WHOLE_CHANNEL))
		{
			Masks.Add({LastExpression, 0});
			continue;
		}

		const int32 InnateOutputIndex = InStageInputs[InStageSourceInputIdx]->GetInnateMaskOutput(
			InInputConnectionMap[InStageSourceInputIdx].Channels[StageInputIdx].OutputIndex,
			InInputConnectionMap[InStageSourceInputIdx].Channels[StageInputIdx].OutputChannel
		);

		if (InnateOutputIndex != INDEX_NONE)
		{
			Masks.Add({LastExpression, InnateOutputIndex});
		}
		else
		{
			const TArray<FDMMaterialStageConnector>& OutputConnectors = InStageInputs[InStageSourceInputIdx]->GetOutputConnectors();
			check(OutputConnectors.IsValidIndex(InputChannel->OutputIndex));

			const int32 NodeOutputIndex = OutputConnectors[InputChannel->OutputIndex].Index;

			UMaterialExpressionComponentMask* Mask = CreateExpressionBitMask(
				LastExpression,
				NodeOutputIndex,
				InInputConnectionMap[InStageSourceInputIdx].Channels[StageInputIdx].OutputChannel
			);

			Masks.Add({Mask, 0});
			Expressions.Add(Mask);
		}
	}

	TArray<UMaterialExpressionAppendVector*> Appends;

	for (int32 StageInputIdx = 1; StageInputIdx < InStageInputs.Num(); ++StageInputIdx)
	{
		UMaterialExpression* PreviousExpression = nullptr;
		int32 PreviousOutputIndex = INDEX_NONE;

		if (StageInputIdx == 1)
		{
			PreviousExpression = Masks[0].Mask;
			PreviousOutputIndex = Masks[0].OutputIndex;
		}
		else
		{
			PreviousExpression = Appends[StageInputIdx - 1];
			PreviousOutputIndex = 0;
		}

		UMaterialExpressionAppendVector* Append = CreateExpressionAppend(
			PreviousExpression,
			PreviousOutputIndex,
			Masks[StageInputIdx].Mask,
			Masks[StageInputIdx].OutputIndex
		);

		Appends.Add(Append);
	}

	Expressions.Append(Appends);
	
	OutOutputIndex = 0;
	OutOutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

	return Expressions;
}

TArray<UMaterialExpression*> FDMMaterialBuildUtils::CreateExpressionInput(UDMMaterialStageInput* InInput) const
{
	if (!InInput)
	{
		UMaterialExpressionConstant4Vector* Constant = Cast<UMaterialExpressionConstant4Vector>(
			CreateExpression<UMaterialExpressionConstant4Vector>(UE_DM_NodeComment_Default)
		);

		Constant->Constant = FLinearColor::Black;
		Constant->Constant.A = 0.f;
		BuildState.AddOtherExpressions({Constant});
		return {Constant};
	}

	InInput->GenerateExpressions(BuildState.AsShared());
	return BuildState.GetStageSourceExpressions(InInput);
}

int32 FDMMaterialBuildUtils::FindOutputForBitmask(UMaterialExpression* InExpression, int32 InOutputChannels) const
{
	const bool bNeedsR = !!(InOutputChannels & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
	const bool bNeedsG = !!(InOutputChannels & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
	const bool bNeedsB = !!(InOutputChannels & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
	const bool bNeedsA = !!(InOutputChannels & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);

	const TArray<FExpressionOutput>& Outputs = InExpression->GetOutputs();

	for (int32 OutputIndex = 0; OutputIndex < Outputs.Num(); ++OutputIndex)
	{
		const FExpressionOutput& Output = Outputs[OutputIndex];

		const bool bMaskMatchExactR = bNeedsR == (Output.MaskR == 1);
		const bool bMaskMatchExactG = bNeedsG == (Output.MaskG == 1);
		const bool bMaskMatchExactB = bNeedsB == (Output.MaskB == 1);
		const bool bMaskMatchExactA = bNeedsA == (Output.MaskA == 1);

		if (bMaskMatchExactR && bMaskMatchExactG && bMaskMatchExactB && bMaskMatchExactA)
		{
			return OutputIndex;
		}
	}

	return INDEX_NONE;
}

UMaterialExpressionComponentMask* FDMMaterialBuildUtils::CreateExpressionBitMask(UMaterialExpression* InExpression, 
	int32 InOutputIndex, int32 InOutputChannels) const
{
	UMaterialExpressionComponentMask* Mask = Cast<UMaterialExpressionComponentMask>(
		CreateExpression<UMaterialExpressionComponentMask>(UE_DM_NodeComment_Default)
	);

	BuildState.AddOtherExpressions({Mask});

	Mask->Input.Expression = InExpression;

	const bool bNeeds[4] = {
		!!(InOutputChannels & FDMMaterialStageConnectorChannel::FIRST_CHANNEL),
		!!(InOutputChannels & FDMMaterialStageConnectorChannel::SECOND_CHANNEL),
		!!(InOutputChannels & FDMMaterialStageConnectorChannel::THIRD_CHANNEL),
		!!(InOutputChannels & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL)
	};

	/**
	 * Find the best matching expression output for our requirements.
	 */

	const TArray<FExpressionOutput>& Outputs = InExpression->GetOutputs();
	int32 OutputCountIndices[4] = {INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE};
	int32 RequiredExactOutputIndex = INDEX_NONE;
	int32 RequiredOutputIndex = INDEX_NONE;
	int32 UnmaskedOutputIndex = INDEX_NONE;

	for (int32 OutputIndex = 0; OutputIndex < Outputs.Num(); ++OutputIndex)
	{
		const FExpressionOutput& Output = Outputs[OutputIndex];

		if (!Output.Mask)
		{
			if (UnmaskedOutputIndex == INDEX_NONE)
			{
				UnmaskedOutputIndex = OutputIndex;
			}

			continue;
		}

		if (Output.MaskR)
		{
			if (OutputCountIndices[0] == INDEX_NONE)
			{
				OutputCountIndices[0] = OutputIndex;
			}

			if (Output.MaskG)
			{
				if (OutputCountIndices[1] == INDEX_NONE)
				{
					OutputCountIndices[1] = OutputIndex;
				}

				if (Output.MaskB)
				{
					if (OutputCountIndices[2] == INDEX_NONE)
					{
						OutputCountIndices[2] = OutputIndex;
					}

					if (Output.MaskA)
					{
						if (OutputCountIndices[3] == INDEX_NONE)
						{
							OutputCountIndices[3] = OutputIndex;
						}
					}
				}
			}
		}

		if (RequiredExactOutputIndex == INDEX_NONE)
		{
			const bool bMaskMatchExactR = bNeeds[0] == (Output.MaskR == 1);
			const bool bMaskMatchExactG = bNeeds[1] == (Output.MaskG == 1);
			const bool bMaskMatchExactB = bNeeds[2] == (Output.MaskB == 1);
			const bool bMaskMatchExactA = bNeeds[3] == (Output.MaskA == 1);

			if (bMaskMatchExactR && bMaskMatchExactG && bMaskMatchExactB && bMaskMatchExactA)
			{
				RequiredExactOutputIndex = OutputIndex;
			}
		}

		if (RequiredOutputIndex == INDEX_NONE)
		{
			const bool bMaskMatchR = !bNeeds[0] || Output.MaskR;
			const bool bMaskMatchG = !bNeeds[1] || Output.MaskG;
			const bool bMaskMatchB = !bNeeds[2] || Output.MaskB;
			const bool bMaskMatchA = !bNeeds[3] || Output.MaskA;

			if (bMaskMatchR && bMaskMatchG && bMaskMatchB && bMaskMatchA)
			{
				RequiredOutputIndex = OutputIndex;
			}
		}
	}

	/**
	 * Set the found output on the mask node.
	 */

	if (RequiredExactOutputIndex != INDEX_NONE)
	{
		Mask->Input.OutputIndex = RequiredExactOutputIndex;
	}
	else if (RequiredOutputIndex != INDEX_NONE)
	{
		Mask->Input.OutputIndex = RequiredOutputIndex;
	}
	else if (bNeeds[3] && OutputCountIndices[3] != INDEX_NONE)
	{
		Mask->Input.OutputIndex = OutputCountIndices[3];
	}
	else if (bNeeds[2] && OutputCountIndices[2] != INDEX_NONE)
	{
		Mask->Input.OutputIndex = OutputCountIndices[2];
	}
	else if (bNeeds[1] && OutputCountIndices[1] != INDEX_NONE)
	{
		Mask->Input.OutputIndex = OutputCountIndices[1];
	}
	else if (bNeeds[0] && OutputCountIndices[0] != INDEX_NONE)
	{
		Mask->Input.OutputIndex = OutputCountIndices[0];
	}
	else if (UnmaskedOutputIndex != INDEX_NONE)
	{
		Mask->Input.OutputIndex = UnmaskedOutputIndex;
	}
	else
	{
		Mask->Input.OutputIndex = InOutputIndex;
	}

	Mask->R = 0;
	Mask->G = 0;
	Mask->B = 0;
	Mask->A = 0;

	/**
	 * Scan the expression for the matching outputs for the mask input.
	 * E.g.
	 * Output: GBA (2,3,4) (no red channel)
	 * Required: GA (2,4)
	 * Mask: RB (1,3) (GA is shifted 1 to the left as the first output channel is missing)
	 */

	int32 StartChannel = INDEX_NONE;

	for (int32 MaskChannel = 0; MaskChannel < 4; ++MaskChannel)
	{
		bool bHasMaskChannel = false;

		if (bNeeds[MaskChannel])
		{
			if (!Outputs[Mask->Input.OutputIndex].Mask)
			{
				bHasMaskChannel = true;
			}
			else
			{
				switch (MaskChannel)
				{
					case 0:
						bHasMaskChannel = !!Outputs[Mask->Input.OutputIndex].MaskR;
						break;

					case 1:
						bHasMaskChannel = !!Outputs[Mask->Input.OutputIndex].MaskG;
						break;

					case 2:
						bHasMaskChannel = !!Outputs[Mask->Input.OutputIndex].MaskB;
						break;

					case 3:
						bHasMaskChannel = !!Outputs[Mask->Input.OutputIndex].MaskA;
						break;
				}
			}
		}

		if (!bHasMaskChannel)
		{
			continue;
		}

		if (StartChannel == INDEX_NONE)
		{
			StartChannel = MaskChannel;
		}

		switch (MaskChannel - StartChannel)
		{
			case 0:
				Mask->R = 1;
				break;

			case 1:
				Mask->G = 1;
				break;

			case 2:
				Mask->B = 1;
				break;

			case 3:
				Mask->A = 1;
				break;
		}
	}

	return Mask;
}

UMaterialExpressionAppendVector* FDMMaterialBuildUtils::CreateExpressionAppend(UMaterialExpression* InExpressionA, 
	int32 InOutputIndexA, UMaterialExpression* InExpressionB, int32 InOutputIndexB) const
{
	UMaterialExpressionAppendVector* Append = Cast<UMaterialExpressionAppendVector>(
		CreateExpression<UMaterialExpressionAppendVector>(UE_DM_NodeComment_Default)
	);

	Append->A.Expression = InExpressionA;
	Append->A.OutputIndex = InOutputIndexA;

	Append->B.Expression = InExpressionB;
	Append->B.OutputIndex = InOutputIndexB;

	return Append;
}

void FDMMaterialBuildUtils::UpdatePreviewMaterial(UMaterialExpression* InLastExpression, int32 OutputIdx, int32 InOutputChannel, 
	int32 InSize) const
{
	FColorMaterialInput& EmissiveColor = BuildState.GetDynamicMaterial()->GetEditorOnlyData()->EmissiveColor;

	EmissiveColor.Expression = nullptr;
	EmissiveColor.OutputIndex = 0;
	EmissiveColor.SetMask(0, 0, 0, 0, 0);

	if (!InLastExpression)
	{
		return;
	}

	if (UMaterialExpressionTextureObject* TextureObject = Cast<UMaterialExpressionTextureObject>(InLastExpression))
	{
		UMaterialExpressionTextureSample* NewSampler = CreateExpression<UMaterialExpressionTextureSample>(UE_DM_NodeComment_Default);
		NewSampler->Desc = "Auto sampler";
		BuildState.AddOtherExpressions({NewSampler});

		TextureObject->ConnectExpression(NewSampler->GetInput(1), 0);

		EmissiveColor.Expression = NewSampler;
		EmissiveColor.OutputIndex = 0;
	}
	else if (UMaterialExpressionTextureObjectParameter* TextureObjectParam = Cast<UMaterialExpressionTextureObjectParameter>(InLastExpression))
	{
		UMaterialExpressionTextureSample* NewSampler = CreateExpression<UMaterialExpressionTextureSample>(UE_DM_NodeComment_Default);
		NewSampler->Desc = "Auto sampler";
		BuildState.AddOtherExpressions({NewSampler});

		TextureObjectParam->ConnectExpression(NewSampler->GetInput(1), 0);

		EmissiveColor.Expression = NewSampler;
		EmissiveColor.OutputIndex = 0;
	}
	// Single material property, connect it up to emissive and output it.
	else
	{
		EmissiveColor.Expression = InLastExpression;

		const TArray<FExpressionOutput>& Outputs = InLastExpression->GetOutputs();

		if (Outputs.IsValidIndex(OutputIdx))
		{
			EmissiveColor.OutputIndex = OutputIdx;
			UE::DynamicMaterialEditor::Private::SetMask(EmissiveColor, Outputs[OutputIdx], InOutputChannel);
		}
	}
}
