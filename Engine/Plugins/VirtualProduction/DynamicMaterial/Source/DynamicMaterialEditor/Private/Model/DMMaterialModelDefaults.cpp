// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DMMaterialModelDefaults.h"

#include "Components/DecalComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DynamicMaterialEditorModule.h"
#include "Delegates/IDelegateInstance.h"
#include "Model/DMOnWizardCompleteCallback.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

namespace UE::DynamicMaterialEditor::Private
{
	TSharedPtr<IDMOnWizardCompleteCallback> DefaultsCallback_StaticMeshComponent;
	TSharedPtr<IDMOnWizardCompleteCallback> DefaultsCallback_DecalComponent;
}

struct FDMMaterialModelCreatedCallback_StaticMeshComponent : FDMMaterialModelCreatedCallbackBase
{
	FDMMaterialModelCreatedCallback_StaticMeshComponent(uint32 InPriority)
		: FDMMaterialModelCreatedCallbackBase(InPriority)
	{		
	}

	virtual ~FDMMaterialModelCreatedCallback_StaticMeshComponent() override = default;

	//~ Begin IDMMaterialModelCreatedCallback
	virtual void OnModelCreated(const FDMOnWizardCompleteCallbackParams& InParams) override
	{
		if (!::IsValid(InParams.EditorOnlyData))
		{
			return;
		}

		UStaticMeshComponent* Component = InParams.MaterialModel->GetTypedOuter<UStaticMeshComponent>();

		if (!::IsValid(Component))
		{
			return;
		}

		FVector Min, Max;
		Component->GetLocalBounds(Min, Max);

		if (FMath::IsNearlyEqual(Min.X, Max.X)
			|| FMath::IsNearlyEqual(Min.Y, Max.Y)
			|| FMath::IsNearlyEqual(Min.Z, Max.Z))
		{
			InParams.EditorOnlyData->SetBlendMode(EBlendMode::BLEND_Translucent);
		}
		else
		{
			InParams.EditorOnlyData->SetBlendMode(EBlendMode::BLEND_Opaque);
		}
	}
	//~ End IDMMaterialModelCreatedCallback
};

struct FDMMaterialModelCreatedCallback_DecalComponent : FDMMaterialModelCreatedCallbackBase
{
	FDMMaterialModelCreatedCallback_DecalComponent(uint32 InPriority)
		: FDMMaterialModelCreatedCallbackBase(InPriority)
	{
	}

	virtual ~FDMMaterialModelCreatedCallback_DecalComponent() override = default;

	//~ Begin IDMMaterialModelCreatedCallback
	virtual void OnModelCreated(const FDMOnWizardCompleteCallbackParams& InParams) override
	{
		if (!::IsValid(InParams.EditorOnlyData))
		{
			return;
		}

		UDecalComponent* Component = InParams.MaterialModel->GetTypedOuter<UDecalComponent>();

		if (!::IsValid(Component))
		{
			return;
		}

		InParams.EditorOnlyData->SetDomain(EMaterialDomain::MD_DeferredDecal);
	}
	//~ End IDMMaterialModelCreatedCallback
};

void FDMMaterialModelDefaults::RegisterDefaultsDelegates()
{
	using namespace UE::DynamicMaterialEditor::Private;

	UnregsiterDefaultsDelegates();

	DefaultsCallback_StaticMeshComponent = IDynamicMaterialEditorModule::Get().RegisterMaterialModelCreatedCallback<FDMMaterialModelCreatedCallback_StaticMeshComponent>(
		1000
	);

	DefaultsCallback_DecalComponent = IDynamicMaterialEditorModule::Get().RegisterMaterialModelCreatedCallback<FDMMaterialModelCreatedCallback_DecalComponent>(
		2000
	);
}

void FDMMaterialModelDefaults::UnregsiterDefaultsDelegates()
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (DefaultsCallback_StaticMeshComponent.IsValid())
	{
		FDynamicMaterialEditorModule::Get().UnregisterMaterialModelCreatedCallback(DefaultsCallback_StaticMeshComponent.ToSharedRef());
		DefaultsCallback_StaticMeshComponent.Reset();
	}

	if (DefaultsCallback_DecalComponent.IsValid())
	{
		FDynamicMaterialEditorModule::Get().UnregisterMaterialModelCreatedCallback(DefaultsCallback_DecalComponent.ToSharedRef());
		DefaultsCallback_DecalComponent.Reset();
	}
}
