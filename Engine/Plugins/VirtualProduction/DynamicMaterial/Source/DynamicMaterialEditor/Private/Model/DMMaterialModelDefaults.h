// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UDynamicMaterialModel;

class FDMMaterialModelDefaults
{
public:
	static void RegisterDefaultsDelegates();

	static void UnregsiterDefaultsDelegates();
};
