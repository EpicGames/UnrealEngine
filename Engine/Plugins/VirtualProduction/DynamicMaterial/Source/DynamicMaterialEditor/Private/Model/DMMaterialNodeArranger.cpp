// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DMMaterialNodeArranger.h"
#include "Containers/ArrayView.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"

namespace UE::DynamicMaterialEditor::BuildState::Private
{
	static constexpr int32 SpaceBetweenNodes = 50;
}

FDMMaterialNodeArranger::FDMMaterialNodeArranger(UMaterial* InDynamicMaterial)
	: DynamicMaterial(InDynamicMaterial)
	, OffsetStart({0, 0})
{
}

void FDMMaterialNodeArranger::ArrangeNodes()
{
	UMaterialEditorOnlyData* EditorOnlyData = DynamicMaterial->GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return;
	}

	ArrangeMaterialInputNodes(EditorOnlyData->BaseColor.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->EmissiveColor.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Opacity.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->OpacityMask.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Metallic.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Specular.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Roughness.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Anisotropy.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Normal.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Tangent.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->WorldPositionOffset.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Refraction.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->AmbientOcclusion.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->PixelDepthOffset.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->Displacement.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->SubsurfaceColor.Expression);
	ArrangeMaterialInputNodes(EditorOnlyData->SurfaceThickness.Expression);

	FInt32Interval Vertical = {0, 0};

	for (const TPair<UMaterialExpression*, FIntPoint>& Pair : NodePositions)
	{
		Vertical.Min = FMath::Min(Vertical.Min, Pair.Value.Y);
		Vertical.Max = FMath::Max(Vertical.Max, Pair.Value.Y);
	}

	const int32 Offset = (Vertical.Max - Vertical.Min) / -2;

	for (const TPair<UMaterialExpression*, FIntPoint>& Pair : NodePositions)
	{
		Pair.Key->MaterialExpressionEditorY += Offset;
	}
}

int32 FDMMaterialNodeArranger::GetNodeWidth(UMaterialExpression& InNode)
{
	constexpr int32 LetterWidth = 8; // It's not monospaced, but it's a good estimate.
	constexpr int32 Padding = 50;

	auto EstimateNodeWidthBasedOnTitle = [](const FString& InTitle)
		{
			if (!InTitle.IsEmpty())
			{
				return (InTitle.Len() * LetterWidth) + Padding;
			}

			return -1;
		};

	const int32 NodeWidth = InNode.GetWidth() * 2; // Because it's just not enough

	// We have no graph node, so let's estimate.
	if (InNode.HasAParameterName())
	{
		const FString ParameterName = InNode.GetParameterName().ToString();
		const int32 Width = EstimateNodeWidthBasedOnTitle(ParameterName);

		if (Width > 0)
		{
			return FMath::Max(NodeWidth, Width);
		}
	}

	TArray<FString> Captions;
	InNode.GetCaption(Captions);
	int32 Width = 0;

	for (const FString& Caption : Captions)
	{
		Width = FMath::Max(Width, EstimateNodeWidthBasedOnTitle(Caption));
	}

	if (Width > 0)
	{
		return FMath::Max(NodeWidth, Width);
	}

	Width = EstimateNodeWidthBasedOnTitle(InNode.GetClass()->GetDescription());

	return FMath::Max(NodeWidth, Width);
}

void FDMMaterialNodeArranger::ArrangeMaterialInputNodes(UMaterialExpression* MaterialInputExpression)
{
	if (!MaterialInputExpression)
	{
		return;
	}

	FIntPoint NodeSize = {0, 0};
	ArrangeNode(NodePositions, OffsetStart, MaterialInputExpression, NodeSize);

	OffsetStart.X += NodeSize.X;
	OffsetStart.Y += NodeSize.Y + UE::DynamicMaterialEditor::BuildState::Private::SpaceBetweenNodes;
}

void FDMMaterialNodeArranger::ArrangeNode(TMap<UMaterialExpression*, FIntPoint>& InOutNodePositions, const FIntPoint& InOffsetStart, 
	UMaterialExpression* InNode, FIntPoint& InOutNodeSize)
{
	if (!InNode)
	{
		return;
	}

	InOutNodeSize = {0, 0};
	const FIntPoint ThisNodeSize = {GetNodeWidth(*InNode), InNode->GetHeight()};
	FIntPoint ChildOffsetStart = InOffsetStart;
	ChildOffsetStart.X += ThisNodeSize.X + UE::DynamicMaterialEditor::BuildState::Private::SpaceBetweenNodes;

	for (FExpressionInputIterator It{ InNode }; It; ++It)
	{
		if (!It->IsConnected())
		{
			continue;
		}

		FIntPoint ChildNodeSize;
		const FIntPoint* NodePosition = InOutNodePositions.Find(It->Expression);

		if (NodePosition && (-NodePosition->X) > InOffsetStart.X)
		{
			ChildNodeSize = {It->Expression->GetWidth(), It->Expression->GetHeight()};
		}
		else
		{
			ArrangeNode(InOutNodePositions, ChildOffsetStart, It->Expression, ChildNodeSize);
		}

		if (ChildNodeSize.X > 0)
		{
			InOutNodeSize.X = FMath::Max(InOutNodeSize.X, ChildNodeSize.X);
		}

		if (ChildNodeSize.Y > 0)
		{
			if (InOutNodeSize.Y > 0)
			{
				InOutNodeSize.Y += UE::DynamicMaterialEditor::BuildState::Private::SpaceBetweenNodes;
			}

			InOutNodeSize.Y += ChildNodeSize.Y;
		}

		ChildOffsetStart.Y = InOffsetStart.Y + InOutNodeSize.Y + UE::DynamicMaterialEditor::BuildState::Private::SpaceBetweenNodes;
	}

	if (InOutNodeSize.X > 0)
	{
		InOutNodeSize.X += UE::DynamicMaterialEditor::BuildState::Private::SpaceBetweenNodes;
	}

	InOutNodeSize.X += ThisNodeSize.X;
	InNode->MaterialExpressionEditorX = -InOffsetStart.X - ThisNodeSize.X;

	if (InOutNodeSize.Y == 0)
	{
		InOutNodeSize.Y = ThisNodeSize.Y;
		InNode->MaterialExpressionEditorY = InOffsetStart.Y;
	}
	else if (InOutNodeSize.Y <= ThisNodeSize.Y)
	{
		InOutNodeSize = ThisNodeSize.Y;
		InNode->MaterialExpressionEditorY = InOffsetStart.Y;
	}
	else
	{
		InNode->MaterialExpressionEditorY = InOffsetStart.Y + (InOutNodeSize.Y - ThisNodeSize.Y) / 2;
	}

	InOutNodePositions.Emplace(InNode, FIntPoint(InNode->MaterialExpressionEditorX, InNode->MaterialExpressionEditorY));
}
