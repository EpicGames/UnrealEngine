// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Math/IntPoint.h"

class UMaterial;
class UMaterialExpression;

/**
 * Algorithm for arranging material nodes.
 */
class FDMMaterialNodeArranger
{
public:
	FDMMaterialNodeArranger(UMaterial* InDynamicMaterial);

	void ArrangeNodes();

protected:
	static int32 GetNodeWidth(UMaterialExpression& InNode);

	UMaterial* DynamicMaterial;
	TMap<UMaterialExpression*, FIntPoint> NodePositions;
	FIntPoint OffsetStart = {0, 0};

	void ArrangeMaterialInputNodes(UMaterialExpression* MaterialInputExpression);

	void ArrangeNode(TMap<UMaterialExpression*, FIntPoint>& InOutNodePositions, const FIntPoint& InOffsetStart,
		UMaterialExpression* InNode, FIntPoint& InOutNodeSize);
};
