// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DMOnWizardCompleteCallback.h"

FDMMaterialModelCreatedCallbackBase::FDMMaterialModelCreatedCallbackBase(uint32 InPriority)
	: Priority(InPriority)
{
}

uint32 FDMMaterialModelCreatedCallbackBase::GetPriority() const
{
	return Priority;
}

FDMMaterialModelCreatedCallbackDelegate::FDMMaterialModelCreatedCallbackDelegate(uint32 InPriority, const FOnModelCreated& InOnModelCreatedDelegate)
	: FDMMaterialModelCreatedCallbackBase(InPriority)
	, OnModelCreatedDelegate(InOnModelCreatedDelegate)
{
}

void FDMMaterialModelCreatedCallbackDelegate::OnModelCreated(const FDMOnWizardCompleteCallbackParams& InParams)
{
	OnModelCreatedDelegate.ExecuteIfBound(InParams);
}
