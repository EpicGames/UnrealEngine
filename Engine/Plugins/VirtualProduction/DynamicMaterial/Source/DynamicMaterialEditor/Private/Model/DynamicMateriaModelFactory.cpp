// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialEditorModule.h"
#include "Components/DMMaterialComponent.h"
#include "Model/DynamicMaterialModelFactory.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

#define LOCTEXT_NAMESPACE "MaterialDesignerModelFactory"

const FString UDynamicMaterialModelFactory::BaseDirectory = FString("/Game/DynamicMaterials");
const FString UDynamicMaterialModelFactory::BaseName = FString("M_DynMat_");

UDynamicMaterialModelFactory::UDynamicMaterialModelFactory()
{
	SupportedClass = UDynamicMaterialModelEditorOnlyData::StaticClass();

	bCreateNew = false;
	bEditAfterNew = true;
	bEditorImport = false;
	bText = false;
}

UObject* UDynamicMaterialModelFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, 
	EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	check(InClass->IsChildOf(UDynamicMaterialModel::StaticClass()));

	if (InName.IsNone())
	{
		InName = MakeUniqueObjectName(InParent, UDynamicMaterialModel::StaticClass(), TEXT("MaterialDesignerModel"));
	}

	UDynamicMaterialModel* NewModel = NewObject<UDynamicMaterialModel>(InParent, InClass, InName, InFlags | RF_Transactional);

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = NewObject<UDynamicMaterialModelEditorOnlyData>(
		NewModel,
		UDynamicMaterialModelEditorOnlyData::StaticClass(),
		"EditorOnlyDataSI",
		RF_Transactional
	);

	NewModel->EditorOnlyDataSI = ModelEditorOnlyData;
	ModelEditorOnlyData->MaterialModel = NewModel;

	const FDMInitializationGuard InitGuard;
	ModelEditorOnlyData->Initialize();

	return NewModel;
}

FText UDynamicMaterialModelFactory::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("MaterialDesignerModel", "Material Designer Model");
	return DisplayName;
}

FText UDynamicMaterialModelFactory::GetToolTip() const
{
	static const FText Tooltip = LOCTEXT("MaterialDesignerModelTooltip", "The Material Designer is a more intuitive way to create materials for people coming from other software.");
	return Tooltip;
}

#undef LOCTEXT_NAMESPACE
