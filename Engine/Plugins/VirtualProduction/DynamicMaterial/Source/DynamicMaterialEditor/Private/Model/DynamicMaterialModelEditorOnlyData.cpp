// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DynamicMaterialModelEditorOnlyData.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialValue.h"
#include "Components/DMTextureUV.h"
#include "Components/MaterialProperties/DMMPAmbientOcclusion.h"
#include "Components/MaterialProperties/DMMPAnisotropy.h"
#include "Components/MaterialProperties/DMMPBaseColor.h"
#include "Components/MaterialProperties/DMMPDisplacement.h"
#include "Components/MaterialProperties/DMMPEmissiveColor.h"
#include "Components/MaterialProperties/DMMPMetallic.h"
#include "Components/MaterialProperties/DMMPNormal.h"
#include "Components/MaterialProperties/DMMPOpacity.h"
#include "Components/MaterialProperties/DMMPOpacityMask.h"
#include "Components/MaterialProperties/DMMPPixelDepthOffset.h"
#include "Components/MaterialProperties/DMMPRefraction.h"
#include "Components/MaterialProperties/DMMPRoughness.h"
#include "Components/MaterialProperties/DMMPSpecular.h"
#include "Components/MaterialProperties/DMMPSubsurfaceColor.h"
#include "Components/MaterialProperties/DMMPSurfaceThickness.h"
#include "Components/MaterialProperties/DMMPTangent.h"
#include "Components/MaterialProperties/DMMPWorldPositionOffset.h"
#include "CoreGlobals.h"
#include "DMComponentPath.h"
#include "DMDefs.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialModule.h"
#include "Factories/MaterialFactoryNew.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "Misc/Guid.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyDataVersion.h"
#include "UObject/Package.h"
#include "Utils/DMBuildRequestSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialModelEditorOnlyData)

#define LOCTEXT_NAMESPACE "MaterialDesignerModel"

const FString UDynamicMaterialModelEditorOnlyData::SlotsPathToken               = FString(TEXT("Slots"));
const FString UDynamicMaterialModelEditorOnlyData::BaseColorSlotPathToken       = FString(TEXT("BaseColor"));
const FString UDynamicMaterialModelEditorOnlyData::EmissiveSlotPathToken        = FString(TEXT("Emissive"));
const FString UDynamicMaterialModelEditorOnlyData::OpacitySlotPathToken         = FString(TEXT("Opacity"));
const FString UDynamicMaterialModelEditorOnlyData::RoughnessPathToken           = FString(TEXT("Roughness"));
const FString UDynamicMaterialModelEditorOnlyData::SpecularPathToken            = FString(TEXT("Specular"));
const FString UDynamicMaterialModelEditorOnlyData::MetallicPathToken            = FString(TEXT("Metallic"));
const FString UDynamicMaterialModelEditorOnlyData::NormalPathToken              = FString(TEXT("Normal"));
const FString UDynamicMaterialModelEditorOnlyData::PixelDepthOffsetPathToken    = FString(TEXT("PDO"));
const FString UDynamicMaterialModelEditorOnlyData::WorldPositionOffsetPathToken = FString(TEXT("WPO"));
const FString UDynamicMaterialModelEditorOnlyData::AmbientOcclusionPathToken    = FString(TEXT("AO"));
const FString UDynamicMaterialModelEditorOnlyData::AnisotropyPathToken          = FString(TEXT("Anisotropy"));
const FString UDynamicMaterialModelEditorOnlyData::RefractionPathToken          = FString(TEXT("Refraction"));
const FString UDynamicMaterialModelEditorOnlyData::TangentPathToken             = FString(TEXT("Tangent"));
const FString UDynamicMaterialModelEditorOnlyData::DisplacementPathToken        = FString(TEXT("Displacement"));
const FString UDynamicMaterialModelEditorOnlyData::SubsurfaceColorPathToken     = FString(TEXT("SubsurfaceColor"));
const FString UDynamicMaterialModelEditorOnlyData::SurfaceThicknessPathToken    = FString(TEXT("SurfaceThickness"));
const FString UDynamicMaterialModelEditorOnlyData::Custom1PathToken             = FString(TEXT("Custom1"));
const FString UDynamicMaterialModelEditorOnlyData::Custom2PathToken             = FString(TEXT("Custom2"));
const FString UDynamicMaterialModelEditorOnlyData::Custom3PathToken             = FString(TEXT("Custom3"));
const FString UDynamicMaterialModelEditorOnlyData::Custom4PathToken             = FString(TEXT("Custom4"));
const FString UDynamicMaterialModelEditorOnlyData::PropertiesPathToken          = FString(TEXT("Properties"));

namespace UE::DynamicMaterialEditor::Private
{
	const TMap<FString, EDMMaterialPropertyType> TokenToPropertyMap = {
		{UDynamicMaterialModelEditorOnlyData::BaseColorSlotPathToken,       EDMMaterialPropertyType::BaseColor},
		{UDynamicMaterialModelEditorOnlyData::EmissiveSlotPathToken,        EDMMaterialPropertyType::EmissiveColor},
		{UDynamicMaterialModelEditorOnlyData::OpacitySlotPathToken,         EDMMaterialPropertyType::Opacity},
		{UDynamicMaterialModelEditorOnlyData::RoughnessPathToken,           EDMMaterialPropertyType::Roughness},
		{UDynamicMaterialModelEditorOnlyData::SpecularPathToken,            EDMMaterialPropertyType::Specular},
		{UDynamicMaterialModelEditorOnlyData::MetallicPathToken,            EDMMaterialPropertyType::Metallic},
		{UDynamicMaterialModelEditorOnlyData::NormalPathToken,              EDMMaterialPropertyType::Normal},
		{UDynamicMaterialModelEditorOnlyData::PixelDepthOffsetPathToken,    EDMMaterialPropertyType::PixelDepthOffset},
		{UDynamicMaterialModelEditorOnlyData::WorldPositionOffsetPathToken, EDMMaterialPropertyType::WorldPositionOffset},
		{UDynamicMaterialModelEditorOnlyData::AmbientOcclusionPathToken,    EDMMaterialPropertyType::AmbientOcclusion},
		{UDynamicMaterialModelEditorOnlyData::AnisotropyPathToken,          EDMMaterialPropertyType::Anisotropy},
		{UDynamicMaterialModelEditorOnlyData::RefractionPathToken,          EDMMaterialPropertyType::Refraction},
		{UDynamicMaterialModelEditorOnlyData::TangentPathToken,             EDMMaterialPropertyType::Tangent},
		{UDynamicMaterialModelEditorOnlyData::DisplacementPathToken,        EDMMaterialPropertyType::Displacement},
		{UDynamicMaterialModelEditorOnlyData::SubsurfaceColorPathToken,     EDMMaterialPropertyType::SubsurfaceColor},
		{UDynamicMaterialModelEditorOnlyData::SurfaceThicknessPathToken,    EDMMaterialPropertyType::SurfaceThickness},
		{UDynamicMaterialModelEditorOnlyData::Custom1PathToken,             EDMMaterialPropertyType::Custom1},
		{UDynamicMaterialModelEditorOnlyData::Custom2PathToken,             EDMMaterialPropertyType::Custom2},
		{UDynamicMaterialModelEditorOnlyData::Custom3PathToken,             EDMMaterialPropertyType::Custom3},
		{UDynamicMaterialModelEditorOnlyData::Custom4PathToken,             EDMMaterialPropertyType::Custom4}
	};
}

const FName UDynamicMaterialModelEditorOnlyData::AlphaValueName = TEXT("AlphaValue");

const TArray<EMaterialDomain> UDynamicMaterialModelEditorOnlyData::SupportedDomains =
{
	EMaterialDomain::MD_Surface,
	EMaterialDomain::MD_PostProcess,
	EMaterialDomain::MD_DeferredDecal,
	EMaterialDomain::MD_LightFunction
};

const TArray<EBlendMode> UDynamicMaterialModelEditorOnlyData::SupportedBlendModes =
{
	EBlendMode::BLEND_Opaque,
	EBlendMode::BLEND_Masked,
	EBlendMode::BLEND_Translucent,
	EBlendMode::BLEND_Additive,
	EBlendMode::BLEND_Modulate
};

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(UDynamicMaterialModelBase* InModelBase)
{
	if (InModelBase)
	{
		return Get(InModelBase->ResolveMaterialModel());
	}

	return nullptr;
}

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(const TWeakObjectPtr<UDynamicMaterialModelBase>& InModelBaseWeak)
{
	return Get(InModelBaseWeak.Get());
}

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(UDynamicMaterialModel* InModel)
{
	if (InModel)
	{
		return Get(InModel->GetEditorOnlyData());
	}

	return nullptr;
}

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(const TWeakObjectPtr<UDynamicMaterialModel>& InModelWeak)
{
	return Get(InModelWeak.Get());
}

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(const TScriptInterface<IDynamicMaterialModelEditorOnlyDataInterface>& InInterface)
{
	return Cast<UDynamicMaterialModelEditorOnlyData>(InInterface.GetObject());
}

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(IDynamicMaterialModelEditorOnlyDataInterface* InInterface)
{
	return Cast<UDynamicMaterialModelEditorOnlyData>(InInterface);
}

UDynamicMaterialModelEditorOnlyData* UDynamicMaterialModelEditorOnlyData::Get(UDynamicMaterialInstance* InInstance)
{
	if (InInstance)
	{
		return Get(InInstance->GetMaterialModel());
	}

	return nullptr;
}

UDynamicMaterialModelEditorOnlyData::UDynamicMaterialModelEditorOnlyData()
	: State(EDMState::Idle)
	, Domain(EMaterialDomain::MD_Surface)
	, BlendMode(EBlendMode::BLEND_Opaque)
	, ShadingModel(EDMMaterialShadingModel::DefaultLit)
	, UsageFlags(0)
	, GeneralFlags(static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided))
	, LightingFlags(0)
	, TranslucencyFlags(0)
	, MotionFlags(static_cast<uint8>(EDMMaterialFlagsMotion::OutputTranslucentVelocity | EDMMaterialFlagsMotion::ResponsiveAA))
	, ForwardRendererFlags(0)
	, OpacityMaskClipValue(0.5f)
	, DisplacementCenter(0.5f)
	, DisplacementMagnitude(1.f)
	, bBuildRequested(false)
{
	BaseColor =           CreateDefaultSubobject<UDMMaterialPropertyBaseColor>(          "MaterialProperty_BaseColor");
	EmissiveColor =       CreateDefaultSubobject<UDMMaterialPropertyEmissiveColor>(      "MaterialProperty_EmissiveColor");
	Opacity =             CreateDefaultSubobject<UDMMaterialPropertyOpacity>(            "MaterialProperty_Opacity");
	OpacityMask =         CreateDefaultSubobject<UDMMaterialPropertyOpacityMask>(        "MaterialProperty_OpacityMask");
	Roughness =           CreateDefaultSubobject<UDMMaterialPropertyRoughness>(          "MaterialProperty_Roughness");
	Specular =            CreateDefaultSubobject<UDMMaterialPropertySpecular>(           "MaterialProperty_Specular");
	Metallic =            CreateDefaultSubobject<UDMMaterialPropertyMetallic>(           "MaterialProperty_Metallic");
	Normal =              CreateDefaultSubobject<UDMMaterialPropertyNormal>(             "MaterialProperty_Normal");
	PixelDepthOffset =    CreateDefaultSubobject<UDMMaterialPropertyPixelDepthOffset>(   "MaterialProperty_PixelDepthOffset");
	WorldPositionOffset = CreateDefaultSubobject<UDMMaterialPropertyWorldPositionOffset>("MaterialProperty_WorldPositionOffset");
	AmbientOcclusion =    CreateDefaultSubobject<UDMMaterialPropertyAmbientOcclusion>(   "MaterialProperty_AmbientOcclusion");
	Anisotropy =          CreateDefaultSubobject<UDMMaterialPropertyAnisotropy>(         "MaterialProperty_Anisotropy");
	Refraction =          CreateDefaultSubobject<UDMMaterialPropertyRefraction>(         "MaterialProperty_Refraction");
	Tangent =             CreateDefaultSubobject<UDMMaterialPropertyTangent>(            "MaterialProperty_Tangent");
	Displacement =        CreateDefaultSubobject<UDMMaterialPropertyDisplacement>(       "MaterialProperty_Displacement");
	SubsurfaceColor =     CreateDefaultSubobject<UDMMaterialPropertySubsurfaceColor>(    "MaterialProperty_SubsurfaceColor");
	SurfaceThickness =    CreateDefaultSubobject<UDMMaterialPropertySurfaceThickness>(   "MaterialProperty_SurfaceThickness");

	Custom1 = UDMMaterialProperty::CreateCustomMaterialPropertyDefaultSubobject(this, EDMMaterialPropertyType::Custom1, "MaterialProperty_Custom1");
	Custom2 = UDMMaterialProperty::CreateCustomMaterialPropertyDefaultSubobject(this, EDMMaterialPropertyType::Custom2, "MaterialProperty_Custom2");
	Custom3 = UDMMaterialProperty::CreateCustomMaterialPropertyDefaultSubobject(this, EDMMaterialPropertyType::Custom3, "MaterialProperty_Custom3");
	Custom4 = UDMMaterialProperty::CreateCustomMaterialPropertyDefaultSubobject(this, EDMMaterialPropertyType::Custom4, "MaterialProperty_Custom4");

	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this](EDMMaterialPropertyType InType)
		{
			if (UDMMaterialProperty* Property = GetMaterialProperty(InType))
			{
				Property->SetComponentState(EDMComponentLifetimeState::Added);
			}

			return EDMIterationResult::Continue;
		}
	);
}

void UDynamicMaterialModelEditorOnlyData::AssignPropertyAlphaValues()
{
	BaseColor          ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalBaseColorValueName));
	EmissiveColor      ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalEmissiveColorValueName));
	Opacity            ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOpacityValueName));
	OpacityMask        ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOpacityValueName));
	Metallic           ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalMetallicValueName));
	Specular           ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalSpecularValueName));
	Roughness          ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalRoughnessValueName));
	Normal             ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalNormalValueName));
	Anisotropy         ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalAnisotropyValueName));
	WorldPositionOffset->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalWorldPositionOffsetValueName));
	AmbientOcclusion   ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalAmbientOcclusionValueName));
	Refraction         ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalRefractionValueName));
	Tangent            ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalTangentValueName));
	PixelDepthOffset   ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalPixelDepthOffsetValueName));
	Displacement       ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalDisplacementValueName));
	SubsurfaceColor    ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalSubsurfaceColorValueName));
	SurfaceThickness   ->AddComponent(AlphaValueName, MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalSurfaceThicknessValueName));
}

void UDynamicMaterialModelEditorOnlyData::OnDomainChanged()
{
	switch (Domain)
	{
		case EMaterialDomain::MD_PostProcess:
		case EMaterialDomain::MD_LightFunction:
		{
			const FDMUpdateGuard Guard;

			// Post process only supports emissive.
			UDMMaterialSlot* BaseColorSlot = GetSlotForMaterialProperty(EDMMaterialPropertyType::BaseColor);
			UDMMaterialSlot* EmissiveSlot = GetSlotForMaterialProperty(EDMMaterialPropertyType::EmissiveColor);

			if (!EmissiveSlot)
			{
				if (BaseColorSlot)
				{
					EnsureSwapSlotMaterialProperty(EDMMaterialPropertyType::BaseColor, EDMMaterialPropertyType::EmissiveColor);
				}
				else
				{
					AddSlotForMaterialProperty(EDMMaterialPropertyType::EmissiveColor);
				}
			}

			SetShadingModel(EDMMaterialShadingModel::Unlit);
			SetBlendMode(EBlendMode::BLEND_Opaque);
			break;
		}

		case EMaterialDomain::MD_DeferredDecal:
			SetShadingModel(EDMMaterialShadingModel::DefaultLit);
			SetBlendMode(EBlendMode::BLEND_Translucent);
			break;
	}

	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::OnBlendModeChanged()
{
	switch (BlendMode)
	{
		case EBlendMode::BLEND_Opaque:
			RemoveSlotForMaterialProperty(EDMMaterialPropertyType::Opacity);
			RemoveSlotForMaterialProperty(EDMMaterialPropertyType::OpacityMask);
			break;

		case EBlendMode::BLEND_Masked:
			EnsureSwapSlotMaterialProperty(EDMMaterialPropertyType::Opacity, EDMMaterialPropertyType::OpacityMask);
			break;

		case EBlendMode::BLEND_Translucent:
		case EBlendMode::BLEND_Additive:
		case EBlendMode::BLEND_Modulate:
			EnsureSwapSlotMaterialProperty(EDMMaterialPropertyType::OpacityMask, EDMMaterialPropertyType::Opacity);
			break;
	}

	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::OnShadingModelChanged()
{
	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::OnMaterialFlagChanged()
{
	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::OnMaterialSettingsChanged()
{
	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::Initialize()
{
	if (!Slots.IsEmpty())
	{
		return;
	}

	AssignPropertyAlphaValues();
}

UMaterial* UDynamicMaterialModelEditorOnlyData::GetGeneratedMaterial() const
{
	return IsValid(MaterialModel) ? MaterialModel->DynamicMaterial : nullptr;
}

void UDynamicMaterialModelEditorOnlyData::CreateMaterial()
{
	if (!IsValid(MaterialModel))
	{
		return;
	}

	if (FDynamicMaterialModule::IsMaterialExportEnabled() == false)
	{
		UMaterialFactoryNew* MaterialFactory = NewObject<UMaterialFactoryNew>();
		check(MaterialFactory);

		// Replace existing material
		MaterialModel->DynamicMaterial = Cast<UMaterial>(MaterialFactory->FactoryCreateNew(
			UMaterial::StaticClass(),
			MaterialModel,
			TEXT("GeneratedMaterial"),
			RF_DuplicateTransient | RF_TextExportTransient | RF_Public,
			nullptr,
			GWarn
		));
	}
	else
	{
		FString MaterialBaseName = GetName() + "-" + FGuid::NewGuid().ToString();
		const FString FullName = "/Game/DynamicMaterials/" + MaterialBaseName;

		UPackage* Package = LoadObject<UPackage>(GetTransientPackage(), *FullName);

		if (!Package)
		{
			Package = CreatePackage(*FullName);
		}

		UMaterialFactoryNew* MaterialFactory = NewObject<UMaterialFactoryNew>();
		check(MaterialFactory);

		MaterialModel->DynamicMaterial = Cast<UMaterial>(MaterialFactory->FactoryCreateNew(
			UMaterial::StaticClass(),
			Package,
			*MaterialBaseName,
			RF_DuplicateTransient | RF_TextExportTransient | RF_Standalone | RF_Public,
			nullptr,
			GWarn
		));

		FAssetRegistryModule::AssetCreated(MaterialModel->DynamicMaterial);
	}
}

void UDynamicMaterialModelEditorOnlyData::BuildMaterial(bool bInDirtyAssets)
{
	if (State != EDMState::Idle)
	{
		checkNoEntry();
		return;
	}

	if (!IsValid(MaterialModel))
	{
		return;
	}

	UE_LOGF(LogDynamicMaterialEditor, Display, "Building Material Designer Material (%ls)...", *MaterialModel->GetFullName());

	CreateMaterial();

	State = EDMState::Building;
	Expressions.Empty();
	MaterialModel->DynamicMaterial->MaterialDomain = Domain;
	MaterialModel->DynamicMaterial->BlendMode = BlendMode;

	SetMaterialFlags(MaterialModel->DynamicMaterial);

	MaterialModel->DynamicMaterial->OpacityMaskClipValue = OpacityMaskClipValue;
	MaterialModel->DynamicMaterial->DisplacementScaling.Magnitude = DisplacementMagnitude;
	MaterialModel->DynamicMaterial->DisplacementScaling.Center = DisplacementCenter;

	switch (ShadingModel)
	{
		case EDMMaterialShadingModel::DefaultLit:
			MaterialModel->DynamicMaterial->SetShadingModel(EMaterialShadingModel::MSM_DefaultLit);
			break;

		case EDMMaterialShadingModel::Unlit:
			MaterialModel->DynamicMaterial->SetShadingModel(EMaterialShadingModel::MSM_Unlit);
			break;

		default:
			checkNoEntry();
			break;
	}

	TSharedPtr<FDMMaterialBuildState> BuildState = CreateBuildState(MaterialModel->DynamicMaterial, bInDirtyAssets);

	/**
	 * Process slots to build base material inputs.
	 */
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &BuildState](EDMMaterialPropertyType InType)
		{
			if (FExpressionInput* Input = BuildState->GetMaterialProperty(InType))
			{
				Input->Expression = nullptr;
			}

			if (UDMMaterialProperty* Property = GetMaterialProperty(InType))
			{
				if (Property->IsEnabled() && Property->IsMaterialPin() && GetSlotForMaterialProperty(InType)
					&& Property->IsValidForModel(*this))
				{
					Property->GenerateExpressions(BuildState.ToSharedRef());

					// Global opacity is handled at later
					if (InType != EDMMaterialPropertyType::Opacity && InType != EDMMaterialPropertyType::OpacityMask)
					{
						Property->AddAlphaMultiplier(BuildState.ToSharedRef());
					}
				}
			}

			return EDMIterationResult::Continue;
		}
	);

	if (Domain != EMaterialDomain::MD_PostProcess && Domain != EMaterialDomain::MD_LightFunction)
	{
		/**
		 * Generate opacity input based on base/emissive if it doesn't already have an input.
		 */
		EDMMaterialPropertyType GenerateOpacityInput = EDMMaterialPropertyType::None;

		// Masked materials use the mask property.
		if (BlendMode == EBlendMode::BLEND_Masked)
		{
			if (GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::OpacityMask) == nullptr)
			{
				GenerateOpacityInput = EDMMaterialPropertyType::OpacityMask;
			}
		}
		// Any other translucent material will use the opacity property.
		else if (BlendMode != EBlendMode::BLEND_Opaque)
		{
			if (GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::Opacity) == nullptr)
			{
				GenerateOpacityInput = EDMMaterialPropertyType::Opacity;
			}
		}

		if (GenerateOpacityInput != EDMMaterialPropertyType::None)
		{
			UDMMaterialSlot* OpacitySlot = nullptr;
			EDMMaterialPropertyType OpacityProperty = EDMMaterialPropertyType::None;

			if (UDMMaterialSlot* BaseColorSlot = GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::BaseColor))
			{
				OpacitySlot = BaseColorSlot;
				OpacityProperty = EDMMaterialPropertyType::BaseColor;
			}
			else if (UDMMaterialSlot* EmissiveSlot = GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType::EmissiveColor))
			{
				OpacitySlot = EmissiveSlot;
				OpacityProperty = EDMMaterialPropertyType::EmissiveColor;
			}

			if (OpacitySlot)
			{
				UMaterialExpression* OpacityOutputNode;
				int32 OutputIndex;
				int32 OutputChannel;
				UDMMaterialProperty::GenerateOpacityExpressions(BuildState.ToSharedRef(), OpacitySlot, OpacityProperty, OpacityOutputNode, OutputIndex, OutputChannel);

				if (OpacityOutputNode)
				{
					if (FExpressionInput* OpacityPropertyPtr = BuildState->GetMaterialProperty(GenerateOpacityInput))
					{
						OpacityPropertyPtr->Expression = OpacityOutputNode;
						OpacityPropertyPtr->OutputIndex = 0;
						OpacityPropertyPtr->Mask = 0;

						if (OutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
						{
							OpacityPropertyPtr->Mask = 1;
							OpacityPropertyPtr->MaskR = !!(OutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
							OpacityPropertyPtr->MaskG = !!(OutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
							OpacityPropertyPtr->MaskB = !!(OutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
							OpacityPropertyPtr->MaskA = !!(OutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);
						}
					}
				}
			}
		}

		/**
		 * Apply global opacity slider after automatic opacity generation
		 */
		UDMMaterialProperty* OpacityProperty = nullptr;

		if (BlendMode == BLEND_Masked)
		{
			OpacityProperty = GetMaterialProperty(EDMMaterialPropertyType::OpacityMask);
		}
		else if (BlendMode != BLEND_Opaque)
		{
			OpacityProperty = GetMaterialProperty(EDMMaterialPropertyType::Opacity);
		}

		if (OpacityProperty != nullptr && OpacityProperty->IsEnabled())
		{
			OpacityProperty->AddAlphaMultiplier(BuildState.ToSharedRef());
		}
	}

	/**
	 * Apply output processors.
	 */
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &BuildState](EDMMaterialPropertyType InType)
		{
			if (UDMMaterialProperty* Property = GetMaterialProperty(InType))
			{
				if (Property->IsMaterialPin() && Property->IsEnabled() && GetSlotForMaterialProperty(InType)
					&& Property->IsValidForModel(*this))
				{
					Property->AddOutputProcessor(BuildState.ToSharedRef());
				}
			}

			return EDMIterationResult::Continue;
		}
	);

	if (IsValid(MaterialModel) && IsValid(MaterialModel->DynamicMaterialInstance))
	{
		MaterialModel->DynamicMaterialInstance->OnMaterialBuilt(MaterialModel);
	}

	/**
	 * To generate the statistics, you need to force a material recompile. The build state object does this in its destructor.
	 * Resetting the build state SharedPtr destroys the object and thus generates the material shaders.
	 */
	BuildState.Reset();

	/**
	 * GetStatistics can call the GC which could potentially delete the material under us.
	 * Add a reference to it while getting statistics to prevent it being destroyed.
	 */
	{
		TStrongObjectPtr<UObject> ScopeReference(MaterialModel->DynamicMaterial);
		MaterialStats = UMaterialEditingLibrary::GetStatistics(MaterialModel->DynamicMaterial);
	}

	State = EDMState::Idle;
	bBuildRequested = false;

	OnMaterialBuiltDelegate.Broadcast(MaterialModel);
}

void UDynamicMaterialModelEditorOnlyData::SetMaterialFlags(UMaterial* InMaterial)
{
	// Match data type of EMaterialUsage (int)
	for (int Flag = 0; Flag < EMaterialUsage::MATUSAGE_MAX; ++Flag)
	{
		const EMaterialUsage UsageFlag = static_cast<EMaterialUsage>(Flag);
		const EDMMaterialFlagsUsage DMUsageFlag = static_cast<EDMMaterialFlagsUsage>(1 << Flag);

		InMaterial->SetUsageByFlag(UsageFlag, HasUsageFlag(DMUsageFlag));
	}

	// Must have at least 1 usage flag
	if (UsageFlags == 0)
	{
		InMaterial->SetUsageByFlag(EMaterialUsage::MATUSAGE_StaticMesh, true);
	}

	InMaterial->TwoSided = HasGeneralFlag(EDMMaterialFlagsGeneral::TwoSided);
	InMaterial->bTangentSpaceNormal = HasGeneralFlag(EDMMaterialFlagsGeneral::TangentSpaceNormal);
	InMaterial->bUsesDistortion = HasGeneralFlag(EDMMaterialFlagsGeneral::UsesDistortion);
	InMaterial->bNormalCurvatureToRoughness = HasGeneralFlag(EDMMaterialFlagsGeneral::NormalCurvatureToRoughness);
	InMaterial->bFullyRough = HasGeneralFlag(EDMMaterialFlagsGeneral::FullyRough);
	InMaterial->DitheredLODTransition = HasGeneralFlag(EDMMaterialFlagsGeneral::DitheredLODTransition);
	InMaterial->bEnableDisplacementFade = HasGeneralFlag(EDMMaterialFlagsGeneral::EnableDisplacementFade);
	InMaterial->bGenerateSphericalParticleNormals = HasGeneralFlag(EDMMaterialFlagsGeneral::GenerateSphericalParticleNormals);

	InMaterial->bCastDynamicShadowAsMasked = HasLightingFlag(EDMMaterialFlagsLighting::CastDynamicShadowAsMasked);
	InMaterial->bContactShadows = HasLightingFlag(EDMMaterialFlagsLighting::ContactShadows);
	InMaterial->bUseEmissiveForDynamicAreaLighting = HasLightingFlag(EDMMaterialFlagsLighting::UseEmissiveForDynamicAreaLighting);
	InMaterial->bCompatibleWithLumenCardSharing = HasLightingFlag(EDMMaterialFlagsLighting::CompatibleWithLumenCardSharing);
	InMaterial->bAllowTranslucentLocalLightShadow = HasLightingFlag(EDMMaterialFlagsLighting::AllowTranslucentLocalLightShadow);
	InMaterial->bUseLightmapDirectionality = HasLightingFlag(EDMMaterialFlagsLighting::UseLightmapDirectionality);
	InMaterial->bAllowNegativeEmissiveColor = HasLightingFlag(EDMMaterialFlagsLighting::AllowNegativeEmissiveColor);

	InMaterial->bDisableDepthTest = HasTranslucencyFlag(EDMMaterialFlagsTranslucency::DisableDepthTest);
	InMaterial->bScreenSpaceReflections = HasTranslucencyFlag(EDMMaterialFlagsTranslucency::ScreenSpaceReflections);
	InMaterial->AllowTranslucentCustomDepthWrites = HasTranslucencyFlag(EDMMaterialFlagsTranslucency::AllowTranslucentCustomDepthWrites);
	InMaterial->bAllowFrontLayerTranslucency = HasTranslucencyFlag(EDMMaterialFlagsTranslucency::AllowFrontLayerTranslucency);
	InMaterial->bWriteOnlyAlpha = HasTranslucencyFlag(EDMMaterialFlagsTranslucency::WriteOnlyAlpha);
	InMaterial->DitherOpacityMask = HasTranslucencyFlag(EDMMaterialFlagsTranslucency::DitherOpacityMask);

	InMaterial->bHasPixelAnimation = HasMotionFlag(EDMMaterialFlagsMotion::PixelAnimation);
	InMaterial->bEnableResponsiveAA = HasMotionFlag(EDMMaterialFlagsMotion::ResponsiveAA);
	InMaterial->bOutputTranslucentVelocity = HasMotionFlag(EDMMaterialFlagsMotion::OutputTranslucentVelocity);

	InMaterial->bEnableMobileSeparateTranslucency = HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer::MobileSeparateTranslucency);
	InMaterial->bUseAlphaToCoverage = HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer::AlphaToCoverage);
	InMaterial->bForwardRenderUsePreintegratedGFForSimpleIBL = HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer::ForwardRenderUsePreintegratedGFForSimpleIBL);
	InMaterial->bUseHQForwardReflections = HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer::HighQualityForwardReflections);
	InMaterial->bForwardBlendsSkyLightCubemaps = HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer::ForwardBlendsSkyLightCubemaps);
	InMaterial->bUsePlanarForwardReflections = HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer::PlanarForwardReflections);
}

void UDynamicMaterialModelEditorOnlyData::RequestMaterialBuild(EDMBuildRequestType InRequestType)
{
	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();
	UDMBuildRequestSubsystem* BuildRequestSubsystem = UDMBuildRequestSubsystem::Get();

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		if (IsIn(GetTransientPackage()))
		{
			if (MaterialModel)
			{
				MaterialModel->MarkPreviewModified();
			}
		}

		switch (InRequestType)
		{
			case EDMBuildRequestType::Immediate:
				BuildMaterial(/* Dirty Assets */ false);
				break;

			case EDMBuildRequestType::Async:
				if (BuildRequestSubsystem)
				{
					BuildRequestSubsystem->AddBuildRequest(this, /* Dirty Packages */ !UE::GetIsEditorLoadingPackage());
				}
				else
				{
					BuildMaterial(/* Dirty Assets */ false);
				}
				break;

			case EDMBuildRequestType::Preview:
				if (Settings && !Settings->ShouldAutomaticallyCompilePreviewMaterial())
				{
					bBuildRequested = true;
				}
				else if (BuildRequestSubsystem)
				{
					BuildRequestSubsystem->AddBuildRequest(this, /* Dirty Packages */ !UE::GetIsEditorLoadingPackage());
				}
				else
				{
					BuildMaterial(/* Dirty Assets */ false);
				}
				break;
		}
	}
}

void UDynamicMaterialModelEditorOnlyData::OnValueListUpdate()
{
	OnValueListUpdateDelegate.Broadcast(MaterialModel);
}

void UDynamicMaterialModelEditorOnlyData::OnPropertyUpdate(UDMMaterialProperty* InProperty)
{
	RequestMaterialBuild();

	OnPropertyUpdateDelegate.Broadcast(MaterialModel);
}

TSharedRef<FDMMaterialBuildState> UDynamicMaterialModelEditorOnlyData::CreateBuildState(UMaterial* InMaterialToBuild, bool bInDirtyAssets) const
{
	check(MaterialModel);
	check(InMaterialToBuild);

	TSharedRef<FDMMaterialBuildState> BuildState = MakeShared<FDMMaterialBuildState>(InMaterialToBuild, MaterialModel, bInDirtyAssets);

	/**
	 * Add global UV parameters
	 */
	if (UDMMaterialValue* GlobalOffsetValue = MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOffsetValueName))
	{
		GlobalOffsetValue->GenerateExpression(BuildState);
		BuildState->SetGlobalExpression(UDynamicMaterialModel::GlobalOffsetValueName, BuildState->GetLastValueExpression(GlobalOffsetValue));
	}

	if (UDMMaterialValue* GlobalTilingValue = MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalTilingValueName))
	{
		GlobalTilingValue->GenerateExpression(BuildState);
		BuildState->SetGlobalExpression(UDynamicMaterialModel::GlobalTilingValueName, BuildState->GetLastValueExpression(GlobalTilingValue));
	}

	if (UDMMaterialValue* GlobalRotationValue = MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalRotationValueName))
	{
		GlobalRotationValue->GenerateExpression(BuildState);
		BuildState->SetGlobalExpression(UDynamicMaterialModel::GlobalRotationValueName, BuildState->GetLastValueExpression(GlobalRotationValue));
	}

	return BuildState;
}

bool UDynamicMaterialModelEditorOnlyData::NeedsWizard() const
{
	return PropertySlotMap.IsEmpty();
}

void UDynamicMaterialModelEditorOnlyData::OnWizardComplete()
{
	if (UDynamicMaterialModel* MaterialModelLocal = MaterialModel.Get())
	{
		FDynamicMaterialEditorModule::Get().OnWizardComplete(MaterialModelLocal);
	}
}

void UDynamicMaterialModelEditorOnlyData::SetChannelListPreset(FName InPresetName)
{
	const FDMMaterialChannelListPreset* Preset = GetDefault<UDynamicMaterialEditorSettings>()->GetPresetByName(InPresetName);

	if (!Preset)
	{
		return;
	}

	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, Preset](EDMMaterialPropertyType InProperty)
		{
			if (InProperty == EDMMaterialPropertyType::OpacityMask)
			{
				return EDMIterationResult::Continue;
			}

			if (Preset->IsPropertyEnabled(InProperty))
			{
				AddSlotForMaterialProperty(InProperty);
			}
			else
			{
				RemoveSlotForMaterialProperty(InProperty);
			}

			return EDMIterationResult::Continue;
		});

	SetBlendMode(Preset->DefaultBlendMode);
	SetShadingModel(Preset->DefaultShadingModel);
	UsageFlags = Preset->UsageFlag;
	GeneralFlags = Preset->GeneralFlags;
	LightingFlags = Preset->LightingFlags;
	TranslucencyFlags = Preset->TranslucencyFlags;
	MotionFlags = Preset->MotionFlags;
	ForwardRendererFlags = Preset->ForwardRendererFlags;
}

const FMaterialStatistics& UDynamicMaterialModelEditorOnlyData::GetMaterialStats() const
{
	return MaterialStats;
}

UDMMaterialComponent* UDynamicMaterialModelEditorOnlyData::GetSubComponentByPath(FDMComponentPath& InPath,
	const FDMComponentPathSegment& InPathSegment) const
{
	if (InPathSegment.GetToken() == SlotsPathToken)
	{
		int32 SlotIndex;

		if (InPathSegment.GetParameter(SlotIndex))
		{
			if (Slots.IsValidIndex(SlotIndex))
			{
				return Slots[SlotIndex]->GetComponentByPath(InPath);
			}
		}

		return nullptr;
	}

	if (InPathSegment.GetToken() == PropertiesPathToken)
	{
		FString PropertyStr;

		if (InPathSegment.GetParameter(PropertyStr))
		{
			UEnum* PropertyEnum = StaticEnum<EDMMaterialPropertyType>();
			int64 IntValue = PropertyEnum->GetValueByNameString(PropertyStr);

			if (IntValue != INDEX_NONE)
			{
				EDMMaterialPropertyType EnumValue = static_cast<EDMMaterialPropertyType>(IntValue);

				if (UDMMaterialProperty* PropertyPtr = GetMaterialProperty(EnumValue))
				{
					return PropertyPtr->GetComponentByPath(InPath);
				}
			}
		}
	}

	// Channels
	using namespace UE::DynamicMaterialEditor::Private;

	if (const EDMMaterialPropertyType* PropertyPtr = TokenToPropertyMap.Find(FString(InPathSegment.GetToken())))
	{
		if (const TObjectPtr<UDMMaterialSlot>* SlotPtr = PropertySlotMap.Find(*PropertyPtr))
		{
			return (*SlotPtr)->GetComponentByPath(InPath);
		}
	}

	return nullptr;
}

TSharedRef<IDMMaterialBuildStateInterface> UDynamicMaterialModelEditorOnlyData::CreateBuildStateInterface(UMaterial* InMaterialToBuild) const
{
	return CreateBuildState(InMaterialToBuild);
}

void UDynamicMaterialModelEditorOnlyData::SetPropertyComponent(EDMMaterialPropertyType InPropertyType, FName InComponentName, UDMMaterialComponent* InComponent)
{
	if (UDMMaterialProperty* Property = GetMaterialProperty(InPropertyType))
	{
		Property->AddComponent(InComponentName, InComponent);
	}
}

UDMMaterialComponent* UDynamicMaterialModelEditorOnlyData::GetSubComponentByPath(FDMComponentPath& InPath) const
{
	if (InPath.IsLeaf())
	{
		// This is not a component
		return nullptr;
	}

	// Fetches the first component of the path and removes it from the path
	const FDMComponentPathSegment FirstComponent = InPath.GetFirstSegment();

	if (UDMMaterialComponent* SubComponent = GetSubComponentByPath(InPath, FirstComponent))
	{
		return SubComponent->GetComponentByPath(InPath);
	}

	return nullptr;
}

void UDynamicMaterialModelEditorOnlyData::DoBuild_Implementation(bool bInDirtyAssets)
{
	BuildMaterial(bInDirtyAssets);
}

void UDynamicMaterialModelEditorOnlyData::SwapSlotMaterialProperty(EDMMaterialPropertyType InPropertyFrom, EDMMaterialPropertyType InPropertyTo)
{
	UDMMaterialSlot* FromSlot = GetSlotForMaterialProperty(InPropertyFrom);
	
	if (!FromSlot)
	{
		return;
	}

	if (UDMMaterialSlot* ToSlot = GetSlotForMaterialProperty(InPropertyTo))
	{
		if (ToSlot == FromSlot)
		{
			return;
		}

		RemoveSlotForMaterialProperty(InPropertyFrom);
	}

	FromSlot->ChangeMaterialProperty(InPropertyFrom, InPropertyTo);
}

void UDynamicMaterialModelEditorOnlyData::EnsureSwapSlotMaterialProperty(EDMMaterialPropertyType InPropertyFrom, EDMMaterialPropertyType InPropertyTo)
{
	if (UDMMaterialSlot* ToSlot = GetSlotForMaterialProperty(InPropertyTo))
	{
		if (UDMMaterialSlot* FromSlot = GetSlotForMaterialProperty(InPropertyFrom))
		{
			if (ToSlot != FromSlot)
			{
				RemoveSlotForMaterialProperty(InPropertyFrom);
			}
		}
	}
	else if (GetSlotForMaterialProperty(InPropertyFrom))
	{
		SwapSlotMaterialProperty(InPropertyFrom, InPropertyTo);
	}
	else
	{
		AddSlotForMaterialProperty(InPropertyTo);
	}
}

void UDynamicMaterialModelEditorOnlyData::SetDomain(TEnumAsByte<EMaterialDomain> InDomain)
{
	if (Domain == InDomain)
	{
		return;
	}

	Domain = InDomain;

	OnDomainChanged();
}

void UDynamicMaterialModelEditorOnlyData::SetBlendMode(TEnumAsByte<EBlendMode> InBlendMode)
{
	if (BlendMode == InBlendMode)
	{
		return;
	}

	BlendMode = InBlendMode;

	OnBlendModeChanged();
}

void UDynamicMaterialModelEditorOnlyData::SetShadingModel(EDMMaterialShadingModel InShadingModel)
{
	if (ShadingModel == InShadingModel)
	{
		return;
	}

	ShadingModel = InShadingModel;

	OnShadingModelChanged();
}

EDMMaterialFlagsUsage UDynamicMaterialModelEditorOnlyData::GetUsageFlags() const
{
	return static_cast<EDMMaterialFlagsUsage>(UsageFlags);
}

bool UDynamicMaterialModelEditorOnlyData::HasUsageFlag(EDMMaterialFlagsUsage InFlag) const
{
	return (UsageFlags & static_cast<int>(InFlag)) > 0;
}

bool UDynamicMaterialModelEditorOnlyData::SetUsageFlag(EDMMaterialFlagsUsage InFlag, bool bInSet)
{
	if (bInSet == HasUsageFlag(InFlag))
	{
		return false;
	}

	if (bInSet)
	{
		UsageFlags |= static_cast<int>(InFlag);
	}
	else
	{
		UsageFlags &= ~static_cast<int>(InFlag);
	}

	OnMaterialFlagChanged();

	return true;
}

EDMMaterialFlagsGeneral UDynamicMaterialModelEditorOnlyData::GetGeneralFlags() const
{
	return static_cast<EDMMaterialFlagsGeneral>(GeneralFlags);
}

bool UDynamicMaterialModelEditorOnlyData::HasGeneralFlag(EDMMaterialFlagsGeneral InFlag) const
{
	return (GeneralFlags & static_cast<uint8>(InFlag)) > 0;
}

bool UDynamicMaterialModelEditorOnlyData::SetGeneralFlag(EDMMaterialFlagsGeneral InFlag, bool bInSet)
{
	if (bInSet == HasGeneralFlag(InFlag))
	{
		return false;
	}

	if (bInSet)
	{
		GeneralFlags |= static_cast<uint8>(InFlag);
	}
	else
	{
		GeneralFlags &= ~static_cast<uint8>(InFlag);
	}

	OnMaterialFlagChanged();

	return true;
}

EDMMaterialFlagsLighting UDynamicMaterialModelEditorOnlyData::GetLightingFlags() const
{
	return static_cast<EDMMaterialFlagsLighting>(LightingFlags);
}

bool UDynamicMaterialModelEditorOnlyData::HasLightingFlag(EDMMaterialFlagsLighting InFlag) const
{
	return (LightingFlags & static_cast<uint8>(InFlag)) > 0;
}

bool UDynamicMaterialModelEditorOnlyData::SetLightingFlag(EDMMaterialFlagsLighting InFlag, bool bInSet)
{
	if (bInSet == HasLightingFlag(InFlag))
	{
		return false;
	}

	if (bInSet)
	{
		LightingFlags |= static_cast<uint8>(InFlag);
	}
	else
	{
		LightingFlags &= ~static_cast<uint8>(InFlag);
	}

	OnMaterialFlagChanged();

	return true;
}

EDMMaterialFlagsTranslucency UDynamicMaterialModelEditorOnlyData::GetTranslucencyFlags() const
{
	return static_cast<EDMMaterialFlagsTranslucency>(TranslucencyFlags);
}

bool UDynamicMaterialModelEditorOnlyData::HasTranslucencyFlag(EDMMaterialFlagsTranslucency InFlag) const
{
	return (TranslucencyFlags & static_cast<uint8>(InFlag)) > 0;
}

bool UDynamicMaterialModelEditorOnlyData::SetTranslucencyFlag(EDMMaterialFlagsTranslucency InFlag, bool bInSet)
{
	if (bInSet == HasTranslucencyFlag(InFlag))
	{
		return false;
	}

	if (bInSet)
	{
		TranslucencyFlags |= static_cast<uint8>(InFlag);
	}
	else
	{
		TranslucencyFlags &= ~static_cast<uint8>(InFlag);
	}

	OnMaterialFlagChanged();

	return true;
}

EDMMaterialFlagsMotion UDynamicMaterialModelEditorOnlyData::GetMotionFlags() const
{
	return static_cast<EDMMaterialFlagsMotion>(MotionFlags);
}

bool UDynamicMaterialModelEditorOnlyData::HasMotionFlag(EDMMaterialFlagsMotion InFlag) const
{
	return (MotionFlags & static_cast<uint8>(InFlag)) > 0;
}

bool UDynamicMaterialModelEditorOnlyData::SetMotionFlag(EDMMaterialFlagsMotion InFlag, bool bInSet)
{
	if (bInSet == HasMotionFlag(InFlag))
	{
		return false;
	}

	if (bInSet)
	{
		MotionFlags |= static_cast<uint8>(InFlag);
	}
	else
	{
		MotionFlags &= ~static_cast<uint8>(InFlag);
	}

	OnMaterialFlagChanged();

	return true;
}

EDMMaterialFlagsForwardRenderer UDynamicMaterialModelEditorOnlyData::GetForwardRendererFlags() const
{
	return static_cast<EDMMaterialFlagsForwardRenderer>(ForwardRendererFlags);
}

bool UDynamicMaterialModelEditorOnlyData::HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer InFlag) const
{
	return (ForwardRendererFlags & static_cast<uint8>(InFlag)) > 0;
}

bool UDynamicMaterialModelEditorOnlyData::SetForwardRendererFlag(EDMMaterialFlagsForwardRenderer InFlag, bool bInSet)
{
	if (bInSet == HasForwardRendererFlag(InFlag))
	{
		return false;
	}

	if (bInSet)
	{
		ForwardRendererFlags |= static_cast<uint8>(InFlag);
	}
	else
	{
		ForwardRendererFlags &= ~static_cast<uint8>(InFlag);
	}

	OnMaterialFlagChanged();

	return true;
}

float UDynamicMaterialModelEditorOnlyData::GetOpacityMaskClipValue() const
{
	return OpacityMaskClipValue;
}

void UDynamicMaterialModelEditorOnlyData::SetOpacityMaskClipValue(float InClipValue)
{
	if (FMath::IsNearlyEqual(OpacityMaskClipValue, InClipValue))
	{
		return;
	}

	OpacityMaskClipValue = InClipValue;

	OnMaterialSettingsChanged();
}

bool UDynamicMaterialModelEditorOnlyData::GetHasPixelAnimation() const
{
	return HasMotionFlag(EDMMaterialFlagsMotion::PixelAnimation);
}

void UDynamicMaterialModelEditorOnlyData::SetHasPixelAnimation(bool bInHasAnimation)
{
	SetMotionFlag(EDMMaterialFlagsMotion::PixelAnimation, bInHasAnimation);
}

bool UDynamicMaterialModelEditorOnlyData::GetIsTwoSided() const
{
	return HasGeneralFlag(EDMMaterialFlagsGeneral::TwoSided);
}

void UDynamicMaterialModelEditorOnlyData::SetIsTwoSided(bool bInEnabled)
{
	SetGeneralFlag(EDMMaterialFlagsGeneral::TwoSided, bInEnabled);
}

bool UDynamicMaterialModelEditorOnlyData::IsOutputTranslucentVelocityEnabled() const
{
	return HasMotionFlag(EDMMaterialFlagsMotion::OutputTranslucentVelocity);
}

void UDynamicMaterialModelEditorOnlyData::SetOutputTranslucentVelocityEnabled(bool bInEnabled)
{
	SetMotionFlag(EDMMaterialFlagsMotion::OutputTranslucentVelocity, bInEnabled);
}

bool UDynamicMaterialModelEditorOnlyData::IsNaniteTessellationEnabled() const
{
	return HasUsageFlag(EDMMaterialFlagsUsage::DMMFU_Nanite);
}

void UDynamicMaterialModelEditorOnlyData::SetNaniteTessellationEnabled(bool bInEnabled)
{
	SetUsageFlag(EDMMaterialFlagsUsage::DMMFU_Nanite, bInEnabled);
}

bool UDynamicMaterialModelEditorOnlyData::IsResponsiveAAEnabled() const
{
	return HasMotionFlag(EDMMaterialFlagsMotion::ResponsiveAA);
}

void UDynamicMaterialModelEditorOnlyData::SetResponsiveAAEnabled(bool bInEnabled)
{
	SetMotionFlag(EDMMaterialFlagsMotion::ResponsiveAA, bInEnabled);
}

float UDynamicMaterialModelEditorOnlyData::GetDisplacementCenter() const
{
	return DisplacementCenter;
}

void UDynamicMaterialModelEditorOnlyData::SetDisplacementCenter(float InCenter)
{
	if (FMath::IsNearlyEqual(DisplacementCenter, InCenter))
	{
		return;
	}

	DisplacementCenter = InCenter;

	OnMaterialSettingsChanged();
}

float UDynamicMaterialModelEditorOnlyData::GetDisplacementMagnitude() const
{
	return DisplacementMagnitude;
}

void UDynamicMaterialModelEditorOnlyData::SetDisplacementMagnitude(float InMagnitude)
{
	if (FMath::IsNearlyEqual(DisplacementMagnitude, InMagnitude))
	{
		return;
	}

	DisplacementMagnitude = InMagnitude;

	OnMaterialSettingsChanged();
}

void UDynamicMaterialModelEditorOnlyData::OpenMaterialEditor() const
{
	if (!IsValid(MaterialModel) || !IsValid(MaterialModel->DynamicMaterial))
	{
		return;
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	AssetTools.OpenEditorForAssets({MaterialModel->DynamicMaterial});
}

TMap<EDMMaterialPropertyType, UDMMaterialProperty*> UDynamicMaterialModelEditorOnlyData::GetMaterialProperties() const
{
	TMap<EDMMaterialPropertyType, UDMMaterialProperty*> LocalProperties;

	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &LocalProperties](EDMMaterialPropertyType InType)
		{
			if (UDMMaterialProperty* Property = GetMaterialProperty(InType))
			{
				LocalProperties.Emplace(InType, Property);
			}

			return EDMIterationResult::Continue;
		}
	);

	return LocalProperties;
}

UDMMaterialProperty* UDynamicMaterialModelEditorOnlyData::GetMaterialProperty(EDMMaterialPropertyType InMaterialProperty) const
{
	switch (InMaterialProperty)
	{
		case EDMMaterialPropertyType::BaseColor:
			return BaseColor;

		case EDMMaterialPropertyType::EmissiveColor:
			return EmissiveColor;

		case EDMMaterialPropertyType::Opacity:
			return Opacity;

		case EDMMaterialPropertyType::OpacityMask:
			return OpacityMask;

		case EDMMaterialPropertyType::Roughness:
			return Roughness;

		case EDMMaterialPropertyType::Specular:
			return Specular;

		case EDMMaterialPropertyType::Metallic:
			return Metallic;

		case EDMMaterialPropertyType::Normal:
			return Normal;

		case EDMMaterialPropertyType::PixelDepthOffset:
			return PixelDepthOffset;

		case EDMMaterialPropertyType::WorldPositionOffset:
			return WorldPositionOffset;

		case EDMMaterialPropertyType::AmbientOcclusion:
			return AmbientOcclusion;

		case EDMMaterialPropertyType::Anisotropy:
			return Anisotropy;

		case EDMMaterialPropertyType::Refraction:
			return Refraction;

		case EDMMaterialPropertyType::Tangent:
			return Tangent;

		case EDMMaterialPropertyType::Displacement:
			return Displacement;

		case EDMMaterialPropertyType::SubsurfaceColor:
			return SubsurfaceColor;

		case EDMMaterialPropertyType::SurfaceThickness:
			return SurfaceThickness;

		case EDMMaterialPropertyType::Custom1:
			return Custom1;

		case EDMMaterialPropertyType::Custom2:
			return Custom2;

		case EDMMaterialPropertyType::Custom3:
			return Custom3;

		case EDMMaterialPropertyType::Custom4:
			return Custom4;

		default:
			return nullptr;
	}	
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::GetSlot(int32 Index) const
{
	if (Slots.IsValidIndex(Index))
	{
		return Slots[Index];		
	}

	return nullptr;
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::GetSlotForMaterialProperty(EDMMaterialPropertyType InType) const
{
	if (const TObjectPtr<UDMMaterialSlot>* SlotPtr = PropertySlotMap.Find(InType))
	{
		return *SlotPtr;
	}

	return nullptr;
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType InType) const
{
	UDMMaterialProperty* Property = GetMaterialProperty(InType);

	if (!Property || !Property->IsEnabled() || !Property->IsValidForModel(*this))
	{
		return nullptr;
	}

	return GetSlotForMaterialProperty(InType);
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::AddSlot()
{
	UDMMaterialSlot* NewSlot = nullptr;

	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &NewSlot](EDMMaterialPropertyType InProperty)
		{
			if (GetSlotForMaterialProperty(InProperty))
			{
				return EDMIterationResult::Continue;
			}

			NewSlot = AddSlotForMaterialProperty(InProperty);
			return EDMIterationResult::Break;
		});

	return NewSlot;
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::AddSlotForMaterialProperty(EDMMaterialPropertyType InType)
{
	if (InType != EDMMaterialPropertyType::EmissiveColor 
		&& (Domain == EMaterialDomain::MD_PostProcess || Domain == EMaterialDomain::MD_LightFunction))
	{
		return nullptr;
	}
	if (InType == EDMMaterialPropertyType::Opacity || InType == EDMMaterialPropertyType::OpacityMask)
	{
		switch (BlendMode)
		{
			case EBlendMode::BLEND_Translucent:
			case EBlendMode::BLEND_Additive:
			case EBlendMode::BLEND_Modulate:
				InType = EDMMaterialPropertyType::Opacity;
				break;

			case EBlendMode::BLEND_Masked:
				InType = EDMMaterialPropertyType::OpacityMask;
				break;

			case EBlendMode::BLEND_Opaque:
				return nullptr;

			default:
				checkNoEntry();
				break;
		}
	}

	if (UDMMaterialSlot* ExistingSlot = GetSlotForMaterialProperty(InType))
	{
		return ExistingSlot;
	}

	// Opacity and OpacityMask are mutually exclusive so if something tries to add one of them, the other must
	// be checked. If it is found, it is converted and returned.
	switch (InType)
	{
		case EDMMaterialPropertyType::Opacity:
			if (UDMMaterialSlot* ExistingSlot = GetSlotForMaterialProperty(EDMMaterialPropertyType::OpacityMask))
			{
				SwapSlotMaterialProperty(EDMMaterialPropertyType::OpacityMask, EDMMaterialPropertyType::Opacity);
				return ExistingSlot;
			}
			break;

		case EDMMaterialPropertyType::OpacityMask:
			if (UDMMaterialSlot* ExistingSlot = GetSlotForMaterialProperty(EDMMaterialPropertyType::Opacity))
			{
				SwapSlotMaterialProperty(EDMMaterialPropertyType::Opacity, EDMMaterialPropertyType::OpacityMask);
				return ExistingSlot;
			}
			break;
	}

	UDMMaterialSlot* NewSlot = NewObject<UDMMaterialSlot>(this, NAME_None, RF_Transactional);
	check(NewSlot);

	AssignMaterialPropertyToSlot(InType, NewSlot);

	NewSlot->SetIndex(Slots.Num());
	Slots.Add(NewSlot);
	NewSlot->SetComponentState(EDMComponentLifetimeState::Added);

	NewSlot->GetOnConnectorsUpdateDelegate().AddUObject(this, &UDynamicMaterialModelEditorOnlyData::OnSlotConnectorsUpdated);

	if (UDMMaterialProperty* Property = GetMaterialProperty(InType))
	{
		Property->OnSlotAdded(NewSlot);
	}

	NewSlot->Update(NewSlot, EDMUpdateType::Structure);

	OnSlotListUpdateDelegate.Broadcast(MaterialModel);

	RequestMaterialBuild();

	return NewSlot;
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::RemoveSlot(int32 Index)
{
	UDMMaterialSlot* Slot = GetSlot(Index);
	
	if (!Slot)
	{
		return nullptr;
	}
	
	if (GUndo)
	{
		Slot->Modify(/* Always mark dirty */ false);
	}

	for (EDMMaterialPropertyType MaterialProperty : GetMaterialPropertiesForSlot(Slot))
	{
		UnassignMaterialProperty(MaterialProperty);
	}

	const EDMMaterialPropertyType* Key = PropertySlotMap.FindKey(Slot);

	if (Key)
	{
		PropertySlotMap.Remove(*Key);
	}

	const int32 SlotIndex = Slots.IndexOfByKey(Slot);

	if (SlotIndex != INDEX_NONE)
	{
		Slots.RemoveAtSwap(SlotIndex);
		
		if (Slots.IsValidIndex(SlotIndex))
		{
			if (GUndo)
			{
				Slots[SlotIndex]->Modify(/* Always mark dirty */ false);
			}

			Slots[SlotIndex]->SetIndex(SlotIndex);
		}
	}

	Slot->GetOnConnectorsUpdateDelegate().RemoveAll(this);
	Slot->SetComponentState(EDMComponentLifetimeState::Removed);

	RequestMaterialBuild();

	OnSlotListUpdateDelegate.Broadcast(MaterialModel);
	
	return Slot;
}

UDMMaterialSlot* UDynamicMaterialModelEditorOnlyData::RemoveSlotForMaterialProperty(EDMMaterialPropertyType InType)
{
	if (const TObjectPtr<UDMMaterialSlot>* SlotPtr = PropertySlotMap.Find(InType))
	{
		return RemoveSlot(Slots.IndexOfByKey(*SlotPtr));
	}

	return nullptr;
}

TArray<EDMMaterialPropertyType> UDynamicMaterialModelEditorOnlyData::GetMaterialPropertiesForSlot(const UDMMaterialSlot* InSlot) const
{
	TArray<EDMMaterialPropertyType> OutProperties;

	for (const TPair<EDMMaterialPropertyType, TObjectPtr<UDMMaterialSlot>>& Pair : PropertySlotMap)
	{
		if (Pair.Value == InSlot)
		{
			OutProperties.Add(Pair.Key);
		}
	}

	return OutProperties;
}

void UDynamicMaterialModelEditorOnlyData::AssignMaterialPropertyToSlot(EDMMaterialPropertyType InProperty, UDMMaterialSlot* InSlot)
{
	if (!InSlot)
	{
		UnassignMaterialProperty(InProperty);
		return;
	}

	UDMMaterialProperty* Property = GetMaterialProperty(InProperty);
	check(Property);

	PropertySlotMap.FindOrAdd(InProperty) = InSlot;
	Property->ResetInputConnectionMap();
	InSlot->OnPropertiesUpdated();

	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::UnassignMaterialProperty(EDMMaterialPropertyType InProperty)
{
	TObjectPtr<UDMMaterialSlot>* SlotPtr = PropertySlotMap.Find(InProperty);
	
	if (!SlotPtr)
	{
		return;
	}

	PropertySlotMap.Remove(InProperty);
	(*SlotPtr)->OnPropertiesUpdated();

	RequestMaterialBuild();
}

bool UDynamicMaterialModelEditorOnlyData::HasBuildBeenRequested() const
{
	return bBuildRequested;
}

void UDynamicMaterialModelEditorOnlyData::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, class FEditPropertyChain* InPropertyThatChanged)
{
	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::PostLoad()
{
 	Super::PostLoad();

	// Backwards compatibility change - materials were originally parented to this object instead of the model.
	if (IsValid(MaterialModel))
	{
		if (UMaterial* Material = MaterialModel->GetGeneratedMaterial())
		{
			if (Material->GetOuter() != MaterialModel)
			{
				Material->Rename(nullptr, MaterialModel, UE::DynamicMaterial::RenameFlags);
			}
		}
	}

	SetFlags(RF_Transactional);

	AssignPropertyAlphaValues();

	ReinitComponents();
}

void UDynamicMaterialModelEditorOnlyData::PostEditUndo()
{
	Super::PostEditUndo();

	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::PostEditImport()
{
	Super::PostEditImport();

	PostEditorDuplicate();
	ReinitComponents();
	RequestMaterialBuild();
}

void UDynamicMaterialModelEditorOnlyData::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (!bInDuplicateForPIE)
	{
		PostEditorDuplicate();
		ReinitComponents();

		RequestMaterialBuild(
			bInDuplicateForPIE
			? EDMBuildRequestType::Immediate
			: EDMBuildRequestType::Async
		);
	}
}

void UDynamicMaterialModelEditorOnlyData::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	UObject::PostEditChangeChainProperty(InPropertyChangedEvent);

	const FName Property = InPropertyChangedEvent.GetMemberPropertyName();

	 if (Property == GET_MEMBER_NAME_CHECKED(ThisClass, Domain))
	{
		OnDomainChanged();
	}
	else if (Property == GET_MEMBER_NAME_CHECKED(ThisClass, BlendMode))
	{
		OnBlendModeChanged();
	}
	else if (Property == GET_MEMBER_NAME_CHECKED(ThisClass, ShadingModel))
	{
		OnShadingModelChanged();
	}
	else if (Property == GET_MEMBER_NAME_CHECKED(ThisClass, UsageFlags)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, GeneralFlags)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, LightingFlags)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, TranslucencyFlags)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, MotionFlags)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, ForwardRendererFlags))
	{
		OnMaterialFlagChanged();
	}
	else if (Property == GET_MEMBER_NAME_CHECKED(ThisClass, OpacityMaskClipValue)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, DisplacementCenter)
		|| Property == GET_MEMBER_NAME_CHECKED(ThisClass, DisplacementMagnitude))
	{
		 OnMaterialSettingsChanged();
	}
}

void UDynamicMaterialModelEditorOnlyData::Serialize(FArchive& InAr)
{
	InAr.UsingCustomVersion(FDynamicMaterialModelEditorOnlyDataVersion::GUID);

	Super::Serialize(InAr);

	if (!InAr.IsLoading())
	{
		return;
	}

	int32 EODVersion = InAr.CustomVer(FDynamicMaterialModelEditorOnlyDataVersion::GUID);

	if (EODVersion < FDynamicMaterialModelEditorOnlyDataVersion::FlagsAdded)
	{
		ConvertDeprecatedVarsToFlags();
	}
}

void UDynamicMaterialModelEditorOnlyData::OnValueUpdated(UDMMaterialValue* InValue, EDMUpdateType InUpdateType)
{
	check(InValue);

	// Non-exported materials have their values update via settings parameters
	// Exported materials need to be rebuilt to update the main material.
	if (MaterialModel)
	{
		const bool bMaterialInDifferentPackage = MaterialModel->DynamicMaterial ? MaterialModel->DynamicMaterial->GetPackage() != GetPackage() : true;

		if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure) || bMaterialInDifferentPackage)
		{
			RequestMaterialBuild();
		}
	}
}

void UDynamicMaterialModelEditorOnlyData::OnTextureUVUpdated(UDMTextureUV* InTextureUV)
{
	check(InTextureUV);

	// Non-exported materials have their values update via settings parameters
	// Exported materials need to be rebuilt to update the main material.
	if (MaterialModel->DynamicMaterial && MaterialModel->DynamicMaterial->GetPackage() != GetPackage())
	{
		RequestMaterialBuild();
	}
}

void UDynamicMaterialModelEditorOnlyData::SaveEditor()
{
	UEditorLoadingAndSavingUtils::SavePackages({GetPackage()}, false);

	if (IsValid(MaterialModel) && IsValid(MaterialModel->DynamicMaterial))
	{
		if (MaterialModel->DynamicMaterial->GetPackage() != GetPackage())
		{
			UEditorLoadingAndSavingUtils::SavePackages({MaterialModel->DynamicMaterial->GetPackage()}, false);
		}
	}
}

FString UDynamicMaterialModelEditorOnlyData::GetMaterialAssetPath() const
{
	return FPaths::GetPath(GetPackage()->GetPathName());
}

FString UDynamicMaterialModelEditorOnlyData::GetMaterialAssetName() const
{
	return GetName() + "_Mat";
}

FString UDynamicMaterialModelEditorOnlyData::GetMaterialPackageName(const FString& InMaterialBaseName) const
{
	return GetPackage()->GetName() + "_Mat";
}

void UDynamicMaterialModelEditorOnlyData::OnSlotConnectorsUpdated(UDMMaterialSlot* InSlot)
{
	check(InSlot);

	RequestMaterialBuild();
		
	TArray<EDMMaterialPropertyType> SlotProperties = GetMaterialPropertiesForSlot(InSlot);

	for (EDMMaterialPropertyType SlotProperty : SlotProperties)
	{
		if (UDMMaterialProperty* Property = GetMaterialProperty(SlotProperty))
		{
			Property->ResetInputConnectionMap();
		}		
	}
}

void UDynamicMaterialModelEditorOnlyData::ReinitComponents()
{
	for (int32 SlotIdx = 0; SlotIdx < Slots.Num(); ++SlotIdx)
	{
		if (IsValid(Slots[SlotIdx]))
		{
			Slots[SlotIdx]->GetOnConnectorsUpdateDelegate().AddUObject(this, &UDynamicMaterialModelEditorOnlyData::OnSlotConnectorsUpdated);
		}
		else
		{
			Slots.RemoveAt(SlotIdx);
			--SlotIdx;
		}
	}
}

void UDynamicMaterialModelEditorOnlyData::ConvertDeprecatedVarsToFlags()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (bNaniteTessellationEnabled_DEPRECATED)
	{
		UsageFlags |= static_cast<int>(EDMMaterialFlagsUsage::DMMFU_Nanite);
	}

	if (bTwoSided_DEPRECATED)
	{
		GeneralFlags |= static_cast<uint8>(EDMMaterialFlagsGeneral::TwoSided);
	}

	if (bHasPixelAnimation_DEPRECATED)
	{
		MotionFlags |= static_cast<uint8>(EDMMaterialFlagsMotion::PixelAnimation);
	}

	if (bOutputTranslucentVelocityEnabled_DEPRECATED)
	{
		MotionFlags |= static_cast<uint8>(EDMMaterialFlagsMotion::OutputTranslucentVelocity);
	}

	if (bResponsiveAAEnabled_DEPRECATED)
	{
		MotionFlags |= static_cast<uint8>(EDMMaterialFlagsMotion::ResponsiveAA);
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDynamicMaterialModelEditorOnlyData::PostEditorDuplicate()
{
	if (GUndo)
	{
		Modify();
	}

	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this](EDMMaterialPropertyType InType)
		{
			if (UDMMaterialProperty* Property = GetMaterialProperty(InType))
			{
				if (GUndo)
				{
					Property->Modify();
				}

				Property->PostEditorDuplicate(MaterialModel, nullptr);
			}

			return EDMIterationResult::Continue;
		}
	);

	for (UDMMaterialSlot* Slot : Slots)
	{
		if (GUndo)
		{
			Slot->Modify();
		}

		Slot->PostEditorDuplicate(MaterialModel, nullptr);
	}

	PropertySlotMap.Empty();

	for (UDMMaterialSlot* Slot : Slots)
	{
		const TArray<TObjectPtr<UDMMaterialLayerObject>>& SlotLayers = Slot->GetLayers();

		for (const TObjectPtr<UDMMaterialLayerObject>& Layer : SlotLayers)
		{
			EDMMaterialPropertyType Property = Layer->GetMaterialProperty();
			TObjectPtr<UDMMaterialSlot>* SlotPtr = PropertySlotMap.Find(Property);

			if (!SlotPtr)
			{
				PropertySlotMap.Emplace(Property, Slot);
			}
			else
			{
				check(*SlotPtr == Slot);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
