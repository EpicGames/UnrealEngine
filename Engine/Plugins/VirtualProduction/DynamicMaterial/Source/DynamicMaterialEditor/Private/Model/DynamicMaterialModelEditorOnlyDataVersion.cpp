// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/DynamicMaterialModelEditorOnlyDataVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FDynamicMaterialModelEditorOnlyDataVersion::GUID(0xFCF57AFC, 0x50764285, 0xB9A9E660, 0xFFA02D34);
FCustomVersionRegistration GRegisterDynamicMaterialModelEditorOnlyDataVersion(FDynamicMaterialModelEditorOnlyDataVersion::GUID, static_cast<int32>(FDynamicMaterialModelEditorOnlyDataVersion::Type::LatestVersion), TEXT("DynamicMaterialModelEditorOnlyData"));
