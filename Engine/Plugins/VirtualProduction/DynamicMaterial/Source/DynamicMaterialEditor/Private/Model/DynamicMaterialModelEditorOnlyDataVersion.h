// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Guid.h"

struct FDynamicMaterialModelEditorOnlyDataVersion
{
private:
	FDynamicMaterialModelEditorOnlyDataVersion() = delete;

public:
	enum Type : uint8
	{
		PreVersioning = 0,

		/** When the global variables were added and a couple were renamed. */
		GlobalValueRename,

		/** When usage and material flags were added. */
		FlagsAdded,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static const FGuid GUID;
};
