// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Containers/UnrealString.h"
#include "DynamicMaterialModelFactory.generated.h"

UCLASS()
class UDynamicMaterialModelFactory : public UFactory
{
	GENERATED_BODY()

public:		
	static const FString BaseDirectory;
	static const FString BaseName;

	UDynamicMaterialModelFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, 
		FFeedbackContext* InWarn) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
};