// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DragDrop/DMLayerEffectsDragDropOperation.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/WidgetRenderer.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectItem.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"

FDMLayerEffectsDragDropOperation::FDMLayerEffectsDragDropOperation(const TSharedRef<SDMMaterialSlotLayerEffectItem>& InLayerItemWidget, const bool bInShouldDuplicate)
	: LayerItemWidgetWeak(InLayerItemWidget), bShouldDuplicate(bInShouldDuplicate)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	WidgetRenderer = MakeShared<FWidgetRenderer>(true);

	const FVector2D DrawSize = InLayerItemWidget->GetTickSpaceGeometry().GetLocalSize();
	TextureRenderTarget = TStrongObjectPtr<UTextureRenderTarget2D>(WidgetRenderer->DrawWidget(InLayerItemWidget, DrawSize));

	WidgetTextureBrush.SetImageSize(DrawSize);
	WidgetTextureBrush.SetResourceObject(TextureRenderTarget.Get());

	Construct();
}

UDMMaterialEffect* FDMLayerEffectsDragDropOperation::GetMaterialEffect() const
{
	if (TSharedPtr<SDMMaterialSlotLayerEffectItem> LayerItemWidget = GetLayerItemWidget())
	{
		return LayerItemWidget->GetMaterialEffect();
	}

	return nullptr;
}

TSharedPtr<SWidget> FDMLayerEffectsDragDropOperation::GetDefaultDecorator() const
{
	static const FLinearColor InvalidLocationColor = FLinearColor(1.0f, 0.0f, 0.0f, 0.5f);

	return 
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SImage)
			.Image(&WidgetTextureBrush)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SColorBlock)
			.Color(InvalidLocationColor)
			.Visibility_Raw(this, &FDMLayerEffectsDragDropOperation::GetInvalidDropVisibility)
		];
}

FCursorReply FDMLayerEffectsDragDropOperation::OnCursorQuery()
{
	return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
}

EVisibility FDMLayerEffectsDragDropOperation::GetInvalidDropVisibility() const
{
	return bValidDropLocation ? EVisibility::Hidden : EVisibility::SelfHitTestInvisible;
}
