// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"

#include "Styling/SlateBrush.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"

class FWidgetRenderer;
class SDMMaterialSlotLayerEffectItem;
class SWidget;
class UDMMaterialEffect;
class UDMMaterialStage;
class UTextureRenderTarget2D;
struct EVisibility;

class FDMLayerEffectsDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMLayerEffectsDragDropOperation, FDragDropOperation)

	FDMLayerEffectsDragDropOperation(const TSharedRef<SDMMaterialSlotLayerEffectItem>& InLayerItemWidget, const bool bInShouldDuplicate);

	FORCEINLINE bool IsValidDropLocation() { return bValidDropLocation; }
	FORCEINLINE void SetValidDropLocation(const bool bIsValid) { bValidDropLocation = bIsValid; }
	FORCEINLINE void SetToValidDropLocation() { bValidDropLocation = true; }
	FORCEINLINE void SetToInvalidDropLocation() { bValidDropLocation = false; }

	TSharedPtr<SDMMaterialSlotLayerEffectItem> GetLayerItemWidget() const { return LayerItemWidgetWeak.Pin(); }

	UDMMaterialEffect* GetMaterialEffect() const;

	//~ Begin FDragDropOperation
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual FCursorReply OnCursorQuery() override;
	//~ End FDragDropOperation

protected:
	TWeakPtr<SDMMaterialSlotLayerEffectItem> LayerItemWidgetWeak;
	bool bShouldDuplicate;

	bool bValidDropLocation = true;

	TSharedPtr<FWidgetRenderer> WidgetRenderer;

	TStrongObjectPtr<UTextureRenderTarget2D> TextureRenderTarget;
	FSlateBrush WidgetTextureBrush;

	EVisibility GetInvalidDropVisibility() const;
};
