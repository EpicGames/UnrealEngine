// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DragDrop/DMSlotLayerDragDropOperation.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/WidgetRenderer.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"

FDMSlotLayerDragDropOperation::FDMSlotLayerDragDropOperation(const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItemWidget, const bool bInShouldDuplicate)
	: LayerItemWidgetWeak(InLayerItemWidget), bShouldDuplicate(bInShouldDuplicate)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	WidgetRenderer = MakeShared<FWidgetRenderer>(true);

	const FVector2D DrawSize = InLayerItemWidget->GetTickSpaceGeometry().GetLocalSize();
	TextureRenderTarget = TStrongObjectPtr<UTextureRenderTarget2D>(WidgetRenderer->DrawWidget(InLayerItemWidget, DrawSize));

	WidgetTextureBrush.SetImageSize(DrawSize);
	WidgetTextureBrush.SetResourceObject(TextureRenderTarget.Get());

	bCreateNewWindow = false;
	Construct();
}

TSharedPtr<SWidget> FDMSlotLayerDragDropOperation::GetDefaultDecorator() const
{
	static const FLinearColor InvalidLocationColor = FLinearColor(1.0f, 0.0f, 0.0f, 0.05f);

	return 
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SImage)
			.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.1f))
			.Image(&WidgetTextureBrush)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SColorBlock)
			.Color(InvalidLocationColor)
			.Visibility_Raw(this, &FDMSlotLayerDragDropOperation::GetInvalidDropVisibility)
		];
}

FCursorReply FDMSlotLayerDragDropOperation::OnCursorQuery()
{
	return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
}

EVisibility FDMSlotLayerDragDropOperation::GetInvalidDropVisibility() const
{
	return bValidDropLocation ? EVisibility::Hidden : EVisibility::SelfHitTestInvisible;
}

UDMMaterialLayerObject* FDMSlotLayerDragDropOperation::GetLayer() const
{
	if (TSharedPtr<SDMMaterialSlotLayerItem> LayerItemWidget = LayerItemWidgetWeak.Pin())
	{
		return LayerItemWidget->GetLayer();
	}

	return nullptr;
}
