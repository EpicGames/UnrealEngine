// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"

#include "Styling/SlateBrush.h"
#include "Templates/SharedPointer.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UObject/StrongObjectPtr.h"

class FWidgetRenderer;
class SDMMaterialSlotLayerItem;
class SWidget;
class UDMMaterialLayerObject;
class UDMMaterialStage;
class UTextureRenderTarget2D;
struct EVisibility;

class FDMSlotLayerDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMSlotLayerDragDropOperation, FDragDropOperation)

	FDMSlotLayerDragDropOperation(const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItemWidget, const bool bInShouldDuplicate);

	FORCEINLINE TSharedPtr<SDMMaterialSlotLayerItem> GetLayerItemWidget() const { return LayerItemWidgetWeak.Pin(); }
	UDMMaterialLayerObject* GetLayer() const;

	FORCEINLINE bool IsValidDropLocation() { return bValidDropLocation; }
	FORCEINLINE void SetValidDropLocation(const bool bIsValid) { bValidDropLocation = bIsValid; }
	FORCEINLINE void SetToValidDropLocation() { bValidDropLocation = true; }
	FORCEINLINE void SetToInvalidDropLocation() { bValidDropLocation = false; }

	//~ Begin FDragDropOperation
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual FCursorReply OnCursorQuery() override;
	//~ End FDragDropOperation

protected:
	TWeakPtr<SDMMaterialSlotLayerItem> LayerItemWidgetWeak;
	bool bShouldDuplicate;

	bool bValidDropLocation = true;

	TSharedPtr<FWidgetRenderer> WidgetRenderer;

	TStrongObjectPtr<UTextureRenderTarget2D> TextureRenderTarget;
	FSlateBrush WidgetTextureBrush;

	EVisibility GetInvalidDropVisibility() const;
};
