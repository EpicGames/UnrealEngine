// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/DragDrop/DMStageDragDropOperation.h"

#include "Components/DMMaterialStage.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialStage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/SOverlay.h"

FDMStageDragDropOperation::FDMStageDragDropOperation(const TSharedRef<SDMMaterialStage>& InStageWidget)
	: StageWidgetWeak(InStageWidget)
{
	Construct();
}

TSharedPtr<SWidget> FDMStageDragDropOperation::GetDefaultDecorator() const
{
	static const FLinearColor InvalidLocationColor = FLinearColor(1.0f, 0.0f, 0.0f, 0.5f);

	TSharedPtr<SDMMaterialStage> StageWidget = StageWidgetWeak.Pin();
	check(StageWidget.IsValid());

	TSharedPtr<SDMMaterialSlotLayerItem> SlotEditorWidget = StageWidget->GetSlotLayerView();
	check(SlotEditorWidget.IsValid());

	UDMMaterialStage* const Stage = StageWidget->GetStage();
	check(Stage);

	return 
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SDMMaterialStage, SlotEditorWidget.ToSharedRef(), Stage)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SColorBlock)
			.Color(InvalidLocationColor)
			.Visibility_Raw(this, &FDMStageDragDropOperation::GetInvalidDropVisibility)
		];
}

FCursorReply FDMStageDragDropOperation::OnCursorQuery()
{
	return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
}

EVisibility FDMStageDragDropOperation::GetInvalidDropVisibility() const
{
	return bValidDropLocation ? EVisibility::Hidden : EVisibility::SelfHitTestInvisible;
}

UDMMaterialStage* FDMStageDragDropOperation::GetStage() const
{
	if (TSharedPtr<SDMMaterialStage> StageWidget = StageWidgetWeak.Pin())
	{
		return StageWidget->GetStage();
	}

	return nullptr;
}
