// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"

#include "Templates/SharedPointer.h"

class SDMMaterialStage;
class UDMMaterialStage;
struct EVisibility;

class FDMStageDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMStageDragDropOperation, FDragDropOperation)

	FDMStageDragDropOperation(const TSharedRef<SDMMaterialStage>& InStageWidget);

	FORCEINLINE TSharedPtr<SDMMaterialStage> GetStageWidget() const { return StageWidgetWeak.Pin(); }
	UDMMaterialStage* GetStage() const;

	FORCEINLINE bool IsValidDropLocation() { return bValidDropLocation; }
	FORCEINLINE void SetValidDropLocation(const bool bIsValid) { bValidDropLocation = bIsValid; }
	FORCEINLINE void SetToValidDropLocation() { bValidDropLocation = true; }
	FORCEINLINE void SetToInvalidDropLocation() { bValidDropLocation = false; }

	//~ Begin FDragDropOperation
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual FCursorReply OnCursorQuery() override;
	//~ End FDragDropOperation

protected:
	TWeakPtr<SDMMaterialStage> StageWidgetWeak;

	bool bValidDropLocation = true;

	EVisibility GetInvalidDropVisibility() const;
};
