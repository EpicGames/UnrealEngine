// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Components/DMMaterialLayer.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

#include "DMMaterialSlotLayerAddEffectContext.generated.h"

class SDMMaterialEditor;
class UDMMaterialLayerObject;
class UToolMenu;

UCLASS(MinimalAPI)
class UDMMaterialSlotLayerAddEffectContext : public UObject
{
	GENERATED_BODY()

public:
	void SetEditorWidget(const TSharedPtr<SDMMaterialEditor>& InEditor) { EditorWidgetWeak = InEditor; }

	TSharedPtr<SDMMaterialEditor> GetEditorWidget() const { return EditorWidgetWeak.Pin(); }

	void SetLayer(UDMMaterialLayerObject* InLayer) { LayerWeak = InLayer; }

	UDMMaterialLayerObject* GetLayer() const { return LayerWeak.Get(); }

private:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<UDMMaterialLayerObject> LayerWeak;
};
