// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMMaterialSlotLayerAddEffectMenus.h"

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialEffectFunction.h"
#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "DynamicMaterialEditorSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Materials/MaterialFunctionInterface.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMaterialSlotLayerAddEffectContext.h"
#include "UI/Menus/DMMenuContext.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectView.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMMaterialEffectStackPresetSubsystem.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "DMMaterialSlotLayerAddEffectMenus"

namespace UE::DynamicMaterialEditor::Private
{
	static const FName AddEffectMenuName(TEXT("MaterialDesigner.Slot.Layer.AddEffect"));
	static const FName AddEffectMenuSection(TEXT("AddEffect"));
}

TSharedRef<SWidget> FDMMaterialSlotLayerAddEffectMenus::OpenAddEffectMenu(const TSharedPtr<SDMMaterialEditor>& InEditor, UDMMaterialLayerObject* InLayer)
{
	using namespace UE::DynamicMaterialEditor::Private;

	RegisterAddEffectMenu();

	UDMMaterialSlotLayerAddEffectContext* ContextObject = NewObject<UDMMaterialSlotLayerAddEffectContext>();
	ContextObject->SetEditorWidget(InEditor);
	ContextObject->SetLayer(InLayer);

	return UToolMenus::Get()->GenerateWidget(AddEffectMenuName, FToolMenuContext(ContextObject));
}

void FDMMaterialSlotLayerAddEffectMenus::AddEffectSubMenu(UToolMenu* InMenu, UDMMaterialLayerObject* InLayer)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (InMenu->ContainsSection(AddEffectMenuSection))
	{
		return;
	}

	FToolMenuSection& NewSection = InMenu->AddSection(
		AddEffectMenuSection,
		LOCTEXT("Effects", "Effects")
	);

	NewSection.AddSubMenu(
		NAME_None,
		LOCTEXT("AddEffect", "Add Effect"),
		FText::GetEmpty(),
		FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&GenerateAddEffectMenu))
	);
}

bool FDMMaterialSlotLayerAddEffectMenus::CanAddEffect(const FToolMenuContext& InContext, TSoftObjectPtr<UMaterialFunctionInterface> InMaterialFunctionPtr)
{
	UDMMaterialLayerObject* Layer = nullptr;

	if (UDMMenuContext* MenuContext = InContext.FindContext<UDMMenuContext>())
	{
		Layer = MenuContext->GetLayer();
	}
	else if (UDMMaterialSlotLayerAddEffectContext* SlotContext = InContext.FindContext<UDMMaterialSlotLayerAddEffectContext>())
	{
		Layer = SlotContext->GetLayer();
	}

	if (!Layer || !IsValid(Layer))
	{
		return false;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack)
	{
		return false;
	}

	UMaterialFunctionInterface* MaterialFunction = InMaterialFunctionPtr.LoadSynchronous();

	if (!MaterialFunction)
	{
		return false;
	}

	for (UDMMaterialEffect* Effect : EffectStack->GetEffects())
	{
		if (UDMMaterialEffectFunction* EffectFunction = Cast<UDMMaterialEffectFunction>(Effect))
		{
			if (EffectFunction->GetMaterialFunction() == MaterialFunction)
			{
				return false;
			}
		}
	}

	return true;
}

void FDMMaterialSlotLayerAddEffectMenus::AddEffect(const FToolMenuContext& InContext, TSoftObjectPtr<UMaterialFunctionInterface> InMaterialFunctionPtr)
{
	UDMMaterialLayerObject* Layer = nullptr;
	TSharedPtr<SDMMaterialEditor> EditorWidget;

	if (UDMMenuContext* MenuContext = InContext.FindContext<UDMMenuContext>())
	{
		Layer = MenuContext->GetLayer();
		EditorWidget = MenuContext->GetEditorWidget();
	}
	else if (UDMMaterialSlotLayerAddEffectContext* SlotContext = InContext.FindContext<UDMMaterialSlotLayerAddEffectContext>())
	{
		Layer = SlotContext->GetLayer();
		EditorWidget = SlotContext->GetEditorWidget();
	}

	if (!Layer || !IsValid(Layer))
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	UMaterialFunctionInterface* MaterialFunction = InMaterialFunctionPtr.LoadSynchronous();

	if (!MaterialFunction)
	{
		return;
	}

	UDMMaterialEffectFunction* EffectFunction = UDMMaterialEffect::CreateEffect<UDMMaterialEffectFunction>(EffectStack);
	EffectFunction->SetMaterialFunction(MaterialFunction);

	// Some error applying it
	if (EffectFunction->GetMaterialFunction() != MaterialFunction)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("AddEffectTransaction", "Add Effect"));
	EffectStack->Modify();

	EffectStack->AddEffect(EffectFunction);

	if (!EditorWidget.IsValid())
	{
		return;
	}

	EditorWidget->EditComponent(EffectFunction);

	if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = EditorWidget->GetSlotEditorWidget())
	{
		if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = SlotEditorWidget->GetLayerView())
		{
			if (TSharedPtr<SDMMaterialSlotLayerItem> LayerWidget = LayerView->GetWidgetForLayer(Layer))
			{
				LayerWidget->SetEffectsExpanded(true);

				if (TSharedPtr<SDMMaterialSlotLayerEffectView> EffectsView = LayerWidget->GetEffectView())
				{
					EffectsView->SetSelectedEffect(EffectFunction);
				}
			}
		}
	}
}

void FDMMaterialSlotLayerAddEffectMenus::GenerateAddEffectSubMenu(UToolMenu* InMenu, int32 InCategoryIndex)
{
	if (!InMenu)
	{
		return;
	}

	const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>();

	if (!Settings)
	{
		return;
	}

	TArray<FDMMaterialEffectList> EffectLists = Settings->GetEffectList();

	if (!EffectLists.IsValidIndex(InCategoryIndex))
	{
		return;
	}

	EffectLists[InCategoryIndex].Effects.StableSort([](const TSoftObjectPtr<UMaterialFunctionInterface>& InA, const TSoftObjectPtr<UMaterialFunctionInterface>& InB)
		{
			UMaterialFunctionInterface* MaterialFunctionA = InA.LoadSynchronous();
			UMaterialFunctionInterface* MaterialFunctionB = InB.LoadSynchronous();

			if (!MaterialFunctionA)
			{
				return false;
			}

			if (!MaterialFunctionB)
			{
				return true;
			}

			const FString MaterialFunctionNameA = MaterialFunctionA->GetUserExposedCaption();
			const FString MaterialFunctionNameB = MaterialFunctionB->GetUserExposedCaption();
			return MaterialFunctionNameA < MaterialFunctionNameB;
		});

	FToolMenuSection& Section = InMenu->AddSection(TEXT("EffectList"), LOCTEXT("EffectList", "Effect List"));

	for (const TSoftObjectPtr<UMaterialFunctionInterface>& Effect : EffectLists[InCategoryIndex].Effects)
	{
		UMaterialFunctionInterface* MaterialFunction = Effect.LoadSynchronous();

		if (!MaterialFunction)
		{
			continue;
		}

		FToolUIAction Action;
		Action.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&AddEffect, Effect);
		Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanAddEffect, Effect);

		const FString& Description = MaterialFunction->GetUserExposedCaption();
		const FString ToolTip = MaterialFunction->GetDescription();

		Section.AddMenuEntry(
			FName(*Description),
			FText::FromString(Description),
			FText::FromString(ToolTip),
			FSlateIcon(),
			FToolUIActionChoice(Action)
		);
	}
}

void FDMMaterialSlotLayerAddEffectMenus::GenerateAddEffectMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>();

	if (!Settings)
	{
		return;
	}

	FToolMenuSection& Section = InMenu->AddSection(TEXT("AddEffect"), LOCTEXT("AddEffect", "Add Effect"));

	TArray<FDMMaterialEffectList> EffectLists = Settings->GetEffectList();

	for (int32 CategoryIndex = 0; CategoryIndex < EffectLists.Num(); ++CategoryIndex)
	{
		Section.AddSubMenu(
			NAME_None,
			FText::FromString(EffectLists[CategoryIndex].Name),
			FText::GetEmpty(),
			FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&GenerateAddEffectSubMenu, CategoryIndex))
		);
	}
}

void FDMMaterialSlotLayerAddEffectMenus::SavePreset(const FToolMenuContext& InContext, const FString& InPresetName)
{
	UDMMaterialLayerObject* Layer = nullptr;

	if (UDMMenuContext* MenuContext = InContext.FindContext<UDMMenuContext>())
	{
		Layer = MenuContext->GetLayer();
	}
	else if (UDMMaterialSlotLayerAddEffectContext* SlotContext = InContext.FindContext<UDMMaterialSlotLayerAddEffectContext>())
	{
		Layer = SlotContext->GetLayer();
	}

	if (!Layer || !IsValid(Layer))
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	FDMMaterialEffectStackJson Preset = EffectStack->CreatePreset();

	UDMMaterialEffectStackPresetSubsystem::Get()->SavePreset(InPresetName, Preset);
}

bool FDMMaterialSlotLayerAddEffectMenus::VerifyFileName(const FText& InValue, FText& OutErrorText)
{
	static const FText TooShortError = LOCTEXT("TooShortError", "Min 3 characters.");
	static const FText TooLongError = LOCTEXT("TooLongError", "Max 50 characters.");
	static const FText InvalidCharacterError = LOCTEXT("InvalidCharacterError", "Valid characters are A-Z, A-z, space, _ and -");

	static const TArray<TPair<TCHAR, TCHAR>> ValidCharacterRanges = {
		{'A', 'Z'},
		{'a', 'z'},
		{'0', '9'},
		{' ', ' '},
		{'-', '-'},
		{'_', '_'}
	};

	const FString ValueStr = InValue.ToString();
	const int32 ValueLen = ValueStr.Len();

	if (ValueLen < 3)
	{
		OutErrorText = TooShortError;
		return false;
	}

	if (ValueLen > 50)
	{
		OutErrorText = TooLongError;
		return false;
	}

	for (int32 Index = 0; Index < ValueLen; ++Index)
	{
		bool bInRange = false;

		for (const TPair<TCHAR, TCHAR>& ValidCharacterRange : ValidCharacterRanges)
		{
			if (ValueStr[Index] >= ValidCharacterRange.Key && ValueStr[Index] <= ValidCharacterRange.Value)
			{
				bInRange = true;
				break;
			}
		}

		if (!bInRange)
		{
			OutErrorText = InvalidCharacterError;
			return false;
		}
	}

	return true;
}

void FDMMaterialSlotLayerAddEffectMenus::GenerateSaveEffectsMenu(FToolMenuSection& InSection)
{
	FNewToolMenuChoice RenameChoice(FNewToolMenuWidget::CreateLambda(
		[](const FToolMenuContext& InContext) -> TSharedRef<SWidget>
		{
			UDMMaterialLayerObject* Layer = nullptr;

			if (UDMMenuContext* MenuContext = InContext.FindContext<UDMMenuContext>())
			{
				Layer = MenuContext->GetLayer();
			}
			else if (UDMMaterialSlotLayerAddEffectContext* SlotContext = InContext.FindContext<UDMMaterialSlotLayerAddEffectContext>())
			{
				Layer = SlotContext->GetLayer();
			}

			if (!Layer || !IsValid(Layer))
			{
				return SNullWidget::NullWidget;
			}

			UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

			if (!EffectStack)
			{
				return SNullWidget::NullWidget;
			}

			TWeakObjectPtr<UDMMaterialEffectStack> EffectStackWeak = EffectStack;

			return SNew(SEditableTextBox)
				.Text(LOCTEXT("NewPreset", "New Preset"))
				.OnVerifyTextChanged_Static(&VerifyFileName)
				.AllowContextMenu(false)
				.ClearKeyboardFocusOnCommit(true)
				.MinDesiredWidth(100.f)
				.OnTextCommitted_Lambda(
					[EffectStackWeak](const FText& InText, ETextCommit::Type InCommitType)
					{
						if (InCommitType != ETextCommit::OnEnter)
						{
							return;
						}

						UDMMaterialEffectStack* EffectStack = EffectStackWeak.Get();

						if (!IsValid(EffectStack))
						{
							return;
						}

						FText Unused;

						if (!VerifyFileName(InText, Unused))
						{
							return;
						}

						FDMMaterialEffectStackJson Preset = EffectStack->CreatePreset();

						if (UDMMaterialEffectStackPresetSubsystem::Get()->SavePreset(InText.ToString(), Preset))
						{
							FNotificationInfo Info(LOCTEXT("PresetSaved", "Preset saved!"));
							Info.ExpireDuration = 5.0f;
							FSlateNotificationManager::Get().AddNotification(Info);
						}
						else
						{
							FNotificationInfo Info(LOCTEXT("PresetNotSaved", "Failed to save preset!"));
							Info.ExpireDuration = 5.0f;
							FSlateNotificationManager::Get().AddNotification(Info);
						}
					});
		}));

	InSection.AddSubMenu(
		TEXT("SavePreset"),
		LOCTEXT("SavePreset", "Save Preset"),
		TAttribute<FText>(),
		RenameChoice
	);
}

void FDMMaterialSlotLayerAddEffectMenus::LoadPreset(const FToolMenuContext& InContext, FString InPresetName)
{
	UDMMaterialLayerObject* Layer = nullptr;

	if (UDMMenuContext* MenuContext = InContext.FindContext<UDMMenuContext>())
	{
		Layer = MenuContext->GetLayer();
	}
	else if (UDMMaterialSlotLayerAddEffectContext* SlotContext = InContext.FindContext<UDMMaterialSlotLayerAddEffectContext>())
	{
		Layer = SlotContext->GetLayer();
	}

	if (!Layer || !IsValid(Layer))
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	FDMMaterialEffectStackJson Preset;

	if (UDMMaterialEffectStackPresetSubsystem::Get()->LoadPreset(InPresetName, Preset))
	{
		EffectStack->ApplyPreset(Preset);

		FNotificationInfo Info(LOCTEXT("PresetApplied", "Preset applied!"));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
	else
	{
		FNotificationInfo Info(LOCTEXT("PresetNotApplied", "Failed to apply preset!"));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FDMMaterialSlotLayerAddEffectMenus::GenerateLoadEffectsMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const UDMMaterialEffectStackPresetSubsystem* PresetSubsystem = UDMMaterialEffectStackPresetSubsystem::Get();

	if (!PresetSubsystem)
	{
		return;
	}

	FToolMenuSection& Section = InMenu->AddSection("LoadPreset", LOCTEXT("LoadPreset", "Load Preset"));

	TArray<FString> PresetList = PresetSubsystem->GetPresetNames();

	for (int32 PresetIndex = 0; PresetIndex < PresetList.Num(); ++PresetIndex)
	{
		FToolUIAction LoadAction;
		LoadAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&LoadPreset, PresetList[PresetIndex]);

		Section.AddMenuEntry(
			*PresetList[PresetIndex],
			FText::FromString(PresetList[PresetIndex]),
			TAttribute<FText>(),
			FSlateIcon(),
			FToolUIActionChoice(LoadAction)
		);
	}
}

void FDMMaterialSlotLayerAddEffectMenus::GenerateEffectPresetMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>();

	if (!Settings)
	{
		return;
	}

	FToolMenuSection& Section = InMenu->AddSection(TEXT("Presets"), LOCTEXT("Presets", "Presets"));

	GenerateSaveEffectsMenu(Section);

	Section.AddSubMenu(
		TEXT("LoadPreset"),
		LOCTEXT("LoadPreset", "Load Preset"),
		TAttribute<FText>(),
		FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&GenerateLoadEffectsMenu))
	);
}

void FDMMaterialSlotLayerAddEffectMenus::RegisterAddEffectMenu()
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (UToolMenus::Get()->IsMenuRegistered(AddEffectMenuName))
	{
		return;
	}

	UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(AddEffectMenuName);

	Menu->AddDynamicSection(TEXT("AddEffectSection"), FNewToolMenuDelegate::CreateLambda(
		[](UToolMenu* InMenu)
		{
			GenerateAddEffectMenu(InMenu);
			GenerateEffectPresetMenu(InMenu);
		}));
}

#undef LOCTEXT_NAMESPACE
