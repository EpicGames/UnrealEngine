// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class FString;
class FText;
class SDMMaterialEditor;
class SWidget;
class UDMMaterialLayerObject;
class UMaterialFunctionInterface;
class UToolMenu;
struct FToolMenuContext;
struct FToolMenuSection;

class FDMMaterialSlotLayerAddEffectMenus final
{
public:
	static TSharedRef<SWidget> OpenAddEffectMenu(const TSharedPtr<SDMMaterialEditor>& InEditor, UDMMaterialLayerObject* InLayer);

private:
	static void GenerateAddEffectSubMenu(UToolMenu* InMenu, int32 InCategoryIndex);

	static void GenerateAddEffectMenu(UToolMenu* InMenu);

	static void GenerateSaveEffectsMenu(FToolMenuSection& InSection);

	static void GenerateLoadEffectsMenu(UToolMenu* InMenu);

	static void GenerateEffectPresetMenu(UToolMenu* InMenu);

	static void RegisterAddEffectMenu();

	static void AddEffectSubMenu(UToolMenu* InMenu, UDMMaterialLayerObject* InLayer);

	static bool CanAddEffect(const FToolMenuContext& InContext, TSoftObjectPtr<UMaterialFunctionInterface> InMaterialFunctionPtr);

	static void AddEffect(const FToolMenuContext& InContext, TSoftObjectPtr<UMaterialFunctionInterface> InMaterialFunctionPtr);

	static void SavePreset(const FToolMenuContext& InContext, const FString& InPresetName);

	static bool VerifyFileName(const FText& InValue, FText& OutErrorText);

	static void LoadPreset(const FToolMenuContext& InContext, FString InPresetName);
};
