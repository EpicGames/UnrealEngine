// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMMaterialSlotLayerMenus.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageExpression.h"
#include "Components/DMMaterialStageFunction.h"
#include "Components/DMMaterialStageGradient.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialStageExpressions/DMMSESceneTexture.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageExpressions/DMMSETextureSampleEdgeColor.h"
#include "Components/MaterialStageExpressions/DMMSEWorldPositionNoise.h"
#include "Components/MaterialValues/DMMaterialValueColorAtlas.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/RenderTargetRenderers/DMRenderTargetTextRenderer.h"
#include "Components/RenderTargetRenderers/DMRenderTargetUMGWidgetRenderer.h"
#include "DMDefs.h"
#include "DMMaterialSlotLayerAddEffectMenus.h"
#include "DMValueDefinition.h"
#include "DynamicMaterialEditorCommands.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialEditorStyle.h"
#include "Framework/Commands/GenericCommands.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UI/Menus/DMMenuContext.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMMaterialSlotFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "FDMMaterialSlotLayerMenus"

namespace UE::DynamicMaterialEditor::Private
{
	static const FName SlotLayerMenuName = TEXT("MaterialDesigner.MaterialSlot.Layer");
	static const FName SlotLayerAddMenuName = TEXT("MaterialDesigner.MaterialSlot.AddLayer");
	static const FName SlotLayerAddSectionName = TEXT("AddLayer");
	static const FName SlotLayerModifySectionName = TEXT("ModifyLayer");
	static const FName GlobalValuesSectionName = TEXT("GlobalValues");
}

TSharedRef<SWidget> FDMMaterialSlotLayerMenus::GenerateSlotLayerMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget, UDMMaterialLayerObject* InLayer)
{
	return InLayer
		? GenerateSlotLayerMenu_Layer(InSlotWidget, InLayer)
		: GenerateSlotLayerMenu_AddLayer(InSlotWidget);
}

TSharedRef<SWidget> FDMMaterialSlotLayerMenus::GenerateSlotLayerMenu_AddLayer(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!UToolMenus::Get()->IsMenuRegistered(SlotLayerAddMenuName))
	{
		UToolMenu* NewToolMenu = UDMMenuContext::GenerateContextMenuDefault(SlotLayerAddMenuName);

		if (!NewToolMenu)
		{
			return SNullWidget::NullWidget;
		}

		FToolMenuSection& AddLayerSection = NewToolMenu->FindOrAddSection(SlotLayerAddSectionName, LOCTEXT("AddLayer", "Add Layer"));
		AddLayerSection.AddDynamicEntry(SlotLayerAddSectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialSlotLayerMenus::AddAddLayerSection));
		AddLayerSection.InsertPosition.Position = EToolMenuInsertType::First;

		if constexpr (UE::DynamicMaterialEditor::bGlobalValuesEnabled)
		{
			FToolMenuSection& GlobalValueSection = NewToolMenu->FindOrAddSection(GlobalValuesSectionName, LOCTEXT("GlobalValue", "Add Global Value Layer"));
			GlobalValueSection.AddDynamicEntry(GlobalValuesSectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialSlotLayerMenus::AddGlobalValueSection));
		}
	}

	FToolMenuContext MenuContext(UDMMenuContext::CreateLayer(InSlotWidget->GetEditorWidget(), nullptr));

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = InSlotWidget->GetEditorWidget())
	{
		MenuContext.AppendCommandList(EditorWidget->GetCommandList());
	}

	return UToolMenus::Get()->GenerateWidget(SlotLayerAddMenuName, MenuContext);
}

TSharedRef<SWidget> FDMMaterialSlotLayerMenus::GenerateSlotLayerMenu_Layer(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget,
	UDMMaterialLayerObject* InLayer)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!UToolMenus::Get()->IsMenuRegistered(SlotLayerMenuName))
	{
		UToolMenu* NewToolMenu = UDMMenuContext::GenerateContextMenuDefault(SlotLayerMenuName);

		if (!NewToolMenu)
		{
			return SNullWidget::NullWidget;
		}

		FToolMenuSection& AddLayerSection = NewToolMenu->FindOrAddSection(SlotLayerAddSectionName, LOCTEXT("AddLayer", "Add Layer"));
		AddLayerSection.AddDynamicEntry(SlotLayerAddSectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialSlotLayerMenus::AddAddLayerSection));
		AddLayerSection.InsertPosition.Position = EToolMenuInsertType::First;

		if constexpr (UE::DynamicMaterialEditor::bGlobalValuesEnabled)
		{
			FToolMenuSection& GlobalValueSection = NewToolMenu->FindOrAddSection(GlobalValuesSectionName, LOCTEXT("GlobalValue", "Add Global Value Layer"));
			GlobalValueSection.AddDynamicEntry(GlobalValuesSectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialSlotLayerMenus::AddGlobalValueSection));
		}

		FToolMenuSection& ModifyLayerSection = NewToolMenu->FindOrAddSection(SlotLayerModifySectionName, LOCTEXT("ModifyLayer", "Modify Layer"));
		ModifyLayerSection.AddDynamicEntry(SlotLayerModifySectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialSlotLayerMenus::AddLayerModifySection));
		ModifyLayerSection.InsertPosition.Position = EToolMenuInsertType::Last;
	}

	FToolMenuContext MenuContext(UDMMenuContext::CreateLayer(InSlotWidget->GetEditorWidget(), InLayer));

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = InSlotWidget->GetEditorWidget())
	{
		MenuContext.AppendCommandList(EditorWidget->GetCommandList());
	}

	return UToolMenus::Get()->GenerateWidget(SlotLayerMenuName, MenuContext);
}

void FDMMaterialSlotLayerMenus::AddAddLayerSection(FToolMenuSection& InSection)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDMMenuContext* MenuContext = InSection.FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = EditorWidget->GetSlotEditorWidget()->GetSlot();

	if (!Slot)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

	if (!ModelEditorOnlyData)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	InSection.AddMenuEntry(
		TEXT("Texture"),
		LOCTEXT("AddTextureSample", "Texture"),
		LOCTEXT("AddTextureSampleTooltip", "Add a Material Stage based on a Texture."),
		GetDefault<UDMMaterialStageExpressionTextureSample>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot, ExpressionClass = TSubclassOf<UDMMaterialStageExpression>(UDMMaterialStageExpressionTextureSample::StaticClass())]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_Expression(Slot, ExpressionClass);
			}
		))
	);

	InSection.AddMenuEntry(
		TEXT("SolidColor"),
		LOCTEXT("AddColor", "Solid Color"),
		LOCTEXT("AddColorTooltip", "Add a new Material Layer with a solid RGB color."),
		GetDefault<UDMMaterialValueFloat3RGB>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_NewLocalValue(Slot, EDMValueType::VT_Float3_RGB);
			}
				))
	);

	InSection.AddMenuEntry(
		TEXT("TextureEdgeColor"),
		LOCTEXT("AddEdgeColor", "Texture Edge Color"),
		LOCTEXT("AddEdgeColorTooltip", "Add a new Material Layer with a solid color based on the edge color on a texture."),
		GetDefault<UDMMaterialStageExpressionTextureSampleEdgeColor>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_Expression(Slot, TSubclassOf<UDMMaterialStageExpression>(UDMMaterialStageExpressionTextureSampleEdgeColor::StaticClass()));
			}
				))
	);

	if (ModelEditorOnlyData->GetDomain() == EMaterialDomain::MD_PostProcess)
	{
		InSection.AddMenuEntry(
			TEXT("PostProcess"),
			LOCTEXT("AddSceneTexture", "Post Process"),
			LOCTEXT("AddSceneTextureTooltip", "Add a new Material Layer that represents the Scene Texture for a post process material."),
			GetDefault<UDMMaterialStageExpressionSceneTexture>()->GetComponentIcon(),
			FUIAction(FExecuteAction::CreateWeakLambda(
				Slot,
				[Slot]
				{
					UDMMaterialSlotFunctionLibrary::AddNewLayer_SceneTexture(Slot);
				}
					))
		);
	}

	InSection.AddMenuEntry(
		TEXT("Noise"),
		LOCTEXT("AddNoise", "Noise"),
		LOCTEXT("AddNoiseTooltip", "Add a new Material Layer with a noise pattern."),
		GetDefault<UDMMaterialStageExpressionWorldPositionNoise>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_Expression(Slot, TSubclassOf<UDMMaterialStageExpression>(UDMMaterialStageExpressionWorldPositionNoise::StaticClass()));
			}
		))
	);

	const TArray<TStrongObjectPtr<UClass>>& Gradients = UDMMaterialStageGradient::GetAvailableGradients();

	if (!Gradients.IsEmpty())
	{
		InSection.AddSubMenu(
			TEXT("GradientMenu"),
			LOCTEXT("AddGradientStage", "Gradient"),
			LOCTEXT("AddGradientStageTooltip", "Add a Material Stage based on a Material Gradient."),
			FNewToolMenuDelegate::CreateStatic(&AddLayerMenu_Gradients)
		);
	}

	if constexpr (UE::DynamicMaterialEditor::bGlobalValuesEnabled)
	{
		AddGlobalValueSection(InSection);
	}

	InSection.AddSubMenu(
		TEXT("AdvancedMenu"),
		LOCTEXT("AddAdvancedStage", "Advanced"),
		LOCTEXT("AddAdvancedStageTooltip", "Add an advanced Material Stage."),
		FNewToolMenuDelegate::CreateStatic(&AddLayerMenu_Advanced)
	);
}

void FDMMaterialSlotLayerMenus::AddLayerModifySection(FToolMenuSection& InSection)
{
	using namespace UE::DynamicMaterialEditor::Private;

	const UDMMenuContext* const MenuContext = InSection.FindContext<UDMMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = MenuContext->GetLayer();
	if (!Layer)
	{
		return;
	}

	UDMMaterialSlot* Slot = Layer->GetSlot();
	if (!Slot)
	{
		return;
	}

	if (Slot->CanRemoveLayer(Layer))
	{
		FSlateIcon ToggleLayerIcon = Layer->IsEnabled()
			? FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Kismet.VariableList.HideForInstance"))
			: FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Kismet.VariableList.ExposeForInstance"));

		InSection.AddMenuEntry(
			TEXT("ToggleLayer"),
			LOCTEXT("ToggleLayer", "Toggle Layer"),
			LOCTEXT("ToggleLayerTooltip", "Toggle the Layer.\n\nAlt+Left Click"),
			ToggleLayerIcon,
			FUIAction(FExecuteAction::CreateWeakLambda(
				Layer,
				[Layer]()
				{
					FScopedTransaction Transaction(LOCTEXT("ToggleAllStageEnabled", "Toggle All Stage Enabled"));

					for (UDMMaterialStage* Stage : Layer->GetStages(EDMMaterialLayerStage::All))
					{
						Stage->Modify();
						Stage->SetEnabled(!Stage->IsEnabled());
					}
				}
			))
		);
	}

	InSection.AddMenuEntry(
		FDynamicMaterialEditorCommands::Get().InsertDefaultLayerAbove,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIconFinder::FindIcon("EditableComboBox.Add")
	);

	InSection.AddMenuEntry(FGenericCommands::Get().Copy);
	InSection.AddMenuEntry(FGenericCommands::Get().Cut);
	InSection.AddMenuEntry(FGenericCommands::Get().Paste);
	InSection.AddMenuEntry(FGenericCommands::Get().Duplicate);
	InSection.AddMenuEntry(FGenericCommands::Get().Delete);
}

void FDMMaterialSlotLayerMenus::AddGlobalValueSection(FToolMenuSection& InSection)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDMMenuContext* MenuContext = InSection.FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MenuContext->GetPreviewModel();

	if (!MaterialModel)
	{
		return;
	}

	const TArray<UDMMaterialValue*>& Values = MaterialModel->GetValues();

	if (Values.IsEmpty())
	{
		return;
	}

	InSection.AddSubMenu(
		TEXT("GlobalValueMenu"),
		LOCTEXT("AddValueStage", "Global Value"),
		LOCTEXT("AddValueStageTooltip", "Add a Material Stage based on a Material Value defined above."),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

				if (!MenuContext)
				{
					return;
				}

				UDynamicMaterialModel* MaterialModel = MenuContext->GetPreviewModel();

				if (!MaterialModel)
				{
					return;
				}

				TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

				if (!EditorWidget.IsValid())
				{
					return;
				}

				const TArray<UDMMaterialValue*>& Values = MaterialModel->GetValues();

				for (UDMMaterialValue* Value : Values)
				{
					InMenu->AddMenuEntry(
						TEXT("Value"),
						FToolMenuEntry::InitMenuEntry(
							Value->GetFName(),
							Value->GetDescription(),
							LOCTEXT("AddValueStageSpecificTooltip", "Add a Material Stage based on this Material Value."),
							Value->GetComponentIcon(),
							FUIAction(FExecuteAction::CreateWeakLambda(
								Value,
								[ValueWeak = TWeakObjectPtr<UDMMaterialValue>(Value)]
								{
									// Currently unimplemented pending later re-adding.
								}
							))
						));
				}
			})
	);

	InSection.AddSubMenu(
		TEXT("NewGlobalValue"),
		LOCTEXT("AddNewValueStage", "New Global Value"),
		LOCTEXT("AddNewValueStageTooltip", "Add a new global Material Value as use it as a Material Stage."),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

				if (!MenuContext)
				{
					return;
				}

				TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

				if (!EditorWidget.IsValid())
				{
					return;
				}

				UDMMaterialSlot* Slot = EditorWidget->GetSlotEditorWidget()->GetSlot();

				if (!Slot)
				{
					return;
				}

				for (EDMValueType ValueType : UDMValueDefinitionLibrary::GetValueTypes())
				{
					const FText Name = UDMValueDefinitionLibrary::GetValueDefinition(ValueType).GetDisplayName();
					const FText FormattedTooltip = FText::Format(LOCTEXT("AddTypeTooltipTemplate", "Add a new {0} Value and use it as a Material Stage."), Name);

					const FSlateIcon ValueIcon = UDMValueDefinitionLibrary::GetValueIcon(ValueType);

					InMenu->AddMenuEntry(
						TEXT("NewGlobalValue"),
						FToolMenuEntry::InitMenuEntry(
							*Name.ToString(),
							Name,
							FormattedTooltip,
							ValueIcon,
							FUIAction(FExecuteAction::CreateWeakLambda(
								Slot,
								[Slot, ValueType]
								{
									UDMMaterialSlotFunctionLibrary::AddNewLayer_NewGlobalValue(Slot, ValueType);
								}
							))
						)
					);
				}
			})
	);
}

void FDMMaterialSlotLayerMenus::AddSlotMenuEntry(const TSharedPtr<SDMMaterialSlotEditor> InSlotWidget, UToolMenu* InMenu, const FText& InName, 
	UDMMaterialSlot* InSourceSlot, EDMMaterialPropertyType InMaterialProperty)
{
	UDMMaterialSlot* TargetSlot = InSlotWidget->GetSlot();

	if (!TargetSlot)
	{
		return;
	}

	InMenu->AddMenuEntry(
		TEXT("Slot"),
		FToolMenuEntry::InitMenuEntry(
			TargetSlot->GetFName(),
			InName,
			LOCTEXT("AddSlotStageSpecificTooltip", "Add a Material Stage based on this Material Slot."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateWeakLambda(
				TargetSlot,
				[TargetSlotWeak = TWeakObjectPtr<UDMMaterialSlot>(TargetSlot), SourceSlotWeak = TWeakObjectPtr<UDMMaterialSlot>(InSourceSlot), InMaterialProperty]
				{
					if (UDMMaterialSlot* SourceSlot = SourceSlotWeak.Get())
					{
						UDMMaterialSlotFunctionLibrary::AddNewLayer_Slot(TargetSlotWeak.Get(), SourceSlot, InMaterialProperty);
					}
				}
			))
		));
}

void FDMMaterialSlotLayerMenus::AddLayerInputsMenu_Slot_Properties(UToolMenu* InMenu, UDMMaterialSlot* InSlot)
{
	if (!IsValid(InMenu))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MenuContext->GetPreviewModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!ensure(ModelEditorOnlyData) || !ensure(InSlot->GetMaterialModelEditorOnlyData() == ModelEditorOnlyData))
	{
		return;
	}

	const TArray<EDMMaterialPropertyType> SlotProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(InSlot);

	for (EDMMaterialPropertyType SlotProperty : SlotProperties)
	{
		UDMMaterialProperty* MaterialProperty = ModelEditorOnlyData->GetMaterialProperty(SlotProperty);

		if (ensure(MaterialProperty))
		{
			AddSlotMenuEntry(
				EditorWidget->GetSlotEditorWidget(),
				InMenu,
				MaterialProperty->GetDescription(),
				InSlot,
				SlotProperty
			);
		}
	}
}

void FDMMaterialSlotLayerMenus::AddLayerInputsMenu_Slots(UToolMenu* InMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!IsValid(InMenu) || InMenu->ContainsSection(SlotLayerAddSectionName))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = EditorWidget->GetSlotEditorWidget()->GetSlot();

	if (!Slot)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

	if (!ModelEditorOnlyData)
	{
		return;
	}

	const TArray<UDMMaterialSlot*>& Slots = ModelEditorOnlyData->GetSlots();

	if (Slots.Num() <= 1)
	{
		return;
	}

	for (UDMMaterialSlot* CurrentSlot : Slots)
	{
		if (Slot == CurrentSlot)
		{
			continue;
		}

		if (CurrentSlot->GetLayers().IsEmpty())
		{
			continue;
		}

		const TArray<EDMMaterialPropertyType> SlotProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(CurrentSlot);

		if (SlotProperties.Num() == 1)
		{
			static const FText SlotNameFormatTemplate = LOCTEXT("SlotAndProperty", "{0} [{1}]");

			UDMMaterialProperty* MaterialProperty = ModelEditorOnlyData->GetMaterialProperty(SlotProperties[0]);

			if (ensure(MaterialProperty))
			{
				AddSlotMenuEntry(
					EditorWidget->GetSlotEditorWidget(),
					InMenu,
					FText::Format(SlotNameFormatTemplate, CurrentSlot->GetDescription(), MaterialProperty->GetDescription()),
					CurrentSlot,
					SlotProperties[0]
				);
			}
		}
		else
		{
			InMenu->AddMenuEntry(
				TEXT("Slot"),
				FToolMenuEntry::InitSubMenu(
					CurrentSlot->GetFName(),
					LOCTEXT("AddSlotStage2", "Add Slot Output"),
					LOCTEXT("AddSlotStageTooltip2", "Add a Material Stage based on the output of another Material Slot."),
					FNewToolMenuDelegate::CreateStatic(&AddLayerInputsMenu_Slot_Properties, CurrentSlot))
			);
		}
	}
}

void FDMMaterialSlotLayerMenus::AddLayerMenu_Gradients(UToolMenu* InMenu)
{
	if (!IsValid(InMenu))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = EditorWidget->GetSlotEditorWidget()->GetSlot();

	if (!Slot)
	{
		return;
	}

	const TArray<TStrongObjectPtr<UClass>>& Gradients = UDMMaterialStageGradient::GetAvailableGradients();

	FToolMenuSection& NewSection = InMenu->AddSection(
		TEXT("Gradient"),
		LOCTEXT("Gradients", "Gradients")
	);

	for (const TStrongObjectPtr<UClass>& Gradient : Gradients)
	{
		UDMMaterialStageGradient* GradientCDO = Cast<UDMMaterialStageGradient>(Gradient->GetDefaultObject());

		if (!ensure(GradientCDO))
		{
			continue;
		}

		const FText MenuName = GradientCDO->GetDescription();

		NewSection.AddMenuEntry(
			GradientCDO->GetFName(),
			MenuName,
			LOCTEXT("ChangeGradientSourceTooltip", "Change the source of this stage to a Material Gradient."),
			GradientCDO->GetComponentIcon(),
			FUIAction(FExecuteAction::CreateWeakLambda(
				Slot,
				[Slot, GradientClass = TSubclassOf<UDMMaterialStageGradient>(Gradient.Get())]
				{
					UDMMaterialSlotFunctionLibrary::AddNewLayer_Gradient(Slot, GradientClass);
				}
			))
		);
	}

	NewSection.AddMenuEntry(
		TEXT("ColorAtlas"),
		LOCTEXT("AddColorAtlas", "Color Atlas"),
		LOCTEXT("AddColorAtlasTooltip", "Add a new Material Layer with a Color Atlas."),
		GetDefault<UDMMaterialValueColorAtlas>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_NewLocalValue(Slot, EDMValueType::VT_ColorAtlas);
			}
		))
	);
}

void FDMMaterialSlotLayerMenus::AddLayerMenu_Advanced(UToolMenu* InMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!IsValid(InMenu))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = MenuContext->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = EditorWidget->GetSlotEditorWidget()->GetSlot();

	if (!Slot)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

	if (!ModelEditorOnlyData)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = ModelEditorOnlyData->GetMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	FToolMenuSection& NewSection = InMenu->AddSection(
		TEXT("Advanced"),
		LOCTEXT("Advanced", "Advanced")
	);

	NewSection.AddMenuEntry(
		TEXT("Text"),
		LOCTEXT("AddText", "Text"),
		LOCTEXT("AddTextTooltip", "Add a Material Stage based on a Text Renderer."),
		GetDefault<UDMRenderTargetTextRenderer>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_Renderer(Slot, TSubclassOf<UDMRenderTargetRenderer>(UDMRenderTargetTextRenderer::StaticClass()));
			}
		))
	);

	NewSection.AddMenuEntry(
		TEXT("Widget"),
		LOCTEXT("AddWidget", "Widget"),
		LOCTEXT("AddWidgetTooltip", "Add a Material Stage based on a Widget Renderer."),
		GetDefault<UDMRenderTargetUMGWidgetRenderer>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_Renderer(Slot, TSubclassOf<UDMRenderTargetRenderer>(UDMRenderTargetUMGWidgetRenderer::StaticClass()));
			}
		))
	);

	NewSection.AddMenuEntry(
		TEXT("MaterialFunction"),
		LOCTEXT("AddMaterialFunction", "Material Function"),
		LOCTEXT("AddMaterialFunctionTooltip", "Add a new Material Layer based on a Material Function."),
		GetDefault<UDMMaterialStageFunction>()->GetComponentIcon(),
		FUIAction(FExecuteAction::CreateWeakLambda(
			Slot,
			[Slot]
			{
				UDMMaterialSlotFunctionLibrary::AddNewLayer_MaterialFunction(Slot);
			}
		))
	);

	if constexpr (UE::DynamicMaterialEditor::bAdvancedSlotsEnabled)
	{
		bool bHasValidSlot = false;
		const TArray<UDMMaterialSlot*>& Slots = ModelEditorOnlyData->GetSlots();

		for (UDMMaterialSlot* SlotIter : Slots)
		{
			if (Slot == SlotIter)
			{
				continue;
			}

			if (SlotIter->GetLayers().IsEmpty())
			{
				continue;
			}

			TArray<EDMMaterialPropertyType> SlotProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(SlotIter);

			if (SlotProperties.IsEmpty())
			{
				continue;
			}

			bHasValidSlot = true;
			break;
		}

		if (bHasValidSlot)
		{
			NewSection.AddSubMenu(
				TEXT("SlotOutput"),
				LOCTEXT("AddSlotStage", "Slot Output"),
				LOCTEXT("AddSlotStageTooltip", "Add a Material Stage based on the output of another Material Slot."),
				FNewToolMenuDelegate::CreateStatic(&AddLayerInputsMenu_Slots)
			);
		}
	}
}

#undef LOCTEXT_NAMESPACE
