// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "Templates/SharedPointerFwd.h"

class SDMMaterialSlotEditor;
class SWidget;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UToolMenu;
enum class EDMMaterialPropertyType : uint8;
struct FToolMenuSection;

class FDMMaterialSlotLayerMenus final
{
public:
	static TSharedRef<SWidget> GenerateSlotLayerMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget, UDMMaterialLayerObject* InLayer);

	static void AddAddLayerSection(FToolMenuSection& InSection);

private:
	static TSharedRef<SWidget> GenerateSlotLayerMenu_AddLayer(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget);

	static TSharedRef<SWidget> GenerateSlotLayerMenu_Layer(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget, UDMMaterialLayerObject* InLayer);

	static void AddLayerModifySection(FToolMenuSection& InSection);

	static void AddGlobalValueSection(FToolMenuSection& InSection);

	static void AddSlotMenuEntry(const TSharedPtr<SDMMaterialSlotEditor> InSlotWidget, UToolMenu* InMenu, const FText& InName, 
		UDMMaterialSlot* InSourceSlot, EDMMaterialPropertyType InMaterialProperty);

	static void AddLayerInputsMenu_Slot_Properties(UToolMenu* InMenu, UDMMaterialSlot* InSlot);

	static void AddLayerInputsMenu_Slots(UToolMenu* InMenu);

	static void AddLayerMenu_Gradients(UToolMenu* InMenu);

	static void AddLayerMenu_Advanced(UToolMenu* InMenu);
};
