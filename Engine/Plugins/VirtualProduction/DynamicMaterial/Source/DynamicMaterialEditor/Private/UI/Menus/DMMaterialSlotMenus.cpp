// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMMaterialSlotMenus.h"

#include "Templates/SharedPointer.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMaterialSlotLayerMenus.h"
#include "UI/Menus/DMMenuContext.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "FDMMaterialSlotMenus"

namespace UE::DynamicMaterialEditor::Private
{
	static const FName SlotAddLayerMenuName = TEXT("MaterialDesigner.MaterialSlot.AddLayer");
}

TSharedRef<SWidget> FDMMaterialSlotMenus::MakeAddLayerButtonMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!UToolMenus::Get()->IsMenuRegistered(SlotAddLayerMenuName))
	{
		UToolMenu* NewToolMenu = UDMMenuContext::GenerateContextMenuDefault(SlotAddLayerMenuName);

		if (!NewToolMenu)
		{
			return SNullWidget::NullWidget;
		}

		FToolMenuSection& AddLayerSection = NewToolMenu->FindOrAddSection(TEXT("AddLayer"), LOCTEXT("AddLayer", "Add Layer"));

		AddLayerSection.AddDynamicEntry(TEXT("AddLayer"), FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialSlotLayerMenus::AddAddLayerSection));
	}

	FToolMenuContext MenuContext(UDMMenuContext::CreateEditor(InSlotWidget->GetEditorWidget()));

	return UToolMenus::Get()->GenerateWidget(SlotAddLayerMenuName, MenuContext);
}

#undef LOCTEXT_NAMESPACE
