// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class SDMMaterialSlotEditor;
class SWidget;
class UToolMenu;

class FDMMaterialSlotMenus final
{
public:
	static TSharedRef<SWidget> MakeAddLayerButtonMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget);
};
