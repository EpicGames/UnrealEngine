// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMMaterialStageMenus.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "DMMaterialStageSourceMenus.h"
#include "DynamicMaterialEditorStyle.h"
#include "ScopedTransaction.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMenuContext.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialStage.h"
#include "UI/Widgets/SDMMaterialEditor.h"

#define LOCTEXT_NAMESPACE "FDMMaterialStageMenus"

namespace UE::DynamicMaterialEditor::Private
{
	const FLazyName StageSettingsMenuName = TEXT("MaterialDesigner.MaterialStage");
	const FLazyName StageSettingsSectionName = TEXT("ChangeStageSettings");
	const FLazyName StageSourceSectionName = TEXT("ChangeStageSource");
}

TSharedRef<SWidget> FDMMaterialStageMenus::GenerateStageMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget,
	const TSharedPtr<SDMMaterialStage>& InStageWidget)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(StageSettingsMenuName))
	{
		UToolMenu* NewToolMenu = UDMMenuContext::GenerateContextMenuDefault(StageSettingsMenuName);

		if (!NewToolMenu)
		{
			return SNullWidget::NullWidget;
		}

		FToolMenuSection& ChangeSection = NewToolMenu->FindOrAddSection(StageSettingsSectionName, LOCTEXT("ModifyStage", "Modify Stage"));
		ChangeSection.AddDynamicEntry(StageSettingsSectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialStageMenus::AddStageSettingsSection));

		FToolMenuSection& SourceSection = NewToolMenu->FindOrAddSection(StageSourceSectionName, LOCTEXT("ChangeStageSource", "Change Stage Source"));
		SourceSection.AddDynamicEntry(StageSourceSectionName, FNewToolMenuSectionDelegate::CreateStatic(&FDMMaterialStageMenus::AddStageSourceSection));
	}

	FToolMenuContext MenuContext(UDMMenuContext::CreateStage(InSlotWidget->GetEditorWidget(), InStageWidget));

	return ToolMenus->GenerateWidget(StageSettingsMenuName, MenuContext);
}

void FDMMaterialStageMenus::AddStageSettingsSection(FToolMenuSection& InSection)
{
	using namespace UE::DynamicMaterialEditor::Private;

	const UDMMenuContext* const MenuContext = InSection.FindContext<UDMMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	UDMMaterialStage* Stage = MenuContext->GetStage();
	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	if (!Layer)
	{
		return;
	}

	UDMMaterialSlot* Slot = Layer->GetSlot();
	if (!Slot)
	{
		return;
	}

	const EDMMaterialLayerStage StageType = Layer->GetStageType(Stage);

	// Only if we can remove a layer can we toggle the base stage.
	const bool bAllowRemoveLayer = Slot->CanRemoveLayer(Layer);

	InSection.InitSection(StageSettingsSectionName, LOCTEXT("MaterialStageMenu", "Stage Actions"), FToolMenuInsert());

	if (bAllowRemoveLayer)
	{
		if (StageType == EDMMaterialLayerStage::Base)
		{
			InSection.AddMenuEntry(
				TEXT("ToggleBase"),
				LOCTEXT("ToggleLayerBase", "Toggle Base"),
				LOCTEXT("ToggleLayerBaseTooltip", "Toggle the Layer Base.\n\nAlt+Shift+Left Click"),
				FSlateIcon(FDynamicMaterialEditorStyle::Get().GetStyleSetName(), TEXT("Icons.Stage.Enabled")),
				FUIAction(FExecuteAction::CreateWeakLambda(
					Layer,
					[Layer]()
					{
						FScopedTransaction Transaction(LOCTEXT("ToggleBaseStageEnabled", "Toggle Base Stage Enabled"));

						if (UDMMaterialStage* Stage = Layer->GetStage(EDMMaterialLayerStage::Base))
						{
							Stage->Modify();
							Stage->SetEnabled(!Stage->IsEnabled());
						}
					}
				))
			);
		}
	}

	if (StageType == EDMMaterialLayerStage::Mask)
	{
		InSection.AddMenuEntry(
			TEXT("ToggleMask"),
			LOCTEXT("ToggleLayerMask", "Toggle Mask"),
			LOCTEXT("ToggleLayerMaskTooltip", "Toggle the Layer Mask.\n\nAlt+Shift+Left Click"),
			FSlateIcon(FDynamicMaterialEditorStyle::Get().GetStyleSetName(), TEXT("Icons.Stage.Enabled")),
			FUIAction(FExecuteAction::CreateWeakLambda(
				Layer,
				[Layer]()
				{
					FScopedTransaction Transaction(LOCTEXT("ToggleMaskStageEnabled", "Toggle Mask Stage Enabled"));

					if (UDMMaterialStage* Stage = Layer->GetStage(EDMMaterialLayerStage::Mask))
					{
						Stage->Modify();
						Stage->SetEnabled(!Stage->IsEnabled());
					}
				}
			))
		);
	}
}

void FDMMaterialStageMenus::AddStageSourceSection(FToolMenuSection& InSection)
{
	using namespace UE::DynamicMaterialEditor::Private;

	InSection.InitSection(StageSourceSectionName, LOCTEXT("MaterialStageSource", "Change Stage Source"), FToolMenuInsert());

	FDMMaterialStageSourceMenus::CreateChangeMaterialStageSource(InSection);
}

#undef LOCTEXT_NAMESPACE
