// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FName;
class SDMMaterialSlotEditor;
class SDMMaterialStage;
class SWidget;
struct FToolMenuSection;

class FDMMaterialStageMenus final
{
public:
	static TSharedRef<SWidget> GenerateStageMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget, const TSharedPtr<SDMMaterialStage>& InStageWidget);

private:
	static void AddStageSettingsSection(FToolMenuSection& InSection);

	static void AddStageSourceSection(FToolMenuSection& InSection);
};
