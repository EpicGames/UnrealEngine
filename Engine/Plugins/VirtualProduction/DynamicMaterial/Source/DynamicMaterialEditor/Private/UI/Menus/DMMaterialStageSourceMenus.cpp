// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMMaterialStageSourceMenus.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageExpression.h"
#include "Components/DMMaterialStageFunction.h"
#include "Components/DMMaterialStageGradient.h"
#include "Components/DMMaterialStageSource.h"
#include "Components/DMMaterialStageThroughput.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialStageExpressions/DMMSESceneTexture.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageExpressions/DMMSETextureSampleEdgeColor.h"
#include "Components/MaterialStageExpressions/DMMSEWorldPositionNoise.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIFunction.h"
#include "Components/MaterialStageInputs/DMMSIGradient.h"
#include "Components/MaterialStageInputs/DMMSISlot.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueColorAtlas.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/RenderTargetRenderers/DMRenderTargetTextRenderer.h"
#include "Components/RenderTargetRenderers/DMRenderTargetUMGWidgetRenderer.h"
#include "DMDefs.h"
#include "DMValueDefinition.h"
#include "DynamicMaterialEditorModule.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "ScopedTransaction.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMenuContext.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialStage.h"
#include "Utils/DMMaterialStageFunctionLibrary.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "FDMMaterialStageSourceMenus"

namespace UE::DynamicMaterialEditor::Private
{
	static const FName ChangeStageSourceMenuName = TEXT("MaterialDesigner.MaterialStage.ChangeSource");	
}

TSharedRef<SWidget> FDMMaterialStageSourceMenus::MakeChangeSourceMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget, const TSharedPtr<SDMMaterialStage>& InStageWidget)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(ChangeStageSourceMenuName))
	{
		UToolMenu* const NewToolMenu = UDMMenuContext::GenerateContextMenuDefault(ChangeStageSourceMenuName);

		if (!NewToolMenu)
		{
			return SNullWidget::NullWidget;
		}

		FToolMenuSection& NewSection = NewToolMenu->FindOrAddSection(TEXT("ChangeStageSource"), LOCTEXT("ChangeStageSource", "Change Stage Source"));
		NewSection.AddDynamicEntry(TEXT("ChangeStageSource"), FNewToolMenuSectionDelegate::CreateStatic(&CreateChangeMaterialStageSource));
	}

	FToolMenuContext MenuContext(UDMMenuContext::CreateStage(InSlotWidget->GetEditorWidget(), InStageWidget));

	return ToolMenus->GenerateWidget(ChangeStageSourceMenuName, MenuContext);
}

void FDMMaterialStageSourceMenus::CreateSourceMenuTree(TFunction<void(EDMExpressionMenu Menu, TArray<UDMMaterialStageExpression*>& SubmenuExpressionList)> InCallback, const TArray<TStrongObjectPtr<UClass>>& InAllExpressions)
{
	TMap<EDMExpressionMenu, TArray<UDMMaterialStageExpression*>> MenuMap;

	for (const TStrongObjectPtr<UClass>& Class : InAllExpressions)
	{
		TSubclassOf<UDMMaterialStageExpression> ExpressionClass = Class.Get();
		if (!ExpressionClass.Get())
		{
			continue;
		}

		UDMMaterialStageExpression* ExpressionCDO = Cast<UDMMaterialStageExpression>(ExpressionClass->GetDefaultObject(true));
		if (!ExpressionCDO)
		{
			continue;
		}

		const TArray<EDMExpressionMenu>& Menus = ExpressionCDO->GetMenus();
		for (EDMExpressionMenu Menu : Menus)
		{
			MenuMap.FindOrAdd(Menu).Add(ExpressionCDO);
		}
	}

	auto CreateMenu = [&InCallback, &MenuMap](EDMExpressionMenu Menu)
	{
		TArray<UDMMaterialStageExpression*>* ExpressionListPtr = MenuMap.Find(Menu);

		if (!ExpressionListPtr || ExpressionListPtr->IsEmpty())
		{
			return;
		}

		InCallback(Menu, *ExpressionListPtr);
	};

	CreateMenu(EDMExpressionMenu::Texture);
	CreateMenu(EDMExpressionMenu::Math);
	CreateMenu(EDMExpressionMenu::Geometry);
	CreateMenu(EDMExpressionMenu::Object);
	CreateMenu(EDMExpressionMenu::WorldSpace);
	CreateMenu(EDMExpressionMenu::Time);
	CreateMenu(EDMExpressionMenu::Camera);
	CreateMenu(EDMExpressionMenu::Particle);
	CreateMenu(EDMExpressionMenu::Decal);
	CreateMenu(EDMExpressionMenu::Landscape);
	CreateMenu(EDMExpressionMenu::Other);
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_NewLocalValues(UToolMenu* InMenu)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!ensure(MenuContext))
	{
		return;
	}

	UDMMaterialStage* Stage = MenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	for (EDMValueType ValueType : UDMValueDefinitionLibrary::GetValueTypes())
	{
		static const FText NameTooltipFormat = LOCTEXT("ChangeSourceNewValueSourceTooltipTemplate", "Add a new {0} Value and use it as the source of this stage.");

		const FText Name = UDMValueDefinitionLibrary::GetValueDefinition(ValueType).GetDisplayName();
		const FText FormattedTooltip = FText::Format(NameTooltipFormat, Name);

		const FSlateIcon ValueIcon = UDMValueDefinitionLibrary::GetValueIcon(ValueType);

		InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry(NAME_None,
			Name,
			FormattedTooltip,
			ValueIcon,
			FUIAction(
				FExecuteAction::CreateWeakLambda(
					MenuContext,
					[ValueType, MenuContext]()
					{
						UDMMaterialStage* const Stage = MenuContext->GetStage();
						if (!Stage)
						{
							return;
						}

						UDMMaterialStageSource* const StageSource = Stage->GetSource();
						if (!StageSource)
						{
							return;
						}

						if (StageSource->IsA<UDMMaterialStageBlend>())
						{
							const int32 OutputChannel = ValueType == EDMValueType::VT_ColorAtlas
								? FDMMaterialStageConnectorChannel::THREE_CHANNELS
								: FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

							FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
							Stage->Modify();

							UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
								Stage, 
								UDMMaterialStageBlend::InputB,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
								ValueType, 
								OutputChannel
							);
						}
						else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
						{
							const int32 OutputChannel = ValueType == EDMValueType::VT_ColorAtlas
								? FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
								: FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

							FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
							Stage->Modify();

							UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
								Stage, 
								UDMMaterialStageThroughputLayerBlend::InputMaskSource,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
								ValueType,
								OutputChannel
							);
						}
						else
						{
							ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
						}
					})
			)
		));
	}
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_GlobalValues(UToolMenu* InMenu)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();
	if (!ensure(MenuContext))
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MenuContext->GetPreviewModel();
	if (!ensure(MaterialModel))
	{
		return;
	}

	const TArray<UDMMaterialValue*>& Values = MaterialModel->GetValues();

	if (Values.IsEmpty())
	{
		return;
	}

	for (UDMMaterialValue* Value : Values)
	{
		if (!IsValid(Value))
		{
			continue;
		}

		InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry(NAME_None,
			Value->GetDescription(),
			LOCTEXT("ChangeSourceValueSourceTooltip2", "Change the source of this stage to this Material Value."),
			Value->GetComponentIcon(),
			FUIAction(FExecuteAction::CreateWeakLambda(
				MenuContext,
				[MenuContext, Value]()
				{
					UDMMaterialStage* const Stage = MenuContext->GetStage();
					if (!Stage)
					{
						return;
					}

					UDMMaterialStageSource* const StageSource = Stage->GetSource();
					if (!StageSource)
					{
						return;
					}

					if (StageSource->IsA<UDMMaterialStageBlend>())
					{
						const int32 OutputChannel = Value->GetType() == EDMValueType::VT_ColorAtlas
							? FDMMaterialStageConnectorChannel::THREE_CHANNELS
							: FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

						FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
						Stage->Modify();

						UDMMaterialStageInputValue::ChangeStageInput_Value(
							Stage, 
							UDMMaterialStageBlend::InputB,
							FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
							Value, 
							OutputChannel
						);
					}
					else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
					{
						const int32 OutputChannel = Value->GetType() == EDMValueType::VT_ColorAtlas
							? FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
							: FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

						FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
						Stage->Modify();

						UDMMaterialStageInputValue::ChangeStageInput_Value(
							Stage, 
							UDMMaterialStageThroughputLayerBlend::InputMaskSource, 
							FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
							Value, 
							OutputChannel
						);
					}
					else
					{
						ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
					}
				})
			)
		));
	}
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_NewGlobalValues(UToolMenu* InMenu)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();
	if (!ensure(MenuContext))
	{
		return;
	}

	UDMMaterialStage* Stage = MenuContext->GetStage();
	if (!Stage)
	{
		return;
	}

	for (EDMValueType ValueType : UDMValueDefinitionLibrary::GetValueTypes())
	{
		static const FText NameTooltipFormat = LOCTEXT("ChangeSourceNewValueSourceTooltipTemplate", "Add a new {0} Value and use it as the source of this stage.");

		const FText Name = UDMValueDefinitionLibrary::GetValueDefinition(ValueType).GetDisplayName();
		const FText FormattedTooltip = FText::Format(NameTooltipFormat, Name);

		const FSlateIcon ValueIcon = UDMValueDefinitionLibrary::GetValueIcon(ValueType);

		InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry(NAME_None,
			Name,
			FormattedTooltip,
			ValueIcon,
			FUIAction(
				FExecuteAction::CreateWeakLambda(
					MenuContext,
					[ValueType, MenuContext]()
					{
						UDMMaterialStage* const Stage = MenuContext->GetStage();
						if (!Stage)
						{
							return;
						}

						UDMMaterialStageSource* const StageSource = Stage->GetSource();
						if (!StageSource)
						{
							return;
						}

						if (StageSource->IsA<UDMMaterialStageBlend>())
						{
							const int32 OutputChannel = ValueType == EDMValueType::VT_ColorAtlas
								? FDMMaterialStageConnectorChannel::THREE_CHANNELS
								: FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

							FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
							Stage->Modify();

							UDMMaterialStageInputValue::ChangeStageInput_NewValue(
								Stage, 
								UDMMaterialStageBlend::InputB, 
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
								ValueType, 
								OutputChannel
							);
						}
						else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
						{
							const int32 OutputChannel = ValueType == EDMValueType::VT_ColorAtlas
								? FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
								: FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

							FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
							Stage->Modify();

							UDMMaterialStageInputValue::ChangeStageInput_NewValue(
								Stage, 
								UDMMaterialStageThroughputLayerBlend::InputMaskSource,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
								ValueType,
								OutputChannel
							);
						}
						else
						{
							ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
						}
					})
			)
		));
	}
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_Slot_Properties(UToolMenu* InMenu, UDMMaterialSlot* InSlot)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();
	if (!ensure(MenuContext))
	{
		return;
	}

	UDMMaterialStage* Stage = MenuContext->GetStage();
	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	if (!Layer)
	{
		return;
	}

	if (!IsValid(InSlot) || Layer->GetSlot() != InSlot)
	{
		return;
	}

	UDynamicMaterialModelBase* MaterialModel = MenuContext->GetPreviewModelBase();
	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);
	if (!ModelEditorOnlyData)
	{
		return;
	}

	for (EDMMaterialPropertyType Property : ModelEditorOnlyData->GetMaterialPropertiesForSlot(InSlot))
	{
		UDMMaterialProperty* PropertyObj = ModelEditorOnlyData->GetMaterialProperty(Property);

		if (ensure(PropertyObj))
		{
			InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry(NAME_None,
				PropertyObj->GetDescription(),
				LOCTEXT("ChangeSourceSlotSourceTooltip3", "Change the source of this stage to the output from this Material Slot's Property."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateWeakLambda(
					MenuContext,
					[MenuContext, Property]()
					{
						UDMMaterialStage* const Stage = MenuContext->GetStage();
						if (!Stage)
						{
							return;
						}

						UDMMaterialStageSource* const StageSource = Stage->GetSource();
						if (!StageSource)
						{
							return;
						}

						UDMMaterialLayerObject* Layer = Stage->GetLayer();
						if (!Layer)
						{
							return;
						}

						UDMMaterialSlot* Slot = Layer->GetSlot();
						if (!Slot)
						{
							return;
						}

						if (StageSource->IsA<UDMMaterialStageBlend>())
						{
							FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
							Stage->Modify();

							UDMMaterialStageInputSlot::ChangeStageInput_Slot(
								Stage, 
								UDMMaterialStageBlend::InputB,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
								Slot, 
								Property,
								0, 
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
							);
						}
						else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
						{
							FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
							Stage->Modify();

							UDMMaterialStageInputSlot::ChangeStageInput_Slot(
								Stage, 
								UDMMaterialStageThroughputLayerBlend::InputMaskSource,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
								Slot, 
								Property, 
								0,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
							);
						}
						else
						{
							ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
						}
					})
				)
			));
		}
	}
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_Slots(UToolMenu* InMenu)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();
	if (!ensure(MenuContext))
	{
		return;
	}

	UDMMaterialStage* Stage = MenuContext->GetStage();
	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* Layer = Stage->GetLayer();
	if (!Layer)
	{
		return;
	}

	UDMMaterialSlot* Slot = Layer->GetSlot();
	if (!Slot)
	{
		return;
	}

	UDynamicMaterialModelBase* MaterialModelBase = MenuContext->GetPreviewModelBase();
	if (!MaterialModelBase)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModelBase);
	if (!MaterialModelBase)
	{
		return;
	}

	const TArray<UDMMaterialSlot*>& Slots = ModelEditorOnlyData->GetSlots();
	if (Slots.Num() <= 1)
	{
		return;
	}

	for (UDMMaterialSlot* CurrentSlot : Slots)
	{
		if (CurrentSlot == Slot)
		{
			continue;
		}

		if (CurrentSlot->GetLayers().IsEmpty())
		{
			continue;
		}

		TArray<EDMMaterialPropertyType> SlotProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(CurrentSlot);
		if (SlotProperties.IsEmpty())
		{
			continue;
		}

		if (SlotProperties.Num() == 1)
		{
			static const FText SlotNameFormatTemplate = LOCTEXT("SlotAndProperty", "{0} [{1}]");

			UDMMaterialProperty* PropertyObj = ModelEditorOnlyData->GetMaterialProperty(SlotProperties[0]);

			if (ensure(PropertyObj))
			{
				InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry(NAME_None,
					FText::Format(SlotNameFormatTemplate, CurrentSlot->GetDescription(), PropertyObj->GetDescription()),
					LOCTEXT("ChangeSourceSlotSourceTooltip3", "Change the source of this stage to the output from this Material Slot's Property."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateWeakLambda(
						CurrentSlot,
						[CurrentSlot, SlotProperty = SlotProperties[0], MenuContext]()
						{
							if (!IsValid(MenuContext))
							{
								return;
							}

							UDMMaterialStage* const Stage = MenuContext->GetStage();
							if (!Stage)
							{
								return;
							}

							UDMMaterialStageSource* const StageSource = Stage->GetSource();
							if (!StageSource)
							{
								return;
							}

							if (StageSource->IsA<UDMMaterialStageBlend>())
							{
								FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
								Stage->Modify();

								UDMMaterialStageInputSlot::ChangeStageInput_Slot(
									Stage, 
									UDMMaterialStageBlend::InputB,
									FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
									CurrentSlot, 
									SlotProperty, 
									0, 
									FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
								);
							}
							else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
							{
								FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
								Stage->Modify();

								UDMMaterialStageInputSlot::ChangeStageInput_Slot(
									Stage, 
									UDMMaterialStageThroughputLayerBlend::InputMaskSource,
									FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
									CurrentSlot, 
									SlotProperty, 
									0,
									FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
								);
							}
							else
							{
								ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
							}
						})
					)
				));
			}
		}
		else
		{
			FToolMenuSection& NewSection = InMenu->FindOrAddSection("ChangeSourceSlotTooltip", LOCTEXT("ChangeSourceSlot", "Change Source Slot"));
			NewSection.AddDynamicEntry(NAME_None,
				FNewToolMenuSectionDelegate::CreateWeakLambda(
					CurrentSlot,
					[CurrentSlot](FToolMenuSection& InSection)
					{
						InSection.AddSubMenu(
							NAME_None,
							CurrentSlot->GetDescription(),
							LOCTEXT("ChangeSourceSlotTooltip", "Change the source of this stage to the output from another Material Slot."),
							FNewToolMenuDelegate::CreateLambda([CurrentSlot](UToolMenu* InMenu) { GenerateChangeSourceMenu_Slot_Properties(InMenu, CurrentSlot); })
						);
					}));
		}
	}
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_Gradients(UToolMenu* const InMenu)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();
	if (!ensure(MenuContext))
	{
		return;
	}

	const TArray<TStrongObjectPtr<UClass>>& Gradients = UDMMaterialStageGradient::GetAvailableGradients();
	if (Gradients.IsEmpty())
	{
		return;
	}

	FToolMenuSection& NewSection = InMenu->AddSection(
		TEXT("Gradient"),
		LOCTEXT("Gradients", "Gradients")
	);

	for (TStrongObjectPtr<UClass> GradientClass : Gradients)
	{
		UDMMaterialStageGradient* GradientCDO = Cast<UDMMaterialStageGradient>(GradientClass->GetDefaultObject());

		if (ensure(GradientCDO))
		{
			const FText MenuName = GradientCDO->GetDescription();

			NewSection.AddMenuEntry(
				GradientCDO->GetFName(),
				MenuName,
				LOCTEXT("ChangeSourceGradientTooltip", "Change the source of this stage to a Material Gradient."),
				GradientCDO->GetComponentIcon(),
				FUIAction(FExecuteAction::CreateWeakLambda(
					MenuContext,
					[MenuContext, GradientClass]()
					{
						UDMMaterialStage* const Stage = MenuContext->GetStage();
						if (!Stage)
						{
							return;
						}

						UDMMaterialStageSource* const StageSource = Stage->GetSource();
						if (!StageSource)
						{
							return;
						}

						if (StageSource->IsA<UDMMaterialStageBlend>())
						{
							FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
							Stage->Modify();

							UDMMaterialStageInputGradient::ChangeStageInput_Gradient(
								Stage,
								GradientClass.Get(), UDMMaterialStageBlend::InputB,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
							);
						}
						else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
						{
							FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
							Stage->Modify();
							UDMMaterialStageInputGradient::ChangeStageInput_Gradient(
								Stage,
								GradientClass.Get(), 
								UDMMaterialStageThroughputLayerBlend::InputMaskSource,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
								FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
							);
						}
						else
						{
							ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
						}
					})
			));
		}
	}

	NewSection.AddMenuEntry(
		TEXT("ColorAtlas"),
		LOCTEXT("ChangeSourceColorAtlas", "Color Atlas"),
		LOCTEXT("ChangeSourceColorAtlasTooltip", "Change the source of this stage to a Color Atlas."),
		GetDefault<UDMMaterialValueColorAtlas>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToColorAtlasFromContext,
				MenuContext
			)
		)
	);
}

void FDMMaterialStageSourceMenus::GenerateChangeSourceMenu_Advanced(UToolMenu* const InMenu)
{
	if (!ensure(IsValid(InMenu)))
	{
		return;
	}

	UDMMenuContext* MenuContext = InMenu->FindContext<UDMMenuContext>();
	if (!ensure(MenuContext))
	{
		return;
	}

	UDMMaterialSlot* Slot = MenuContext->GetSlot();

	UDynamicMaterialModel* MaterialModel = MenuContext->GetPreviewModel();

	if (!ensure(MaterialModel))
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!ensure(ModelEditorOnlyData))
	{
		return;
	}

	FToolMenuSection& NewSection = InMenu->FindOrAddSection(
		NAME_None,
		LOCTEXT("Advanced", "Advanced")
	);

	NewSection.AddMenuEntry("Text",
		LOCTEXT("ChangeSourceText", "Text"),
		LOCTEXT("ChangeSourceTextTooltip", "Change the source of this stage to a Text Renderer."),
		GetDefault<UDMRenderTargetTextRenderer>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToTextFromContext,
				MenuContext
			)
		)
	);

	NewSection.AddMenuEntry("Widget",
		LOCTEXT("ChangeSourceWidget", "Widget"),
		LOCTEXT("ChangeSourceWidgetTooltip", "Change the source of this stage to a Widget Renderer."),
		GetDefault<UDMRenderTargetUMGWidgetRenderer>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToWidgetFromContext,
				MenuContext
			)
		)
	);

	NewSection.AddMenuEntry("MaterialFunction",
		LOCTEXT("ChangeSourceMaterialFunction", "Material Function"),
		LOCTEXT("ChangeSourceMaterialFunctionTooltip", "Change the source of this stage to a Material Function."),
		GetDefault<UDMMaterialStageFunction>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToMaterialFunctionFromContext,
				MenuContext
			)
		)
	);

	if constexpr (UE::DynamicMaterialEditor::bAdvancedSlotsEnabled)
	{
		bool bHasValidSlot = false;

		if constexpr (UE::DynamicMaterialEditor::bAdvancedSlotsEnabled)
		{
			const TArray<UDMMaterialSlot*>& Slots = ModelEditorOnlyData->GetSlots();

			for (UDMMaterialSlot* SlotIter : Slots)
			{
				if (SlotIter == Slot)
				{
					continue;
				}

				if (SlotIter->GetLayers().IsEmpty())
				{
					continue;
				}

				bHasValidSlot = true;
				break;
			}
		}

		if (bHasValidSlot)
		{
			NewSection.AddDynamicEntry(
				NAME_None,
				FNewToolMenuSectionDelegate::CreateLambda(
					[](FToolMenuSection& InSection)
					{
						InSection.AddSubMenu("SlotOutput",
							LOCTEXT("ChangeSourceSlotOuptut", "Slot Output"),
							LOCTEXT("ChangeSourceSlotOutputTooltip", "Change the source of this stage to the output from another Material Slot."),
							FNewToolMenuDelegate::CreateStatic(&GenerateChangeSourceMenu_Slots)
						);
					}));
		}
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToTextureSampleFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = Stage->GetSource();

	if (!StageSource)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionTextureSample::StaticClass(), 
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage, 
			UDMMaterialStageExpressionTextureSample::StaticClass(), 
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToNoiseFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = Stage->GetSource();

	if (!StageSource)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionWorldPositionNoise::StaticClass(),
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionWorldPositionNoise::StaticClass(),
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToSolidColorRGBFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = Stage->GetSource();

	if (!StageSource)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			Stage, 
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			EDMValueType::VT_Float3_RGB,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			Stage, 
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			EDMValueType::VT_Float3_RGB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToColorAtlasFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = Stage->GetSource();

	if (!StageSource)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			Stage, 
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, EDMValueType::VT_ColorAtlas,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);

	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			Stage, 
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			EDMValueType::VT_ColorAtlas,
			FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToTextureSampleEdgeColorFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = InMenuContext->GetStageSource();

	if (!StageSource)
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* const Layer = Stage->GetLayer();

	if (!Layer)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionTextureSampleEdgeColor::StaticClass(), 
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage, 
			UDMMaterialStageExpressionTextureSampleEdgeColor::StaticClass(),
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}

	if (UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask))
	{
		MaskStage->SetEnabled(false);
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToSceneTextureFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = InMenuContext->GetStageSource();

	if (!StageSource)
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* const Layer = Stage->GetLayer();

	if (!Layer)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage, 
			UDMMaterialStageExpressionSceneTexture::StaticClass(), 
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::THREE_CHANNELS
		);

		if (Layer->GetStageType(Stage) == EDMMaterialLayerStage::Base)
		{
			if (UDMMaterialStage* MaskStage = Layer->GetStage(EDMMaterialLayerStage::Mask, true))
			{
				MaskStage->Modify();

				UDMMaterialStageInputExpression::ChangeStageInput_Expression(
					MaskStage,
					UDMMaterialStageExpressionSceneTexture::StaticClass(),
					UDMMaterialStageThroughputLayerBlend::InputMaskSource,
					FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
					 0,
					FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
				);
			}
		}
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage, 
			UDMMaterialStageExpressionSceneTexture::StaticClass(), 
			UDMMaterialStageThroughputLayerBlend::InputMaskSource,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

bool FDMMaterialStageSourceMenus::CanChangeSourceToSceneTextureFromContext(UDMMenuContext* InMenuContext)
{
	if (IsValid(InMenuContext))
	{
		if (UDMMaterialStageSource* const StageSource = InMenuContext->GetStageSource())
		{
			if (UDMMaterialStage* const Stage = InMenuContext->GetStage())
			{
				if (UDMMaterialLayerObject* const Layer = Stage->GetLayer())
				{
					if (UDMMaterialSlot* Slot = Layer->GetSlot())
					{
						if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = Slot->GetMaterialModelEditorOnlyData())
						{
							return EditorOnlyData->GetDomain() == EMaterialDomain::MD_PostProcess;
						}
					}
				}
			}
		}
	}

	return false;
}

void FDMMaterialStageSourceMenus::ChangeSourceToMaterialFunctionFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = Stage->GetSource();

	if (!StageSource)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageInputFunction::ChangeStageInput_Function(
			Stage, 
			UDMMaterialStageFunction::GetNoOpFunction(),
			UDMMaterialStageBlend::InputB, 	
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageInputFunction::ChangeStageInput_Function(
			Stage, 
			UDMMaterialStageFunction::GetNoOpFunction(),
			UDMMaterialStageThroughputLayerBlend::InputMaskSource, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToTextFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = InMenuContext->GetStageSource();

	if (!StageSource)
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* const Layer = Stage->GetLayer();

	if (!Layer)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageFunctionLibrary::SetStageInputToRenderer(Stage, UDMRenderTargetTextRenderer::StaticClass(), UDMMaterialStageBlend::InputB);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageFunctionLibrary::SetStageInputToRenderer(Stage, UDMRenderTargetTextRenderer::StaticClass(), UDMMaterialStageThroughputLayerBlend::InputMaskSource);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::ChangeSourceToWidgetFromContext(UDMMenuContext* InMenuContext)
{
	if (!IsValid(InMenuContext))
	{
		return;
	}

	UDMMaterialStageSource* const StageSource = InMenuContext->GetStageSource();

	if (!StageSource)
	{
		return;
	}

	UDMMaterialStage* const Stage = InMenuContext->GetStage();

	if (!Stage)
	{
		return;
	}

	UDMMaterialLayerObject* const Layer = Stage->GetLayer();

	if (!Layer)
	{
		return;
	}

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputBase", "Set Material Designer Base Source"));
		Stage->Modify();

		UDMMaterialStageFunctionLibrary::SetStageInputToRenderer(Stage, UDMRenderTargetUMGWidgetRenderer::StaticClass(), UDMMaterialStageBlend::InputB);
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStageInputMask", "Set Material Designer Mask Source"));
		Stage->Modify();

		UDMMaterialStageFunctionLibrary::SetStageInputToRenderer(Stage, UDMRenderTargetUMGWidgetRenderer::StaticClass(), UDMMaterialStageThroughputLayerBlend::InputMaskSource);
	}
	else
	{
		ensureMsgf(false, TEXT("Invalid stage type (%s)"), *StageSource->GetClass()->GetName());
	}
}

void FDMMaterialStageSourceMenus::CreateChangeMaterialStageSource(FToolMenuSection& InSection)
{
	UDMMenuContext* MenuContext = InSection.FindContext<UDMMenuContext>();

	if (!ensure(MenuContext))
	{
		return;
	}

	UDMMaterialSlot* Slot = MenuContext->GetSlot();

	UDynamicMaterialModel* MaterialModel = MenuContext->GetPreviewModel();

	if (!ensure(MaterialModel))
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!ensure(ModelEditorOnlyData))
	{
		return;
	}

	const TArray<TStrongObjectPtr<UClass>>& Gradients = UDMMaterialStageGradient::GetAvailableGradients();

	InSection.AddMenuEntry("TextureSample",
		LOCTEXT("TextureSample", "Texture"),
		LOCTEXT("ChangeSourceTextureSampleTooltip", "Change the source of this stage to a texture."),
		GetDefault<UDMMaterialStageExpressionTextureSample>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToTextureSampleFromContext,
				MenuContext
			)
		)
	);

	InSection.AddMenuEntry("SolidColor",
		LOCTEXT("ChangeSourceColorRGB", "Solid Color"),
		LOCTEXT("ChangeSourceColorRGBTooltip", "Change the source of this stage to a Solid Color."),
		GetDefault<UDMMaterialValueFloat3RGB>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToSolidColorRGBFromContext,
				MenuContext
			)
		)
	);

	InSection.AddMenuEntry("TextureSample_EdgeColor",
		LOCTEXT("AddTextureSampleEgdeColor", "Texture Edge Color"),
		LOCTEXT("ChangeSourceTextureSampleEdgeColorTooltip", "Change the source of this stage to the edge color of a texture."),
		GetDefault<UDMMaterialStageExpressionTextureSampleEdgeColor>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToTextureSampleEdgeColorFromContext,
				MenuContext
			)
		)
	);

	if (ModelEditorOnlyData->GetDomain() == EMaterialDomain::MD_PostProcess)
	{
		InSection.AddMenuEntry("SceneTexture",
			LOCTEXT("AddSceneTexture", "Post Process"),
			LOCTEXT("ChangeSourceSceneTextureTooltip", "Change the source of this stage to Scene Texture in post process materials."),
			GetDefault<UDMMaterialStageExpressionSceneTexture>()->GetComponentIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(
					&ChangeSourceToSolidColorRGBFromContext,
					MenuContext
				),
				FCanExecuteAction::CreateStatic(
					&CanChangeSourceToSceneTextureFromContext,
					MenuContext
				)
			)
		);
	}

	InSection.AddMenuEntry("Noise",
		LOCTEXT("ChangeSourceNoise", "Noise"),
		LOCTEXT("ChangeSourceNoiseTooltip", "Change the source of this stage to a Noise Renderer."),
		GetDefault<UDMMaterialStageExpressionWorldPositionNoise>()->GetComponentIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(
				&ChangeSourceToNoiseFromContext,
				MenuContext
			)
		)
	);

	if constexpr (UE::DynamicMaterialEditor::bGlobalValuesEnabled)
	{
		const TArray<UDMMaterialValue*>& Values = MaterialModel->GetValues();

		InSection.AddSubMenu("NewLocalValue",
			LOCTEXT("ChangeSourceNewLocalValue", "New Local Value"),
			LOCTEXT("ChangeSourceNewLocalValueTooltip", "Add a new local Material Value and use it as the source of this stage."),
			FNewToolMenuDelegate::CreateStatic(&GenerateChangeSourceMenu_NewLocalValues)
		);

		if (!Values.IsEmpty())
		{
			InSection.AddSubMenu("GlobalValue",
				LOCTEXT("ChangeSourceValue", "Global Value"),
				LOCTEXT("ChangeSourceValueTooltip", "Change the source of this stage to a global Material Value."),
				FNewToolMenuDelegate::CreateStatic(&GenerateChangeSourceMenu_GlobalValues)
			);
		}

		InSection.AddSubMenu("NewLocalValue",
			LOCTEXT("ChangeSourceNewValue", "New Global Value"),
			LOCTEXT("ChangeSourceNewValueTooltip", "Add a new global Material Value and use it as the source of this stage."),
			FNewToolMenuDelegate::CreateStatic(&GenerateChangeSourceMenu_NewGlobalValues)
		);
	}

	if (!Gradients.IsEmpty())
	{
		InSection.AddSubMenu("Gradient",
			LOCTEXT("ChangeSourceGradient", "Gradient"),
			LOCTEXT("ChangeSourceGradientTooltip", "Change the source of this stage to a Material Gradient."),
			FNewToolMenuDelegate::CreateStatic(&GenerateChangeSourceMenu_Gradients)
		);
	}

	InSection.AddSubMenu("Advanced",
		LOCTEXT("ChangeSourceAdvanced", "Advanced"),
		LOCTEXT("ChangeSourceAdvancedTooltip", "Add an advanced Material Stage."),
		FNewToolMenuDelegate::CreateStatic(&GenerateChangeSourceMenu_Advanced)
	);
}

#undef LOCTEXT_NAMESPACE
