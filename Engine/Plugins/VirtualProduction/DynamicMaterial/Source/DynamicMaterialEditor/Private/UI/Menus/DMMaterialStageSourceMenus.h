// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/StrongObjectPtr.h"

class SDMMaterialSlotEditor;
class SDMMaterialStage;
class SWidget;
class UClass;
class UDMMaterialSlot;
class UDMMaterialStageExpression;
class UDMMenuContext;
class UToolMenu;
enum class EDMExpressionMenu : uint8;
struct FToolMenuSection;

class FDMMaterialStageSourceMenus final
{
public:
	/** Generate right click menu for changing sources */
	static TSharedRef<SWidget> MakeChangeSourceMenu(const TSharedPtr<SDMMaterialSlotEditor>& InSlotWidget, const TSharedPtr<SDMMaterialStage>& InStageWidget);

	static void CreateSourceMenuTree(TFunction<void(EDMExpressionMenu InMenu, TArray<UDMMaterialStageExpression*>& SubmenuExpressionList)> InCallback, 
		const TArray<TStrongObjectPtr<UClass>>& InAllExpressions);

	static void CreateChangeMaterialStageSource(FToolMenuSection& InSection);

private:
	static void GenerateChangeSourceMenu_NewLocalValues(UToolMenu* InMenu);

	static void GenerateChangeSourceMenu_GlobalValues(UToolMenu* InMenu);

	static void GenerateChangeSourceMenu_NewGlobalValues(UToolMenu* InMenu);

	static void GenerateChangeSourceMenu_Slot_Properties(UToolMenu* InMenu, UDMMaterialSlot* InSlot);

	static void GenerateChangeSourceMenu_Slots(UToolMenu* InMenu);

	static void GenerateChangeSourceMenu_Gradients(UToolMenu* const InMenu);

	static void GenerateChangeSourceMenu_Advanced(UToolMenu* const InMenu);

	static void ChangeSourceToTextureSampleFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToNoiseFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToSolidColorRGBFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToColorAtlasFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToTextureSampleEdgeColorFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToSceneTextureFromContext(UDMMenuContext* InMenuContext);

	static bool CanChangeSourceToSceneTextureFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToMaterialFunctionFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToTextFromContext(UDMMenuContext* InMenuContext);

	static void ChangeSourceToWidgetFromContext(UDMMenuContext* InMenuContext);
};
