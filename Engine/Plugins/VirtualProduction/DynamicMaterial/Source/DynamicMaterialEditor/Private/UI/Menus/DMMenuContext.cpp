// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMMenuContext.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageSource.h"
#include "Model/DynamicMaterialModel.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialStage.h"
#include "UI/Widgets/SDMMaterialEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMenuContext)

UDMMenuContext* UDMMenuContext::Create(const TWeakPtr<SDMMaterialEditor>& InEditorWidget, const TWeakPtr<SDMMaterialStage>& InStageWidget, UDMMaterialLayerObject* InLayerObject)
{
	UDMMenuContext* Context = NewObject<UDMMenuContext>();
	Context->EditorWidgetWeak = InEditorWidget;
	Context->StageWidgetWeak = InStageWidget;
	Context->LayerObjectWeak = InLayerObject;
	return Context;
}

UToolMenu* UDMMenuContext::GenerateContextMenu(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget, const TWeakPtr<SDMMaterialStage>& InStageWidget, UDMMaterialLayerObject* InLayerObject)
{
	UToolMenu* NewMenu = UToolMenus::Get()->RegisterMenu(InMenuName, NAME_None, EMultiBoxType::Menu, false);

	if (!NewMenu)
	{
		return nullptr;
	}

	if (!IsValid(InLayerObject))
	{
		if (TSharedPtr<SDMMaterialStage> StageWidget = InStageWidget.Pin())
		{
			if (UDMMaterialStage* Stage = StageWidget->GetStage())
			{
				InLayerObject = Stage->GetLayer();
			}
		}
	}

	NewMenu->bToolBarForceSmallIcons = true;
	NewMenu->bShouldCloseWindowAfterMenuSelection = true;
	NewMenu->bCloseSelfOnly = true;

	TSharedPtr<FUICommandList> CommandList = nullptr;

	if (TSharedPtr<SDMMaterialEditor> Editor = InEditorWidget.Pin())
	{
		CommandList = Editor->GetCommandList();
	}

	NewMenu->Context = FToolMenuContext(CommandList, TSharedPtr<FExtender>(), Create(InEditorWidget, InStageWidget, InLayerObject));

	return NewMenu;
}

UDMMenuContext* UDMMenuContext::CreateEmpty()
{
	return Create(nullptr, nullptr, nullptr);
}

UDMMenuContext* UDMMenuContext::CreateEditor(const TWeakPtr<SDMMaterialEditor>& InEditorWidget)
{
	return Create(InEditorWidget, nullptr, nullptr);
}

UDMMenuContext* UDMMenuContext::CreateLayer(const TWeakPtr<SDMMaterialEditor>& InEditorWidget, UDMMaterialLayerObject* InLayerObject)
{
	return Create(InEditorWidget, nullptr, InLayerObject);
}

UDMMenuContext* UDMMenuContext::CreateStage(const TWeakPtr<SDMMaterialEditor>& InEditorWidget, const TWeakPtr<SDMMaterialStage>& InStageWidget)
{
	return Create(InEditorWidget, InStageWidget, nullptr);
}

UToolMenu* UDMMenuContext::GenerateContextMenuDefault(const FName InMenuName)
{
	return GenerateContextMenu(InMenuName, nullptr, nullptr, nullptr);
}

UToolMenu* UDMMenuContext::GenerateContextMenuEditor(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget)
{
	return GenerateContextMenu(InMenuName, InEditorWidget, nullptr, nullptr);
}

UToolMenu* UDMMenuContext::GenerateContextMenuLayer(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget, UDMMaterialLayerObject* InLayerObject)
{
	return GenerateContextMenu(InMenuName, InEditorWidget, nullptr, InLayerObject);
}

UToolMenu* UDMMenuContext::GenerateContextMenuStage(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget, const TWeakPtr<SDMMaterialStage>& InStageWidget)
{
	return GenerateContextMenu(InMenuName, InEditorWidget, InStageWidget, nullptr);
}

UDMMaterialSlot* UDMMenuContext::GetSlot() const
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		if (TSharedPtr<SDMMaterialSlotEditor> SlotWidget = EditorWidget->GetSlotEditorWidget())
		{
			return SlotWidget->GetSlot();
		}
	}

	return nullptr;;
}

UDynamicMaterialModelBase* UDMMenuContext::GetPreviewModelBase() const
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		return EditorWidget->GetPreviewMaterialModelBase();
	}

	return nullptr;
}

UDynamicMaterialModel* UDMMenuContext::GetPreviewModel() const
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		return EditorWidget->GetPreviewMaterialModel();
	}

	return nullptr;
}

UDMMaterialStage* UDMMenuContext::GetStage() const
{
	if (TSharedPtr<SDMMaterialStage> StageWidget = StageWidgetWeak.Pin())
	{
		return StageWidget->GetStage();
	}

	return nullptr;
}

UDMMaterialStageSource* UDMMenuContext::GetStageSource() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		return Stage->GetSource();
	}

	return nullptr;
}

UDMMaterialStageBlend* UDMMenuContext::GetStageSourceAsBlend() const
{
	return Cast<UDMMaterialStageBlend>(GetStageSource());
}

UDMMaterialLayerObject* UDMMenuContext::GetLayer() const
{
	return LayerObjectWeak.Get();
}
