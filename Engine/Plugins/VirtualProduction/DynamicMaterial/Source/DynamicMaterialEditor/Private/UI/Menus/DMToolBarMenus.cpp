// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Menus/DMToolBarMenus.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialEditorSettings.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "IContentBrowserSingleton.h"
#include "ISinglePropertyView.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMenuContext.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Utils/DMMaterialSnapshotLibrary.h"
#include "Utils/DMPrivate.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "FDMToolBarMenus"

namespace UE::DynamicMaterialEditor::Private
{
	static FName ToolBarEditorLayoutMenuName = TEXT("MaterialDesigner.EditorLayout");
	static FName ToolBarMaterialExportSectionName = TEXT("MaterialExport");
	static FName ToolBarMaterialDesignerSettingsSectionName = TEXT("MaterialDesignerSettings");
}

TSharedRef<SWidget> FDMToolBarMenus::MakeEditorLayoutMenu(const TSharedPtr<SDMMaterialEditor>& InEditorWidget)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(ToolBarEditorLayoutMenuName))
	{
		UToolMenu* const NewToolMenu = UDMMenuContext::GenerateContextMenuDefault(ToolBarEditorLayoutMenuName);

		if (!NewToolMenu)
		{
			return SNullWidget::NullWidget;
		}

		NewToolMenu->AddDynamicSection(
			TEXT("MaterialDesignerSettings"),
			FNewToolMenuDelegate::CreateStatic(&AddMenu)
		);
	}

	FToolMenuContext MenuContext(
		FDynamicMaterialEditorModule::Get().GetCommandList(),
		TSharedPtr<FExtender>(),
		UDMMenuContext::CreateEditor(InEditorWidget)
	);

	return ToolMenus->GenerateWidget(ToolBarEditorLayoutMenuName, MenuContext);
}

void FDMToolBarMenus::AddMenu(UToolMenu* InMenu)
{
	AddExportMenu(InMenu);
	AddSettingsMenu(InMenu);
}

void FDMToolBarMenus::AddExportMenu(UToolMenu* InMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!IsValid(InMenu) || InMenu->ContainsSection(ToolBarMaterialExportSectionName))
	{
		return;
	}

	UDMMenuContext* const MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	UDynamicMaterialModelBase* const MaterialModelBase = MenuContext->GetPreviewModelBase();

	if (!MaterialModelBase)
	{
		return;
	}

	UDynamicMaterialInstance* Material = MaterialModelBase->GetDynamicMaterialInstance();

	if (!Material)
	{
		return;
	}

	const bool bAllowGeneratedMaterialExport = IsValid(MaterialModelBase->GetGeneratedMaterial());

	FToolMenuSection& NewSection = InMenu->AddSection(ToolBarMaterialExportSectionName, LOCTEXT("ExportSection", "Export"));

	NewSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("OpenInUEMaterialEditor", "Open Generated Material"),
		LOCTEXT("OpenInUEMaterialEditorTooltip", "Opens the Generated Material in the standard Material Editor."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateStatic(
			&OpenMaterialEditorFromContext,
			MenuContext
		))
	);

	if (bAllowGeneratedMaterialExport)
	{
		NewSection.AddMenuEntry(
			NAME_None,
			LOCTEXT("ExportGeneratedMaterial", "Export Generated Material"),
			LOCTEXT("ExportGeneratedMaterialTooltip", "Export the Generated Material as an asset."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateStatic(
				&FDMToolBarMenus::ExportMaterialModel,
				TWeakObjectPtr<UDynamicMaterialModelBase>(MaterialModelBase)
			))
		);
	}

	NewSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("ExportMaterial", "Save As new Material Designer Asset"),
		LOCTEXT("ExportMaterialInstanceTooltip", "Export the Material Designer Material as a new asset."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateStatic(
			&FDMToolBarMenus::ExportMaterial,
			TWeakObjectPtr<UDynamicMaterialInstance>(Material)
		))
	);

	NewSection.AddSubMenu(
		NAME_None,
		LOCTEXT("SnapshotMaterial", "Snapshop Material Designer Material"),
		LOCTEXT("SnapshotMaterialTooltip", "Take a snapshot of the Material Designer Material with the current values and export it as a Texture asset."),
		FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(
			&FDMToolBarMenus::CreateSnapshotMaterialMenu
		))
	);
}

void FDMToolBarMenus::AddSettingsMenu(UToolMenu* InMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (!IsValid(InMenu) || InMenu->ContainsSection(ToolBarMaterialDesignerSettingsSectionName))
	{
		return;
	}

	FToolMenuSection& NewSection = InMenu->AddSection(ToolBarMaterialDesignerSettingsSectionName, LOCTEXT("MaterialDesignerSection", "Material Designer"));

	NewSection.AddSubMenu(
		"AdvancedSettings",
		LOCTEXT("AdvancedSettingsSubMenu", "Advanced Settings"),
		LOCTEXT("AdvancedSettingsSubMenu_ToolTip", "Display advanced Material Designer settings"),
		FNewToolMenuDelegate::CreateStatic(&AddAdvancedSection)
	);

	NewSection.AddSubMenu(
		"EditorLayout",
		LOCTEXT("EditorLayoutSubMenu", "Editor Layout"),
		LOCTEXT("EditorLayoutSubMenu_ToolTip", "Change the layout of the Material Designer Editor"),
		FNewToolMenuDelegate::CreateStatic(&AddEditorLayoutSection)
	);

	NewSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("OpenSettings", "Material Designer Editor Settings"),
		LOCTEXT("OpenSettingsTooltip", "Opens the Editor Settings and navigates to Material Designer section."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FoliageEditMode.Settings"),
		FUIAction(FExecuteAction::CreateUObject(
			UDynamicMaterialEditorSettings::Get(),
			&UDynamicMaterialEditorSettings::OpenEditorSettingsWindow)
		)
	);
}

void FDMToolBarMenus::AddEditorLayoutSection(UToolMenu* InMenu)
{
	FToolMenuSection& NewSection = InMenu->AddSection(TEXT("EditorLayout"), LOCTEXT("EditorLayoutSection", "EditorLayout"));

	UEnum* LayoutEnum = StaticEnum<EDMMaterialEditorLayout>();

	for (EDMMaterialEditorLayout Layout = EDMMaterialEditorLayout::First;
		Layout <= EDMMaterialEditorLayout::Last;
		Layout = static_cast<EDMMaterialEditorLayout>(static_cast<uint8>(Layout)+1))
	{
		FUIAction Action;

		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Layout]()
			{
				if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
				{
					if (Settings->Layout == Layout)
					{
						return ECheckBoxState::Checked;
					}
				}

				return ECheckBoxState::Unchecked;
			}
		);

		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Layout]()
			{
				UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>();

				if (!Settings || Settings->Layout == Layout)
				{
					return;
				}

				Settings->Layout = Layout;
				TArray<UObject*> TopLeftObjects = {Settings};

				FPropertyChangedEvent PropertyChangedEvent(
					UDynamicMaterialEditorSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, Layout)),
					EPropertyChangeType::Interactive,
					TopLeftObjects
				);

				Settings->PostEditChangeProperty(PropertyChangedEvent);
			}
		);

		NewSection.AddMenuEntry(
			NAME_None,
			LayoutEnum->GetDisplayNameTextByValue(static_cast<int64>(Layout)),
			FText::GetEmpty(),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::RadioButton
		);
	}
}

void FDMToolBarMenus::AddAdvancedSection(UToolMenu* InMenu)
{
	FToolMenuSection& NewSection = InMenu->AddSection(TEXT("AdvancedSettings"), LOCTEXT("AdvancedSettingsSection", "Advanced Settings"));

	NewSection.AddMenuEntry(
		NAME_None,
		LOCTEXT("ResetAllSettingsToDefaults", "Reset UI Settings"),
		LOCTEXT("ResetAllSettingsToDefaultsTooltip", "Resets all the Material Designer UI settings to their default values."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateUObject(
			UDynamicMaterialEditorSettings::Get(), 
			&UDynamicMaterialEditorSettings::ResetAllLayoutSettings)
		)
	);
}

void FDMToolBarMenus::OpenMaterialEditorFromContext(UDMMenuContext* InMenuContext)
{
	if (IsValid(InMenuContext))
	{
		if (UDynamicMaterialModelEditorOnlyData* const ModelEditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(InMenuContext->GetPreviewModel()))
		{
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("OpenedGeneratedMaterial"));
			}

			ModelEditorOnlyData->OpenMaterialEditor();
		}
	}
}

void FDMToolBarMenus::ExportMaterial(TWeakObjectPtr<UDynamicMaterialInstance> InMaterialInstanceWeak)
{
	if (UDynamicMaterialInstance* Material = InMaterialInstanceWeak.Get())
	{
		const FString CurrentName = TEXT("MD_") + UDMMaterialModelFunctionLibrary::RemoveAssetPrefix(Material->GetName());

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		FString PackageName, AssetName;
		AssetTools.CreateUniqueAssetName(CurrentName, TEXT(""), PackageName, AssetName);

		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
		const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");

		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
		SaveAssetDialogConfig.DefaultPath = PathStr;
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
		SaveAssetDialogConfig.DefaultAssetName = AssetName;

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

		if (!SaveObjectPath.IsEmpty())
		{
			UDMMaterialModelFunctionLibrary::ExportMaterial(Material->GetMaterialModelBase(), SaveObjectPath);

			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("ExportedMaterial"));
			}
		}
	}
}

void FDMToolBarMenus::ExportMaterialModel(TWeakObjectPtr<UDynamicMaterialModelBase> InMaterialModelBaseWeak)
{
	UDynamicMaterialModelBase* MaterialModelBase = InMaterialModelBaseWeak.Get();

	if (!MaterialModelBase)
	{
		return;
	}

	UMaterial* GeneratedMaterial = MaterialModelBase->GetGeneratedMaterial();

	if (!GeneratedMaterial)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to find a generated material to export."), true, MaterialModelBase);
		return;
	}

	UDynamicMaterialInstance* Material = MaterialModelBase->GetDynamicMaterialInstance();

	const FString CurrentName = TEXT("M_") + UDMMaterialModelFunctionLibrary::RemoveAssetPrefix(Material ? Material->GetName() : MaterialModelBase->GetName());

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.DefaultAssetName = CurrentName;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return;
	}

	UDMMaterialModelFunctionLibrary::ExportGeneratedMaterial(MaterialModelBase, SaveObjectPath);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("ExportedGeneratedMaterial"));
	}
}

void FDMToolBarMenus::SnapshotMaterial(TWeakObjectPtr<UDynamicMaterialModelBase> InMaterialModelBaseWeak, FIntPoint InTextureSize)
{
	UDynamicMaterialModelBase* MaterialModelBase = InMaterialModelBaseWeak.Get();

	if (!IsValid(MaterialModelBase))
	{
		return;
	}

	UMaterialInterface* Material = MaterialModelBase->GetGeneratedMaterial();
	UDynamicMaterialInstance* MaterialInstance = MaterialModelBase->GetDynamicMaterialInstance();

	if (MaterialInstance)
	{
		if (!IsValid(MaterialInstance->Parent.Get()))
		{
			UE_LOGF(LogDynamicMaterialEditor, Warning, "Unable to find world to find material parent.");
			return;
		}

		Material = MaterialInstance;
	}

	if (!Material)
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "Unable to find material to snapshot.");
		return;
	}

	const FString CurrentName = TEXT("T_")
		+ UDMMaterialModelFunctionLibrary::RemoveAssetPrefix(MaterialInstance ? MaterialInstance->GetName() : MaterialModelBase->GetName())
		+ TEXT("_")
		+ FString::FromInt(InTextureSize.X)
		+ TEXT("x")
		+ FString::FromInt(InTextureSize.Y);

	// Choose asset location
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : "/Game";

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("ExportMaterialTo", "Export Material To");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = CurrentName;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return;
	}

	FDMMaterialShapshotLibrary::SnapshotMaterial(Material, InTextureSize, SaveObjectPath);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("SnapshotMaterial"));
	}
}

void FDMToolBarMenus::AddBoolOptionMenuEntry(FToolMenuSection& InSection, const FName& InPropertyName, const FUIAction InAction)
{
	const FProperty* const OptionProperty = UDynamicMaterialEditorSettings::StaticClass()->FindPropertyByName(InPropertyName);

	if (ensure(OptionProperty))
	{
		InSection.AddMenuEntry(
			NAME_None,
			OptionProperty->GetDisplayNameText(),
			OptionProperty->GetToolTipText(),
			FSlateIcon(),
			InAction, EUserInterfaceActionType::ToggleButton
		);
	}
}

void FDMToolBarMenus::AddIntOptionMenuEntry(FToolMenuSection& InSection, const FName& InPropertyName,
	TAttribute<bool> InIsEnabledAttribute, TAttribute<EVisibility> InVisibilityAttribute)
{
	InSection.AddDynamicEntry(NAME_None,
		FNewToolMenuSectionDelegate::CreateLambda(
			[InPropertyName, InIsEnabledAttribute, InVisibilityAttribute](FToolMenuSection& InSection)
			{
				const FProperty* const OptionProperty = UDynamicMaterialEditorSettings::StaticClass()->FindPropertyByName(InPropertyName);
				FText DisplayName = FText::GetEmpty();
				FText Tooltip = FText::GetEmpty();

				if (ensure(OptionProperty))
				{
					DisplayName = OptionProperty->GetDisplayNameText();
					Tooltip = OptionProperty->GetToolTipText();
				}

				FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

				FSinglePropertyParams SinglePropertyParams;
				SinglePropertyParams.NamePlacement = EPropertyNamePlacement::Hidden;

				TSharedRef<ISinglePropertyView> SinglePropertyView = PropertyEditor.CreateSingleProperty(UDynamicMaterialEditorSettings::Get(), InPropertyName, SinglePropertyParams).ToSharedRef();
				SinglePropertyView->SetToolTipText(Tooltip);
				SinglePropertyView->SetEnabled(InIsEnabledAttribute);
				SinglePropertyView->SetVisibility(InVisibilityAttribute);

				InSection.AddEntry(FToolMenuEntry::InitWidget(NAME_None,
					SNew(SBox)
					.HAlign(HAlign_Fill)
					[
						SNew(SBox)
							.WidthOverride(80.0f)
							.HAlign(HAlign_Right)
							[
								SinglePropertyView
							]
					],
					DisplayName));
			})
	);
}

void FDMToolBarMenus::CreateSnapshotMaterialMenu(UToolMenu* InMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	const UDMMenuContext* const MenuContext = InMenu->FindContext<UDMMenuContext>();

	if (!MenuContext)
	{
		return;
	}

	UDynamicMaterialModelBase* const MaterialModelBase = MenuContext->GetPreviewModelBase();

	if (!MaterialModelBase)
	{
		return;
	}

	TWeakObjectPtr<UDynamicMaterialModelBase> MaterialModelWeak = MaterialModelBase;
	FToolMenuSection& NewSection = InMenu->AddSection("SnapshotMaterial", LOCTEXT("SnapshotMaterialSection", "Snapshop Material"));

	const FText SnapshotNameFormat = LOCTEXT("SnapshotName", "{0}x{0}");

	const FText SnapshotTooltipFormat = LOCTEXT(
		"SnapshotMaterialMenuEntryTooltip",
		"Take a snapshot of the Material Designer Material with the current values and export it as a Texture asset with a resolution of {0} pixels."
	);

	const TArray<int32> SnapshotTypeResolutions = {512, 1024, 2048, 4096};

	for (int32 SnapshotTypeResolution : SnapshotTypeResolutions)
	{
		const FText Name = FText::Format(SnapshotNameFormat, FText::AsNumber(SnapshotTypeResolution));

		NewSection.AddMenuEntry(
			NAME_None,
			Name,
			FText::Format(SnapshotTooltipFormat, Name),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateStatic(
				&FDMToolBarMenus::SnapshotMaterial,
				MaterialModelWeak,
				FIntPoint(SnapshotTypeResolution, SnapshotTypeResolution)
			))
		);
	}
}

#undef LOCTEXT_NAMESPACE
