// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Math/MathFwd.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointerFwd.h"

class FName;
class SDMMaterialEditor;
class SWidget;
class UDMMenuContext;
class UDynamicMaterialInstance;
class UDynamicMaterialModelBase;
class UToolMenu;
enum class EDMMaterialEditorLayout : uint8;
struct FToolMenuSection;
struct FUIAction;

class FDMToolBarMenus final
{
public:
	static TSharedRef<SWidget> MakeEditorLayoutMenu(const TSharedPtr<SDMMaterialEditor>& InEditorWidget = nullptr);

private:
	static void AddMenu(UToolMenu* InMenu);

	static void AddExportMenu(UToolMenu* InMenu);

	static void AddSettingsMenu(UToolMenu* InMenu);

	static void AddEditorLayoutSection(UToolMenu* InMenu);

	static void AddAdvancedSection(UToolMenu* InMenu);

	static void AddBoolOptionMenuEntry(FToolMenuSection& InSection, const FName& InPropertyName, const FUIAction InAction);

	static void AddIntOptionMenuEntry(FToolMenuSection& InSection, const FName& InPropertyName,
		TAttribute<bool> InIsEnabledAttribute = TAttribute<bool>(),
		TAttribute<EVisibility> InVisibilityAttribute = TAttribute<EVisibility>());

	static void OpenMaterialEditorFromContext(UDMMenuContext* InMenuContext);

	static void ExportMaterial(TWeakObjectPtr<UDynamicMaterialInstance> InMaterialInstanceWeak);

	static void ExportMaterialModel(TWeakObjectPtr<UDynamicMaterialModelBase> InMaterialModelBaseWeak);

	static void SnapshotMaterial(TWeakObjectPtr<UDynamicMaterialModelBase> InMaterialModelBaseWeak, FIntPoint InTextureSize);

	static void CreateSnapshotMaterialMenu(UToolMenu* InMenu);
};
