// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "DynamicMaterialEditorModule.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Runtime/Launch/Resources/Version.h"
#include "UI/Utils/DMWidgetLibrary.h"

const TSharedRef<FDMComponentPropertyRowGenerator>& FDMComponentPropertyRowGenerator::Get()
{
	static TSharedRef<FDMComponentPropertyRowGenerator> Generator = MakeShared<FDMComponentPropertyRowGenerator>();
	return Generator;
}

void FDMComponentPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	UDMMaterialComponent* Component = Cast<UDMMaterialComponent>(InParams.Object);

	if (!Component)
	{
		return;
	}

	const TArray<FName>& Properties = Component->GetEditableProperties();

	for (const FName& Property : Properties)
	{
		if (Component->IsPropertyVisible(Property))
		{
			AddPropertyEditRows(InParams, Property);
		}
	}
}

void FDMComponentPropertyRowGenerator::AddPropertyEditRows(FDMComponentPropertyRowGeneratorParams& InParams, const FName& InProperty)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	FProperty* Property = InParams.Object->GetClass()->FindPropertyByName(InProperty);

	if (!Property)
	{
		return;
	}

	void* MemoryPtr = Property->ContainerPtrToValuePtr<void>(InParams.Object);

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, MemoryPtr);

		for (int32 Idx = 0; Idx < ArrayHelper.Num(); ++Idx)
		{
#if ENGINE_MAJOR_VERSION < 5 || (ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION <= 2)
			void* ElemPtr = ArrayHelper.GetRawPtr(Idx);
#else
			void* ElemPtr = ArrayHelper.GetElementPtr(Idx);
#endif
			AddPropertyEditRows(InParams, ArrayProperty->Inner, ElemPtr);
		}
	}
	else
	{
		AddPropertyEditRows(InParams, Property, MemoryPtr);
	}
}

void FDMComponentPropertyRowGenerator::AddPropertyEditRows(FDMComponentPropertyRowGeneratorParams& InParams, FProperty* InProperty, 
	void* InMemoryPtr)
{
	if (InProperty->IsA<FArrayProperty>())
	{
		return;
	}

	if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty))
	{
		if (ObjectProperty->PropertyClass->IsChildOf(UDMMaterialComponent::StaticClass()))
		{
			UObject** ValuePtr = static_cast<UObject**>(InMemoryPtr);
			UObject* Value = *ValuePtr;
			UDMMaterialComponent* ComponentValue = Cast<UDMMaterialComponent>(Value);

			FDMComponentPropertyRowGeneratorParams ChildParams = InParams;
			ChildParams.Object = ComponentValue;

			FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(ChildParams);
			return;
		}
	}

	FDMPropertyHandle& Handle = InParams.PropertyRows->Add_GetRef(FDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(InProperty->GetFName())));
	Handle.bEnabled = !IsDynamic(InParams);
}

bool FDMComponentPropertyRowGenerator::AllowKeyframeButton(UDMMaterialComponent* InComponent, FProperty* InProperty)
{
	return false;
}

bool FDMComponentPropertyRowGenerator::IsDynamic(FDMComponentPropertyRowGeneratorParams& InParams)
{
	return InParams.PreviewMaterialModelBase && InParams.PreviewMaterialModelBase->IsA<UDynamicMaterialModelDynamic>();
}
