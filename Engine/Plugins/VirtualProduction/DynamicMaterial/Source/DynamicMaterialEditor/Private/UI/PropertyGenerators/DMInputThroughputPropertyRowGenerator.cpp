// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMInputThroughputPropertyRowGenerator.h"

#include "Components/DMMaterialStageThroughput.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "IDynamicMaterialEditorModule.h"

const TSharedRef<FDMInputThroughputPropertyRowGenerator>& FDMInputThroughputPropertyRowGenerator::Get()
{
	static TSharedRef<FDMInputThroughputPropertyRowGenerator> Generator = MakeShared<FDMInputThroughputPropertyRowGenerator>();
	return Generator;
}

void FDMInputThroughputPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialStageInputThroughput* InputThroughput = Cast<UDMMaterialStageInputThroughput>(InParams.Object);

	if (!InputThroughput)
	{
		return;
	}

	InParams.ProcessedObjects.Add(InputThroughput);

	if (UDMMaterialSubStage* SubStage = InputThroughput->GetSubStage())
	{
		InParams.ProcessedObjects.Add(SubStage);
	}

	UDMMaterialStageThroughput* Throughput = InputThroughput->GetMaterialStageThroughput();

	if (!Throughput)
	{
		return;
	}

	FDMComponentPropertyRowGeneratorParams ThroughputParams = InParams;
	ThroughputParams.Object = Throughput;

	FDMThroughputPropertyRowGenerator::AddComponentProperties(ThroughputParams);
}
