// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/PropertyGenerators/DMThroughputPropertyRowGenerator.h"

class FDMInputThroughputPropertyRowGenerator : public FDMThroughputPropertyRowGenerator
{
public:
	static const TSharedRef<FDMInputThroughputPropertyRowGenerator>& Get();

	virtual ~FDMInputThroughputPropertyRowGenerator() override = default;

	//~ Begin FDMComponentPropertyRowGenerator
	virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams) override;
	//~ End FDMComponentPropertyRowGenerator
};
