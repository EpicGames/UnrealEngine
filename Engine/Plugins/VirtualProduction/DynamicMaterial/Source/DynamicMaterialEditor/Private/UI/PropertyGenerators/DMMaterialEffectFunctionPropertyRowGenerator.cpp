// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMMaterialEffectFunctionPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialEffectFunction.h"
#include "Components/DMMaterialValue.h"
#include "DynamicMaterialEditorModule.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Utils/DMMaterialFunctionFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "DMMaterialEffectFunctionPropertyRowGenerator"

const TSharedRef<FDMMaterialEffectFunctionPropertyRowGenerator>& FDMMaterialEffectFunctionPropertyRowGenerator::Get()
{
	static TSharedRef<FDMMaterialEffectFunctionPropertyRowGenerator> Generator = MakeShared<FDMMaterialEffectFunctionPropertyRowGenerator>();
	return Generator;
}

void FDMMaterialEffectFunctionPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialEffectFunction* EffectFunction = Cast<UDMMaterialEffectFunction>(InParams.Object);

	if (!EffectFunction)
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	UMaterialFunctionInterface* MaterialFunction = EffectFunction->GetMaterialFunction();

	if (!IsValid(MaterialFunction))
	{
		return;
	}

	TArray<FFunctionExpressionInput> Inputs;
	TArray<FFunctionExpressionOutput> Outputs;
	MaterialFunction->GetInputsAndOutputs(Inputs, Outputs);

	if (Inputs.Num() != EffectFunction->GetInputValues().Num())
	{
		return;
	}

	const TArray<TObjectPtr<UDMMaterialValue>>& InputValues = EffectFunction->GetInputValues();

	for (int32 InputIndex = 0; InputIndex < InputValues.Num(); ++InputIndex)
	{
		UDMMaterialValue* Value = InputValues[InputIndex].Get();

		if (!IsValid(Value))
		{
			continue;
		}

		if (!Inputs[InputIndex].ExpressionInput)
		{
			continue;
		}

		TArray<FDMPropertyHandle> ValuePropertyRows;

		FDMComponentPropertyRowGeneratorParams ValueParams = InParams;
		ValueParams.Object = Value;
		ValueParams.PropertyRows = &ValuePropertyRows;

		FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(ValueParams);

		if (ValuePropertyRows.Num() == 1)
		{
			ValuePropertyRows[0].NameOverride = FText::FromName(Inputs[InputIndex].ExpressionInput->InputName);
		}
		else
		{
			static const FText NameFormat = LOCTEXT("ValueFormat", "{0}[{1}]");

			for (int32 ValuePropertyIndex = 0; ValuePropertyIndex < ValuePropertyRows.Num(); ++ValuePropertyIndex)
			{
				ValuePropertyRows[ValuePropertyIndex].NameOverride = FText::Format(
					NameFormat,
					FText::FromName(Inputs[InputIndex].ExpressionInput->InputName),
					FText::AsNumber(ValuePropertyIndex + 1)
				);
			}
		}

		const FText Description = FText::FromString(Inputs[InputIndex].ExpressionInput->Description);

		const FText MaterialInputFormat = LOCTEXT("MaterialInputFormat", "{0} Inputs");
		const FText MaterialInputText = FText::Format(MaterialInputFormat, EffectFunction->GetEffectName());
		const FName MaterialInputName = *MaterialInputText.ToString();

		for (FDMPropertyHandle& ValuePropertyRow : ValuePropertyRows)
		{
			ValuePropertyRow.NameToolTipOverride = Description;
			ValuePropertyRow.CategoryOverrideName = MaterialInputName;

			if (ValuePropertyRow.PreviewHandle.PropertyHandle.IsValid())
			{
				UDMMaterialFunctionFunctionLibrary::ApplyMetaData(
					Inputs[InputIndex],
					ValuePropertyRow.PreviewHandle.PropertyHandle.ToSharedRef()
				);
			}

		}

		InParams.PropertyRows->Append(ValuePropertyRows);
	}
}

#undef LOCTEXT_NAMESPACE
