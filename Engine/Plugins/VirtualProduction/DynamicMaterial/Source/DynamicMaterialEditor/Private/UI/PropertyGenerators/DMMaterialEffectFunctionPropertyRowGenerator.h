// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"

class FDMMaterialEffectFunctionPropertyRowGenerator : public FDMComponentPropertyRowGenerator
{
public:
	static const TSharedRef<FDMMaterialEffectFunctionPropertyRowGenerator>& Get();

	virtual ~FDMMaterialEffectFunctionPropertyRowGenerator() override = default;

	//~ Begin FDMComponentPropertyRowGenerator
	virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams) override;
	//~ End FDMComponentPropertyRowGenerator
};
