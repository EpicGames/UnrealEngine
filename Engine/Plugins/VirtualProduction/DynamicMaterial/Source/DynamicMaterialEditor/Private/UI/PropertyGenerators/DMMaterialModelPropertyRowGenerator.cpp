// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMMaterialModelPropertyRowGenerator.h"

#include "Components/DMMaterialProperty.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "IDynamicMaterialEditorModule.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UI/Utils/DMWidgetLibrary.h"

#define LOCTEXT_NAMESPACE "DMMaterialModelPropertyRowGenerator"

void FDMMaterialModelPropertyRowGenerator::AddMaterialModelProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	UDynamicMaterialModelBase* MaterialModelBase = Cast<UDynamicMaterialModelBase>(InParams.Object);

	if (!MaterialModelBase)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	AddGlobalValue(InParams, 
		MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOffsetValueName), 
		LOCTEXT("GlobalOffset", "Global Offset"));

	AddGlobalValue(InParams,
		MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalTilingValueName),
		LOCTEXT("GlobalTiling", "Global Tiling"));

	AddGlobalValue(InParams,
		MaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalRotationValueName), 
		LOCTEXT("GlobalRotation", "Global Rotation"));

	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModelBase))
	{
		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, Domain));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, BlendMode));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, ShadingModel));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, UsageFlags));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, GeneralFlags));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, LightingFlags));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, TranslucencyFlags));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, OpacityMaskClipValue));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, MotionFlags));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, ForwardRendererFlags));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, DisplacementMagnitude));

		AddVariable(InParams, EditorOnlyData,
			GET_MEMBER_NAME_CHECKED(UDynamicMaterialModelEditorOnlyData, DisplacementCenter));
	}
}

void FDMMaterialModelPropertyRowGenerator::AddGlobalValue(FDMComponentPropertyRowGeneratorParams& InParams, UDMMaterialComponent* InComponent,
	const FText& InNameOverride)
{
	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(InParams.Object))
	{
		InComponent = MaterialModelDynamic->GetComponentDynamic(InComponent->GetFName());

		if (!InComponent)
		{
			return;
		}
	}

	FDMComponentPropertyRowGeneratorParams GlobalValueParams = InParams;
	GlobalValueParams.Object = InComponent;

	FDMPropertyHandle& ComponentHandle = InParams.PropertyRows->Add_GetRef(FDMWidgetLibrary::Get().GetPropertyHandle(
		GlobalValueParams.CreatePropertyHandleParams(UDMMaterialValue::ValueName)
	));

	ComponentHandle.CategoryOverrideName = *LOCTEXT("MaterialSettings", "Material Settings").ToString();
	ComponentHandle.NameOverride = InNameOverride;

	if (UDMMaterialValue* MaterialValue = Cast<UDMMaterialValue>(InComponent))
	{
		ComponentHandle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(MaterialValue, &UDMMaterialValue::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(MaterialValue, &UDMMaterialValue::ResetToDefault),
			/* Propagate to children */ false
		);
	}
	else if (UDMMaterialValueDynamic* MaterialValueDynamic = Cast<UDMMaterialValueDynamic>(InComponent))
	{
		ComponentHandle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(MaterialValueDynamic, &UDMMaterialValueDynamic::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(MaterialValueDynamic, &UDMMaterialValueDynamic::ResetToDefault),
			/* Propagate to children */ false
		);
	}
}

void FDMMaterialModelPropertyRowGenerator::AddVariable(FDMComponentPropertyRowGeneratorParams& InParams, UObject* InObject, FName InPropertyName)
{
	FDMComponentPropertyRowGeneratorParams VariableParams = InParams;
	VariableParams.Object = InObject;

	FDMPropertyHandle& ValueHandle = InParams.PropertyRows->Add_GetRef(FDMWidgetLibrary::Get().GetPropertyHandle(
		VariableParams.CreatePropertyHandleParams(InPropertyName)
	));

	ValueHandle.CategoryOverrideName = *LOCTEXT("MaterialType", "Material Type").ToString();
	ValueHandle.bEnabled = !IsDynamic(Cast<UDynamicMaterialModelBase>(InParams.Object));
}

bool FDMMaterialModelPropertyRowGenerator::IsDynamic(UDynamicMaterialModelBase* InMaterialModelBase)
{
	return !!Cast<UDynamicMaterialModelDynamic>(InMaterialModelBase);
}

#undef LOCTEXT_NAMESPACE
