// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"

class FName;
class FProperty;
class SDMMaterialGlobalSettingsEditor;
class UDMMaterialComponent;
class UDynamicMaterialModelBase;
class UDynamicMaterialModelEditorOnlyData;
enum class EDMMaterialPropertyType : uint8;
struct FDMComponentPropertyRowGeneratorParams;
struct FDMPropertyHandle;

/** Used to generate editable properties for material models. */
class FDMMaterialModelPropertyRowGenerator
{
public:
	/**
	 * Generate properties for the given component.
	 * @param InGlobalSettingEditorWidget The edit widget generating the properties.
	 * @param InMaterialModelBase The model being edited.
	 * @param InOutPropertyRows The generated rows.
	 */
	static void AddMaterialModelProperties(FDMComponentPropertyRowGeneratorParams& InParams);

	static void AddGlobalValue(FDMComponentPropertyRowGeneratorParams& InParams, UDMMaterialComponent* InComponent,
		const FText& InNameOverride);

	static void AddVariable(FDMComponentPropertyRowGeneratorParams& InParams, UObject* InObject, FName InPropertyName);

protected:
	/** Returns true if the component edit widget is editing a Material Designer Dynamic. */
	static bool IsDynamic(UDynamicMaterialModelBase* InMaterialModelBase);
};
