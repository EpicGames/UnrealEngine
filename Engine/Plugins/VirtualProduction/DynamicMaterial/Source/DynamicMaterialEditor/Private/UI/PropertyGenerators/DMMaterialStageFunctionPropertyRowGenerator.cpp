// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMMaterialStageFunctionPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialStageFunction.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialStageInputs/DMMSIFunction.h"
#include "DynamicMaterialEditorModule.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Utils/DMMaterialFunctionFunctionLibrary.h"

#define LOCTEXT_NAMESPACE "DMMaterialStageFunctionPropertyRowGenerator"

const TSharedRef<FDMMaterialStageFunctionPropertyRowGenerator>& FDMMaterialStageFunctionPropertyRowGenerator::Get()
{
	static TSharedRef<FDMMaterialStageFunctionPropertyRowGenerator> Generator = MakeShared<FDMMaterialStageFunctionPropertyRowGenerator>();
	return Generator;
}

void FDMMaterialStageFunctionPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialStageInputFunction* StageInputFunction = Cast<UDMMaterialStageInputFunction>(InParams.Object);

	if (!StageInputFunction)
	{
		return;
	}

	UDMMaterialStageFunction* MaterialStageFunction = StageInputFunction->GetMaterialStageFunction();

	if (!MaterialStageFunction)
	{
		return;
	}

	TArray<FDMPropertyHandle> AllValuePropertyRows;

	InParams.ProcessedObjects.Add(InParams.Object);

	if (UMaterialFunctionInterface* MaterialFunction = MaterialStageFunction->GetMaterialFunction())
	{
		TArray<FFunctionExpressionInput> Inputs;
		TArray<FFunctionExpressionOutput> Outputs;
		MaterialFunction->GetInputsAndOutputs(Inputs, Outputs);

		TArray<UDMMaterialValue*> InputValues = MaterialStageFunction->GetInputValues();

		// 1 input, the previous stage, does not have a value.
		if (Inputs.Num() != (InputValues.Num() + 1))
		{
			return;
		}

		for (int32 InputIndex = 0; InputIndex < InputValues.Num(); ++InputIndex)
		{
			FFunctionExpressionInput& Input = Inputs[InputIndex + 1];
			UDMMaterialValue* Value = InputValues[InputIndex];

			if (!IsValid(Value))
			{
				continue;
			}

			if (!Input.ExpressionInput)
			{
				continue;
			}

			TArray<FDMPropertyHandle> ValuePropertyRows;

			FDMComponentPropertyRowGeneratorParams ValueParams = InParams;
			ValueParams.Object = Value;
			ValueParams.PropertyRows = &ValuePropertyRows;

			FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(ValueParams);

			if (ValuePropertyRows.Num() == 1)
			{
				ValuePropertyRows[0].NameOverride = FText::FromName(Input.ExpressionInput->InputName);
			}
			else
			{
				static const FText NameFormat = LOCTEXT("ValueFormat", "{0}[{1}]");

				for (int32 ValuePropertyIndex = 0; ValuePropertyIndex < ValuePropertyRows.Num(); ++ValuePropertyIndex)
				{
					ValuePropertyRows[ValuePropertyIndex].NameOverride = FText::Format(
						NameFormat,
						FText::FromName(Input.ExpressionInput->InputName),
						FText::AsNumber(ValuePropertyIndex + 1)
					);
				}
			}

			const FText Description = FText::FromString(Input.ExpressionInput->Description);

			FText MaterialInputText = FText::FromString(MaterialFunction->GetUserExposedCaption());

			if (MaterialInputText.IsEmpty())
			{
				MaterialInputText = LOCTEXT("Function", "Function");
			}

			const FText MaterialInputFormat = LOCTEXT("MaterialInputFormat", "{0} Inputs");
			MaterialInputText = FText::Format(MaterialInputFormat, MaterialInputText);
			const FName MaterialInputName = *MaterialInputText.ToString();

			for (FDMPropertyHandle& ValuePropertyRow : ValuePropertyRows)
			{
				ValuePropertyRow.NameToolTipOverride = Description;
				ValuePropertyRow.CategoryOverrideName = MaterialInputName;

				if (ValuePropertyRow.PreviewHandle.PropertyHandle.IsValid())
				{
					// The first input is the previous layer, so it does not have a value.
					UDMMaterialFunctionFunctionLibrary::ApplyMetaData(
						Inputs[InputIndex + 1],
						ValuePropertyRow.PreviewHandle.PropertyHandle.ToSharedRef()
					);
				}
			}

			AllValuePropertyRows.Append(ValuePropertyRows);
		}
	}

	FDMComponentPropertyRowGeneratorParams FunctionParams = InParams;
	FunctionParams.Object = MaterialStageFunction;

	FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(FunctionParams);

	InParams.PropertyRows->Append(AllValuePropertyRows);
}

#undef LOCTEXT_NAMESPACE
