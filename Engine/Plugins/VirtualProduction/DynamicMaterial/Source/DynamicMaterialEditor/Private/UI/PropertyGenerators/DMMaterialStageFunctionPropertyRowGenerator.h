// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"

class FDMMaterialStageFunctionPropertyRowGenerator : public FDMComponentPropertyRowGenerator
{
public:
	static const TSharedRef<FDMMaterialStageFunctionPropertyRowGenerator>& Get();

	virtual ~FDMMaterialStageFunctionPropertyRowGenerator() override = default;

	//~ Begin FDMComponentPropertyRowGenerator
	virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams) override;
	//~ End FDMComponentPropertyRowGenerator
};
