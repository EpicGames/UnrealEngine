// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMMaterialValueDynamicPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialValue.h"
#include "Components/DMMaterialValueDynamic.h"
#include "DMEDefs.h"
#include "IDetailPropertyRow.h"
#include "IDynamicMaterialEditorModule.h"
#include "UI/Utils/DMWidgetLibrary.h"

const TSharedRef<FDMMaterialValueDynamicPropertyRowGenerator>& FDMMaterialValueDynamicPropertyRowGenerator::Get()
{
	static TSharedRef<FDMMaterialValueDynamicPropertyRowGenerator> Generator = MakeShared<FDMMaterialValueDynamicPropertyRowGenerator>();
	return Generator;
}

void FDMMaterialValueDynamicPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialValueDynamic* ValueDynamic = Cast<UDMMaterialValueDynamic>(InParams.Object);

	if (!ValueDynamic)
	{
		return;
	}

	// The base material value class is abstract and not allowed.
	if (ValueDynamic->GetClass() == UDMMaterialValueDynamic::StaticClass())
	{
		return;
	}

	UDMMaterialValue* ParentValue = ValueDynamic->GetParentValue();

	if (!ParentValue)
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	if (ParentValue->AllowEditValue())
	{
		FDMPropertyHandleGenerateParams Params;
		Params.Widget = InParams.Owner;
		FDMPropertyHandle Handle = FDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(UDMMaterialValue::ValueName));

		Handle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(ValueDynamic, &UDMMaterialValueDynamic::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(ValueDynamic, &UDMMaterialValueDynamic::ResetToDefault)
		);

		Handle.bEnabled = true;

		InParams.PropertyRows->Add(Handle);
	}

	const TArray<FName>& Properties = ParentValue->GetEditableProperties();
	const int32 StartRow = InParams.PropertyRows->Num();

	for (const FName& Property : Properties)
	{
		if (Property == UDMMaterialValue::ValueName)
		{
			continue;
		}
			
		if (ValueDynamic->IsPropertyVisible(Property))
		{
			AddPropertyEditRows(InParams, Property);
		}
	}

	const int32 EndRow = InParams.PropertyRows->Num();

	for (int32 RowIndex = StartRow; RowIndex < EndRow; ++RowIndex)
	{
		(*InParams.PropertyRows)[RowIndex].bEnabled = false;
	}
}
