// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"

class FDMMaterialValueDynamicPropertyRowGenerator : public FDMComponentPropertyRowGenerator
{
public:
	static const TSharedRef<FDMMaterialValueDynamicPropertyRowGenerator>& Get();

	virtual ~FDMMaterialValueDynamicPropertyRowGenerator() override = default;

	//~ Begin FDMComponentPropertyRowGenerator
	virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams) override;
	//~ End FDMComponentPropertyRowGenerator
};
