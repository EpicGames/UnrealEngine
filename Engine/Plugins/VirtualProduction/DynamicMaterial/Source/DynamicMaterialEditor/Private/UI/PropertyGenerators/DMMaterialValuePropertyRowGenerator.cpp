// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMMaterialValuePropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialValue.h"
#include "Components/DMMaterialValueDynamic.h"
#include "DynamicMaterialEditorModule.h"
#include "IDetailPropertyRow.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/SDMMaterialEditor.h"

const TSharedRef<FDMMaterialValuePropertyRowGenerator>& FDMMaterialValuePropertyRowGenerator::Get()
{
	static TSharedRef<FDMMaterialValuePropertyRowGenerator> Generator = MakeShared<FDMMaterialValuePropertyRowGenerator>();
	return Generator;
}

void FDMMaterialValuePropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialValue* Value = Cast<UDMMaterialValue>(InParams.Object);

	if (!Value)
	{
		return;
	}

	// The base material value class is abstract and not allowed.
	if (Value->GetClass() == UDMMaterialValue::StaticClass())
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = static_cast<const SDMMaterialComponentEditor*>(InParams.Owner)->GetEditorWidget())
	{
		if (UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase())
		{
			if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(PreviewMaterialModelBase))
			{
				if (UDMMaterialComponentDynamic* ComponentDynamic = MaterialModelDynamic->GetComponentDynamic(Value->GetFName()))
				{
					FDMComponentPropertyRowGeneratorParams DynamicComponentParams = InParams;
					DynamicComponentParams.Object = ComponentDynamic;

					FDynamicMaterialEditorModule::Get().GeneratorComponentPropertyRows(DynamicComponentParams);
				}

				return;
			}
		}
	}

	if (Value->AllowEditValue())
	{
		FDMPropertyHandle Handle = FDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(UDMMaterialValue::ValueName));

		Handle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(Value, &UDMMaterialValue::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(Value, &UDMMaterialValue::ResetToDefault)
		);

		Handle.bEnabled = true;

		InParams.PropertyRows->Add(Handle);
	}

	const TArray<FName>& Properties = Value->GetEditableProperties();

	for (const FName& Property : Properties)
	{
		if (Property == UDMMaterialValue::ValueName)
		{
			continue;
		}
			
		if (Value->IsPropertyVisible(Property))
		{
			AddPropertyEditRows(InParams, Property);
		}
	}
}
