// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMStagePropertyRowGenerator.h"

#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageSource.h"
#include "DynamicMaterialEditorModule.h"
#include "Internationalization/Text.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "DMStagePropertyRowGenerator"

const TSharedRef<FDMStagePropertyRowGenerator>& FDMStagePropertyRowGenerator::Get()
{
	static TSharedRef<FDMStagePropertyRowGenerator> Generator = MakeShared<FDMStagePropertyRowGenerator>();
	return Generator;
}

void FDMStagePropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialStage* Stage = Cast<UDMMaterialStage>(InParams.Object);

	if (!Stage)
	{
		return;
	}

	UDMMaterialStageSource* Source = Stage->GetSource();

	if (!Source)
	{
		return;
	}

	FDMComponentPropertyRowGeneratorParams SourceParams = InParams;
	SourceParams.Object = Source;

	FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(SourceParams);

	FDMComponentPropertyRowGeneratorParams StageParams = InParams;
	StageParams.Object = Stage;

	FDMComponentPropertyRowGenerator::AddComponentProperties(StageParams);
}

#undef LOCTEXT_NAMESPACE
