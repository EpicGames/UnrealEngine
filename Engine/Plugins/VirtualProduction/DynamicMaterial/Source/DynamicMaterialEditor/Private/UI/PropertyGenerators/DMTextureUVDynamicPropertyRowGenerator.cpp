// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMTextureUVDynamicPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "DMEDefs.h"
#include "IDynamicMaterialEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/Visualizers/SDMTextureUVVisualizerProperty.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "DMTextureUVDynamicPropertyRowGenerator"

const TSharedRef<FDMTextureUVDynamicPropertyRowGenerator>& FDMTextureUVDynamicPropertyRowGenerator::Get()
{
	static TSharedRef<FDMTextureUVDynamicPropertyRowGenerator> Generator = MakeShared<FDMTextureUVDynamicPropertyRowGenerator>();
	return Generator;
}

namespace UE::DynamicMaterialEditor::Private
{
	void AddTextureUVDynamicPropertyRow(FDMComponentPropertyRowGeneratorParams& InParams, UDMMaterialComponent* InComponent, 
		FName InProperty, bool bInEnabled);

	void AddTextureUVDynamicVisualizerRow(FDMComponentPropertyRowGeneratorParams& InParams, UDMTextureUVDynamic* InTextureUVDynamic);

	bool CanResetTextureUVDynamicDynamicPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	void ResetTextureUVDynamicDynamicPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
}

void FDMTextureUVDynamicPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMTextureUVDynamic* TextureUVDynamic = Cast<UDMTextureUVDynamic>(InParams.Object);

	if (!TextureUVDynamic)
	{
		return;
	}

	UDMTextureUV* TextureUV = TextureUVDynamic->GetParentTextureUV();

	if (!TextureUV)
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	using namespace UE::DynamicMaterialEditor::Private;

	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Offset, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Rotation, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Tiling, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Pivot, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUV, UDMTextureUV::NAME_bMirrorOnX, /* Enabled */ false);
	AddTextureUVDynamicPropertyRow(InParams, TextureUV, UDMTextureUV::NAME_bMirrorOnY, /* Enabled */ false);
	AddTextureUVDynamicVisualizerRow(InParams, TextureUVDynamic);
}

void FDMTextureUVDynamicPropertyRowGenerator::AddPopoutComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	UDMTextureUVDynamic* TextureUVDynamic = Cast<UDMTextureUVDynamic>(InParams.Object);

	if (!TextureUVDynamic)
	{
		return;
	}

	UDMTextureUV* TextureUV = TextureUVDynamic->GetParentTextureUV();

	if (!TextureUV)
	{
		return;
	}

	using namespace UE::DynamicMaterialEditor::Private;

	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Offset, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Rotation, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Tiling, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUVDynamic, UDMTextureUV::NAME_Pivot, /* Enabled */ true);
	AddTextureUVDynamicPropertyRow(InParams, TextureUV, UDMTextureUV::NAME_bMirrorOnX, /* Enabled */ false);
	AddTextureUVDynamicPropertyRow(InParams, TextureUV, UDMTextureUV::NAME_bMirrorOnY, /* Enabled */ false);
}

bool FDMTextureUVDynamicPropertyRowGenerator::AllowKeyframeButton(UDMMaterialComponent* InComponent, FProperty* InProperty)
{
	if (InProperty)
	{
		const bool* AddKeyframeButtonPtr = UDMTextureUV::TextureProperties.Find(InProperty->GetFName());

		if (AddKeyframeButtonPtr)
		{
			return *AddKeyframeButtonPtr;
		}
	}

	return FDMComponentPropertyRowGenerator::AllowKeyframeButton(InComponent, InProperty);
}

void UE::DynamicMaterialEditor::Private::AddTextureUVDynamicPropertyRow(FDMComponentPropertyRowGeneratorParams& InParams,
	UDMMaterialComponent* InComponent, FName InProperty, bool bInEnabled)
{
	FDMComponentPropertyRowGeneratorParams ComponentParams = InParams;
	ComponentParams.Object = InComponent;

	FDMPropertyHandle& NewHandle = InParams.PropertyRows->Add_GetRef(FDMWidgetLibrary::Get().GetPropertyHandle(ComponentParams.CreatePropertyHandleParams(InProperty)));

	NewHandle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateStatic(&UE::DynamicMaterialEditor::Private::CanResetTextureUVDynamicDynamicPropertyToDefault),
		FResetToDefaultHandler::CreateStatic(&UE::DynamicMaterialEditor::Private::ResetTextureUVDynamicDynamicPropertyToDefault)
	);

	NewHandle.bEnabled = bInEnabled;
}

void UE::DynamicMaterialEditor::Private::AddTextureUVDynamicVisualizerRow(FDMComponentPropertyRowGeneratorParams& InParams, 
	UDMTextureUVDynamic* InTextureUVDynamic)
{
	UDMTextureUV* TextureUV = InTextureUVDynamic->GetParentTextureUV();

	if (!TextureUV)
	{
		return;
	}

	// Make sure we don't get a substage
	UDMMaterialStage* Stage = TextureUV->GetTypedParent<UDMMaterialStage>(/* Allow Subclasses */ false);

	if (!Stage)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = static_cast<const SDMMaterialComponentEditor*>(InParams.Owner)->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	FDMPropertyHandle VisualizerHandle;
	VisualizerHandle.NameOverride = LOCTEXT("Visualizer", "UV Visualizer");
	VisualizerHandle.NameToolTipOverride = LOCTEXT("VisualizerToolTip", "A graphical Texture UV editor.\n\n- Offset Mode: Change the Texture UV offset.\n- Pivot Mode: Change the Texture UV pivot, rotation and tiling.\n\nControl+click to reset values to default.");
	VisualizerHandle.ValueName = FName(*InTextureUVDynamic->GetComponentPath());
	VisualizerHandle.ValueWidget = SNew(SDMTextureUVVisualizerProperty, EditorWidget.ToSharedRef(), Stage).TextureUVDynamic(InTextureUVDynamic);
	VisualizerHandle.CategoryOverrideName = TEXT("Texture UV");
	VisualizerHandle.bEnabled = true;
	InParams.PropertyRows->Add(VisualizerHandle);
}


bool UE::DynamicMaterialEditor::Private::CanResetTextureUVDynamicDynamicPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FProperty* Property = InPropertyHandle->GetProperty();

	if (!Property)
	{
		return false;
	}

	const FName PropertyName = Property->GetFName();

	if (PropertyName.IsNone())
	{
		return false;
	}

	TArray<UObject*> Outers;
	InPropertyHandle->GetOuterObjects(Outers);

	if (Outers.IsEmpty())
	{
		return false;
	}

	const UDMTextureUVDynamic* PropertyObject = Cast<UDMTextureUVDynamic>(Outers[0]);

	if (!PropertyObject)
	{
		return false;
	}

	const UDMTextureUVDynamic* DefaultObject = GetDefault<UDMTextureUVDynamic>();

	if (!DefaultObject)
	{
		return false;
	}

	if (PropertyName == UDMTextureUV::NAME_Offset)
	{
		return !DefaultObject->GetOffset().Equals(PropertyObject->GetOffset());
	}

	if (PropertyName == UDMTextureUV::NAME_Pivot)
	{
		return !DefaultObject->GetPivot().Equals(PropertyObject->GetPivot());
	}

	if (PropertyName == UDMTextureUV::NAME_Rotation)
	{
		return DefaultObject->GetRotation() != PropertyObject->GetRotation();
	}

	if (PropertyName == UDMTextureUV::NAME_Tiling)
	{
		return !DefaultObject->GetTiling().Equals(PropertyObject->GetTiling());
	}

	return false;
}

void UE::DynamicMaterialEditor::Private::ResetTextureUVDynamicDynamicPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	InPropertyHandle->ResetToDefault();
}

#undef LOCTEXT_NAMESPACE
