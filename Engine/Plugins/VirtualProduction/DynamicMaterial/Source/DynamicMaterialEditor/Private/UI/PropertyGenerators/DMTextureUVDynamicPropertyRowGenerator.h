// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/PropertyGenerators/DMComponentPropertyRowGenerator.h"

class FDMTextureUVDynamicPropertyRowGenerator : public FDMComponentPropertyRowGenerator
{
public:
	static const TSharedRef<FDMTextureUVDynamicPropertyRowGenerator>& Get();

	virtual ~FDMTextureUVDynamicPropertyRowGenerator() override = default;

	static void AddPopoutComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams);

	//~ Begin FDMComponentPropertyRowGenerator
	virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams) override;

	virtual bool AllowKeyframeButton(UDMMaterialComponent* InComponent, FProperty* InProperty) override;
	//~ End FDMComponentPropertyRowGenerator
};
