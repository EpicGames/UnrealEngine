// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMTextureUVPropertyRowGenerator.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMTextureUV.h"
#include "DMEDefs.h"
#include "DynamicMaterialEditorModule.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/Visualizers/SDMTextureUVVisualizerProperty.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "DMTextureUVPropertyRowGenerator"

const TSharedRef<FDMTextureUVPropertyRowGenerator>& FDMTextureUVPropertyRowGenerator::Get()
{
	static TSharedRef<FDMTextureUVPropertyRowGenerator> Generator = MakeShared<FDMTextureUVPropertyRowGenerator>();
	return Generator;
}

namespace UE::DynamicMaterialEditor::Private
{
	void AddTextureUVPropertyRow(FDMComponentPropertyRowGeneratorParams& InParams, FName InProperty);

	void AddTextureUVVisualizerRow(FDMComponentPropertyRowGeneratorParams& InParams);

	bool CanResetTextureUVPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	void ResetTextureUVPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
}

void FDMTextureUVPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMTextureUV* TextureUV = Cast<UDMTextureUV>(InParams.Object);

	if (!TextureUV)
	{
		return;
	}

	InParams.ProcessedObjects.Add(InParams.Object);

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = static_cast<const SDMMaterialComponentEditor*>(InParams.Owner)->GetEditorWidget())
	{
		if (UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase())
		{
			if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(PreviewMaterialModelBase))
			{
				if (UDMMaterialComponentDynamic* ComponentDynamic = MaterialModelDynamic->GetComponentDynamic(TextureUV->GetFName()))
				{
					FDMComponentPropertyRowGeneratorParams DynamicComponentParams = InParams;
					DynamicComponentParams.Object = ComponentDynamic;

					FDynamicMaterialEditorModule::Get().GeneratorComponentPropertyRows(DynamicComponentParams);
				}

				return;
			}
		}
	}

	using namespace UE::DynamicMaterialEditor::Private;

	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Offset);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Rotation);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Tiling);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Pivot);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_bMirrorOnX);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_bMirrorOnY);
	AddTextureUVVisualizerRow(InParams);
}

void FDMTextureUVPropertyRowGenerator::AddPopoutComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	UDMTextureUV* TextureUV = Cast<UDMTextureUV>(InParams.Object);

	if (!TextureUV)
	{
		return;
	}

	using namespace UE::DynamicMaterialEditor::Private;

	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Offset);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Rotation);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Tiling);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_Pivot);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_bMirrorOnX);
	AddTextureUVPropertyRow(InParams, UDMTextureUV::NAME_bMirrorOnY);
}

bool FDMTextureUVPropertyRowGenerator::AllowKeyframeButton(UDMMaterialComponent* InComponent, FProperty* InProperty)
{
	if (InProperty)
	{
		const bool* AddKeyframeButtonPtr = UDMTextureUV::TextureProperties.Find(InProperty->GetFName());

		if (AddKeyframeButtonPtr)
		{
			return *AddKeyframeButtonPtr;
		}
	}

	return FDMComponentPropertyRowGenerator::AllowKeyframeButton(InComponent, InProperty);
}

void UE::DynamicMaterialEditor::Private::AddTextureUVPropertyRow(FDMComponentPropertyRowGeneratorParams& InParams, FName InProperty)
{
	FDMPropertyHandle& NewHandle = InParams.PropertyRows->Add_GetRef(FDMWidgetLibrary::Get().GetPropertyHandle(InParams.CreatePropertyHandleParams(InProperty)));

	NewHandle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateStatic(&UE::DynamicMaterialEditor::Private::CanResetTextureUVPropertyToDefault),
		FResetToDefaultHandler::CreateStatic(&UE::DynamicMaterialEditor::Private::ResetTextureUVPropertyToDefault)
	);

	NewHandle.bEnabled = true;
}

void UE::DynamicMaterialEditor::Private::AddTextureUVVisualizerRow(FDMComponentPropertyRowGeneratorParams& InParams)
{
	UDMTextureUV* TextureUV = CastChecked<UDMTextureUV>(InParams.Object);

	// Make sure we don't get a substage
	UDMMaterialStage* Stage = TextureUV->GetTypedParent<UDMMaterialStage>(/* Allow Subclasses */ false);

	if (!Stage)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = static_cast<const SDMMaterialComponentEditor*>(InParams.Owner)->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	FDMPropertyHandle VisualizerHandle;
	VisualizerHandle.NameOverride = LOCTEXT("Visualizer", "UV Visualizer");
	VisualizerHandle.NameToolTipOverride = LOCTEXT("VisualizerToolTip", "A graphical Texture UV editor.\n\n- Offset Mode: Change the Texture UV offset.\n- Pivot Mode: Change the Texture UV pivot, rotation and tiling.\n\nControl+click to reset values to default.");
	VisualizerHandle.ValueName = FName(*TextureUV->GetComponentPath());
	VisualizerHandle.ValueWidget = SNew(SDMTextureUVVisualizerProperty, EditorWidget.ToSharedRef(), Stage).TextureUV(TextureUV);
	VisualizerHandle.CategoryOverrideName = TEXT("Texture UV");
	VisualizerHandle.bEnabled = true;
	InParams.PropertyRows->Add(VisualizerHandle);
}


bool UE::DynamicMaterialEditor::Private::CanResetTextureUVPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FProperty* Property = InPropertyHandle->GetProperty();

	if (!Property)
	{
		return false;
	}

	const FName PropertyName = Property->GetFName();

	if (PropertyName.IsNone())
	{
		return false;
	}

	TArray<UObject*> Outers;
	InPropertyHandle->GetOuterObjects(Outers);

	if (Outers.IsEmpty())
	{
		return false;
	}

	const UDMTextureUV* PropertyObject = Cast<UDMTextureUV>(Outers[0]);

	if (!PropertyObject)
	{
		return false;
	}

	const UDMTextureUV* DefaultObject = GetDefault<UDMTextureUV>();

	if (!DefaultObject)
	{
		return false;
	}

	if (PropertyName == UDMTextureUV::NAME_UVSource)
	{
		return DefaultObject->GetUVSource() != PropertyObject->GetUVSource();
	}

	if (PropertyName == UDMTextureUV::NAME_bMirrorOnX)
	{
		return DefaultObject->GetMirrorOnX() != PropertyObject->GetMirrorOnX();
	}

	if (PropertyName == UDMTextureUV::NAME_bMirrorOnY)
	{
		return DefaultObject->GetMirrorOnY() != PropertyObject->GetMirrorOnY();
	}

	if (PropertyName == UDMTextureUV::NAME_Offset)
	{
		return !DefaultObject->GetOffset().Equals(PropertyObject->GetOffset());
	}

	if (PropertyName == UDMTextureUV::NAME_Pivot)
	{
		return !DefaultObject->GetPivot().Equals(PropertyObject->GetPivot());
	}

	if (PropertyName == UDMTextureUV::NAME_Rotation)
	{
		return DefaultObject->GetRotation() != PropertyObject->GetRotation();
	}

	if (PropertyName == UDMTextureUV::NAME_Tiling)
	{
		return !DefaultObject->GetTiling().Equals(PropertyObject->GetTiling());
	}

	return false;
}

void UE::DynamicMaterialEditor::Private::ResetTextureUVPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	InPropertyHandle->ResetToDefault();
}

#undef LOCTEXT_NAMESPACE
