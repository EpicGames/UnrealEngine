// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/PropertyGenerators/DMThroughputPropertyRowGenerator.h"

#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageInput.h"
#include "Components/DMMaterialStageThroughput.h"
#include "Components/DMMaterialValue.h"
#include "DynamicMaterialEditorModule.h"

const TSharedRef<FDMThroughputPropertyRowGenerator>& FDMThroughputPropertyRowGenerator::Get()
{
	static TSharedRef<FDMThroughputPropertyRowGenerator> Generator = MakeShared<FDMThroughputPropertyRowGenerator>();
	return Generator;
}

void FDMThroughputPropertyRowGenerator::AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams)
{
	if (!IsValid(InParams.Object))
	{
		return;
	}

	if (InParams.ProcessedObjects.Contains(InParams.Object))
	{
		return;
	}

	UDMMaterialStageThroughput* Throughput = Cast<UDMMaterialStageThroughput>(InParams.Object);

	if (!Throughput)
	{
		return;
	}

	InParams.ProcessedObjects.Add(Throughput);

	const TArray<FName>& ThroughputProperties = Throughput->GetEditableProperties();

	for (const FName& ThroughputProperty : ThroughputProperties)
	{
		if (Throughput->IsPropertyVisible(ThroughputProperty))
		{
			AddPropertyEditRows(InParams, ThroughputProperty);
		}
	}

	UDMMaterialStage* Stage = Throughput->GetStage();

	if (!Stage)
	{
		return;
	}

	InParams.ProcessedObjects.Add(Stage);

	static const FName StageInputsName = GET_MEMBER_NAME_CHECKED(UDMMaterialStage, Inputs);

	{
		const TArray<FDMMaterialStageConnector>& InputConnectors = Throughput->GetInputConnectors();
		const TArray<FDMMaterialStageConnection>& InputMap = Stage->GetInputConnectionMap();
		TArray<UDMMaterialStageInput*> Inputs = Stage->GetInputs();

		for (int32 InputIdx = 0; InputIdx < InputConnectors.Num(); ++InputIdx)
		{
			if (!Throughput->IsInputVisible(InputIdx) || !Throughput->CanChangeInput(InputIdx))
			{
				continue;
			}

			const int32 StartRow = InParams.PropertyRows->Num();

			for (const FDMMaterialStageConnectorChannel& Channel : InputMap[InputIdx].Channels)
			{
				const int32 StageInputIdx = Channel.SourceIndex - FDMMaterialStageConnectorChannel::FIRST_STAGE_INPUT;

				if (Inputs.IsValidIndex(StageInputIdx))
				{
					FDMComponentPropertyRowGeneratorParams InputParams = InParams;
					InputParams.Object = Inputs[StageInputIdx];

					FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(InputParams);
				}
			}

			for (int32 PropertyRowIdx = StartRow; PropertyRowIdx < InParams.PropertyRows->Num(); ++PropertyRowIdx)
			{
				FDMPropertyHandle& PropertyRow = (*InParams.PropertyRows)[PropertyRowIdx];

				if (PropertyRow.NameOverride.IsSet())
				{
					continue;
				}

				if (!PropertyRow.PreviewHandle.PropertyHandle.IsValid()
					|| !PropertyRow.PreviewHandle.PropertyHandle->GetProperty())
				{
					continue;
				}

				TArray<UObject*> Outers;
				PropertyRow.PreviewHandle.PropertyHandle->GetOuterObjects(Outers);

				if (Outers.IsEmpty() || !Outers[0]->IsA<UDMMaterialValue>())
				{
					continue;
				}

				if (!PropertyRow.NameOverride.IsSet() || PropertyRow.NameOverride.GetValue().IsEmpty())

				{
					if (!PropertyRow.ValueName.IsNone())
					{
						PropertyRow.NameOverride = FText::FromName(PropertyRow.ValueName);
					}
					else if (!InputConnectors[InputIdx].Name.IsEmpty())
					{
						PropertyRow.NameOverride = InputConnectors[InputIdx].Name;
					}
					else if (PropertyRow.PreviewHandle.PropertyHandle.IsValid())
					{
						PropertyRow.NameOverride = PropertyRow.PreviewHandle.PropertyHandle->GetPropertyDisplayName();
					}
				}
			}
		}
	}

	FDMComponentPropertyRowGeneratorParams StageParams = InParams;
	StageParams.Object = Stage;

	const TArray<FName>& StageProperties = Stage->GetEditableProperties();

	for (const FName& StageProperty : StageProperties)
	{
		if (StageProperty != StageInputsName)
		{
			AddPropertyEditRows(StageParams, StageProperty);
		}
	}
}
