// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SDropTarget.h"
#include "Styling/SlateColor.h"

/** This allows setting of the private variable InvalidColor in the SDropTarget class. */
namespace UE::DynamicMaterialEditor::Private
{
	namespace DropTarget
	{
		template <auto SDropTarget::* Member>
		struct FDropTargetSetter
		{
			friend void SetInvalidColor(SDropTarget* InDropTarget, const FSlateColor& InNewColor)
			{
				InDropTarget->*Member = InNewColor;
			}
		};

		template struct FDropTargetSetter<&SDropTarget::InvalidColor>;
		void SetInvalidColor(SDropTarget* InDropTarget, const FSlateColor& InNewColor);
	}
}
