// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/DMEditorSelectionContext.h"

#include "Components/DMMaterialProperty.h"

FDMMaterialEditorPage FDMMaterialEditorPage::Preview = {EDMMaterialEditorMode::MaterialPreview, EDMMaterialPropertyType::None};
FDMMaterialEditorPage FDMMaterialEditorPage::GlobalSettings = {EDMMaterialEditorMode::GlobalSettings, EDMMaterialPropertyType::None};
FDMMaterialEditorPage FDMMaterialEditorPage::Properties = {EDMMaterialEditorMode::Properties, EDMMaterialPropertyType::None};

bool FDMMaterialEditorPage::operator==(const FDMMaterialEditorPage& InOther) const
{
	return EditorMode == InOther.EditorMode && MaterialProperty == InOther.MaterialProperty;
}
