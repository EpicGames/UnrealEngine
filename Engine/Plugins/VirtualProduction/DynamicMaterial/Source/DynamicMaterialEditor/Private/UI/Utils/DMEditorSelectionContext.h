// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "UObject/WeakObjectPtr.h"

class UDMMaterialComponent;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
enum class EDMMaterialPropertyType : uint8;

enum class EDMMaterialEditorMode : uint8
{
	GlobalSettings,
	Properties,
	EditSlot,
	MaterialPreview
};

struct FDMMaterialEditorPage
{
	static FDMMaterialEditorPage Preview;
	static FDMMaterialEditorPage GlobalSettings;
	static FDMMaterialEditorPage Properties;

	EDMMaterialEditorMode EditorMode;
	EDMMaterialPropertyType MaterialProperty;

	bool operator==(const FDMMaterialEditorPage& InOther) const;
};

struct FDMEditorSelectionContext
{
	EDMMaterialEditorMode EditorMode = EDMMaterialEditorMode::GlobalSettings;

	bool bModeChanged = false;

	EDMMaterialPropertyType Property;

	TWeakObjectPtr<UDMMaterialSlot> Slot = nullptr;

	TWeakObjectPtr<UDMMaterialLayerObject> Layer = nullptr;

	TWeakObjectPtr<UDMMaterialComponent> Component = nullptr;

	TArray<FDMMaterialEditorPage> PageHistory;

	int32 PageHistoryActive = 0;

	int32 PageHistoryCount = 0;
};
