// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/DMPreviewMaterialManager.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DynamicMaterialModule.h"
#include "Factories/MaterialFactoryNew.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"

UMaterial* FDMPreviewMaterialManager::CreatePreviewMaterial(UObject* InPreviewing)
{
	UMaterial* PreviewMaterial = nullptr;

	if (FDynamicMaterialModule::IsMaterialExportEnabled() == false)
	{
		PreviewMaterial = Cast<UMaterial>(GetMutableDefault<UMaterialFactoryNew>()->FactoryCreateNew(
			UMaterial::StaticClass(),
			GetTransientPackage(),
			NAME_None,
			RF_Transient,
			nullptr,
			GWarn
		));

		PreviewMaterial->bIsPreviewMaterial = true;
	}
	else
	{
		FString MaterialBaseName = InPreviewing->GetName() + "-" + FGuid::NewGuid().ToString();
		const FString FullName = TEXT("/Game/MaterialDesignerMaterials/") + MaterialBaseName;
		UPackage* Package = CreatePackage(*FullName);

		PreviewMaterial = Cast<UMaterial>(GetMutableDefault<UMaterialFactoryNew>()->FactoryCreateNew(
			UMaterial::StaticClass(),
			Package,
			*MaterialBaseName,
			RF_Standalone | RF_Public,
			nullptr,
			GWarn
		));

		FAssetRegistryModule::AssetCreated(PreviewMaterial);
	}

	PreviewMaterials.FindOrAdd(InPreviewing).Reset(PreviewMaterial);

	return PreviewMaterial;
}

void FDMPreviewMaterialManager::FreePreviewMaterial(UObject* InPreviewing)
{
	if (TStrongObjectPtr<UMaterial>* PreviewMaterial = PreviewMaterials.Find(InPreviewing))
	{
		if (UMaterial* Material = PreviewMaterial->Get())
		{
			FreePreviewMaterialDynamic(Material);
		}

		PreviewMaterials.Remove(InPreviewing);
	}
}

UMaterialInstanceDynamic* FDMPreviewMaterialManager::CreatePreviewMaterialDynamic(UMaterial* InMaterialBase)
{
	if (!InMaterialBase)
	{
		return nullptr;
	}

	UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create(InMaterialBase, GetTransientPackage());
	PreviewMaterialDynamics.FindOrAdd(InMaterialBase).Reset(MID);

	return MID;
}

void FDMPreviewMaterialManager::FreePreviewMaterialDynamic(UMaterial* InMaterialBase)
{
	PreviewMaterialDynamics.Remove(InMaterialBase);
}
