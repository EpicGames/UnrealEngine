// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/ObjectKey.h"
#include "UObject/StrongObjectPtr.h"

class UMaterial;
class UMaterialInstanceDynamic;
class UObject;

class FDMPreviewMaterialManager
{
public:
	UMaterial* CreatePreviewMaterial(UObject* InPreviewing);

	void FreePreviewMaterial(UObject* InPreviewing);

	UMaterialInstanceDynamic* CreatePreviewMaterialDynamic(UMaterial* InMaterialBase);

	void FreePreviewMaterialDynamic(UMaterial* InMaterialBase);

private:
	TMap<FObjectKey, TStrongObjectPtr<UMaterial>> PreviewMaterials;
	TMap<FObjectKey, TStrongObjectPtr<UMaterialInstanceDynamic>> PreviewMaterialDynamics;
};
