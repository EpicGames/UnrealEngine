// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/DMWidgetLibrary.h"

#include "Components/MaterialValues/DMMaterialValueFloat.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Utils/DMPrivate.h"

IDMWidgetLibrary& IDMWidgetLibrary::Get()
{
	return FDMWidgetLibrary::Get();
}

const FLazyName FDMWidgetLibrary::PropertyValueWidget = TEXT("SPropertyValueWidget");

FDMWidgetLibrary& FDMWidgetLibrary::Get()
{
	 static FDMWidgetLibrary Instance;
	 return Instance;
}

bool FDMWidgetLibrary::GetExpansionState(UObject* InOwner, FName InName, bool& bOutExpanded)
{
	const FExpansionItem ExpansionItem = {InOwner, InName};

	if (const bool* State = ExpansionStates.Find(ExpansionItem))
	{
		bOutExpanded = *State;
		return true;
	}

	return false;
}

void FDMWidgetLibrary::SetExpansionState(UObject* InOwner, FName InName, bool bInIsExpanded)
{
	const FExpansionItem ExpansionItem = {InOwner, InName};
	ExpansionStates.FindOrAdd(ExpansionItem) = bInIsExpanded;
}

FDMPropertyHandle FDMWidgetLibrary::GetPropertyHandle(const FDMPropertyHandleGenerateParams& InParams)
{
	TArray<FDMPropertyHandle>& PropertyHandles = PropertyHandleMap.FindOrAdd(InParams.Widget);

	for (const FDMPropertyHandle& ExistingHandle : PropertyHandles)
	{
		if (ExistingHandle.PreviewHandle.PropertyHandle && ExistingHandle.PreviewHandle.PropertyHandle->GetProperty()->GetFName() == InParams.PropertyName)
		{
			TArray<UObject*> Outers;
			ExistingHandle.PreviewHandle.PropertyHandle->GetOuterObjects(Outers);

			if (Outers.IsEmpty() == false && Outers[0] == InParams.Object)
			{
				return ExistingHandle;
			}
		}
	}

	if (const FDMPropertyHandle* ParentHandle = SearchForHandle(PropertyHandles, InParams.Object))
	{
		FDMPropertyHandle NewChildHandle;
		NewChildHandle.PreviewHandle.PropertyRowGenerator = ParentHandle->PreviewHandle.PropertyRowGenerator;
		NewChildHandle.OriginalHandle.PropertyRowGenerator = ParentHandle->OriginalHandle.PropertyRowGenerator;

		if (NewChildHandle.PreviewHandle.PropertyRowGenerator.IsValid())
		{
			if (TSharedPtr<IDetailTreeNode> DetailTreeNode = SearchGeneratorForNode(NewChildHandle.PreviewHandle.PropertyRowGenerator.ToSharedRef(), InParams.PropertyName))
			{
				NewChildHandle.PreviewHandle.DetailTreeNode = DetailTreeNode;
				NewChildHandle.PreviewHandle.PropertyHandle = DetailTreeNode->CreatePropertyHandle();
			}
		}

		if (NewChildHandle.OriginalHandle.PropertyRowGenerator.IsValid())
		{
			if (TSharedPtr<IDetailTreeNode> DetailTreeNode = SearchGeneratorForNode(NewChildHandle.OriginalHandle.PropertyRowGenerator.ToSharedRef(), InParams.PropertyName))
			{
				NewChildHandle.OriginalHandle.DetailTreeNode = DetailTreeNode;
				NewChildHandle.OriginalHandle.PropertyHandle = DetailTreeNode->CreatePropertyHandle();
			}
		}

		if (NewChildHandle.PreviewHandle.PropertyHandle.IsValid())
		{
			AddPropertyMetaData(InParams.Object, InParams.PropertyName, NewChildHandle);
		}

		PropertyHandles.Add(NewChildHandle);

		return NewChildHandle;
	}

	FDMPropertyHandle NewHandle = CreatePropertyHandle(InParams);

	if (!NewHandle.PreviewHandle.PropertyHandle.IsValid() && NewHandle.PreviewHandle.DetailTreeNode.IsValid())
	{
		NewHandle.PreviewHandle.PropertyHandle = NewHandle.PreviewHandle.DetailTreeNode->CreatePropertyHandle();
	}

	if (!NewHandle.OriginalHandle.PropertyHandle.IsValid() && NewHandle.OriginalHandle.DetailTreeNode.IsValid())
	{
		NewHandle.OriginalHandle.PropertyHandle = NewHandle.OriginalHandle.DetailTreeNode->CreatePropertyHandle();
	}

	AddPropertyMetaData(InParams.Object, InParams.PropertyName, NewHandle);

	PropertyHandles.Add(NewHandle);

	return NewHandle;
}

void FDMWidgetLibrary::ClearPropertyHandles(const SWidget* InOwningWidget)
{
	PropertyHandleMap.Remove(InOwningWidget);
}

TSharedPtr<SWidget> FDMWidgetLibrary::FindWidgetInHierarchy(const TSharedRef<SWidget>& InParent, const FName& InName)
{
	if (InParent->GetType() == InName)
	{
		return InParent;
	}

	FChildren* Children = InParent->GetChildren();

	if (!Children)
	{
		return nullptr;
	}

	const int32 ChildNum = Children->Num();

	for (int32 Index = 0; Index < ChildNum; ++Index)
	{
		const TSharedRef<SWidget>& Widget = Children->GetChildAt(Index);

		if (Widget->GetType() == InName)
		{
			return Widget;
		}
	}

	for (int32 Index = 0; Index < ChildNum; ++Index)
	{
		if (TSharedPtr<SWidget> FoundChild = FindWidgetInHierarchy(Children->GetChildAt(Index), InName))
		{
			return FoundChild;
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> FDMWidgetLibrary::GetInnerPropertyValueWidget(const TSharedRef<SWidget>& InWidget)
{
	if (FChildren* Children = InWidget->GetChildren())
	{
		if (Children->Num() > 0)
		{
			return Children->GetChildAt(0);
		}
	}

	return nullptr;
}

void FDMWidgetLibrary::ClearData()
{
	ExpansionStates.Empty();
	PropertyHandleMap.Empty();
}

FDMPropertyHandle FDMWidgetLibrary::CreatePropertyHandle(const FDMPropertyHandleGenerateParams& InParams)
{
	FDMPropertyHandle PropertyHandle;

	if (!InParams.Object)
	{
		return PropertyHandle;
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FPropertyRowGeneratorArgs PreviewRowGeneratorArgs;
	PreviewRowGeneratorArgs.NotifyHook = InParams.NotifyHook;
	PreviewRowGeneratorArgs.bShouldShowHiddenProperties = true;

	PropertyHandle.PreviewHandle.PropertyRowGenerator = PropertyEditor.CreatePropertyRowGenerator(PreviewRowGeneratorArgs);
	PropertyHandle.PreviewHandle.PropertyRowGenerator->SetObjects({InParams.Object});

	if (const TSharedPtr<IDetailTreeNode> FoundTreeNode = SearchGeneratorForNode(
		PropertyHandle.PreviewHandle.PropertyRowGenerator.ToSharedRef(), InParams.PropertyName))
	{
		PropertyHandle.PreviewHandle.DetailTreeNode = FoundTreeNode;
		PropertyHandle.PreviewHandle.PropertyHandle = FoundTreeNode->CreatePropertyHandle();
	}

	if (InParams.PreviewMaterialModelBase && InParams.OriginalMaterialModelBase && InParams.Object)
	{
		UObject* OriginalObject = nullptr;

		if (InParams.Object == InParams.PreviewMaterialModelBase)
		{
			OriginalObject = InParams.OriginalMaterialModelBase;
		}
		else if (InParams.Object->IsIn(InParams.PreviewMaterialModelBase))
		{
			const FString RelativePath = InParams.Object->GetPathName(InParams.PreviewMaterialModelBase);
			OriginalObject = UDMMaterialModelFunctionLibrary::FindSubobject(InParams.OriginalMaterialModelBase, RelativePath);
		}

		if (OriginalObject)
		{
			// No notify hook for the original
			FPropertyRowGeneratorArgs OriginalRowGeneratorArgs;
			OriginalRowGeneratorArgs.bShouldShowHiddenProperties = true;

			PropertyHandle.OriginalHandle.PropertyRowGenerator = PropertyEditor.CreatePropertyRowGenerator(OriginalRowGeneratorArgs);
			PropertyHandle.OriginalHandle.PropertyRowGenerator->SetObjects({OriginalObject});

			if (const TSharedPtr<IDetailTreeNode> FoundTreeNode = SearchGeneratorForNode(
				PropertyHandle.OriginalHandle.PropertyRowGenerator.ToSharedRef(), InParams.PropertyName))
			{
				PropertyHandle.OriginalHandle.DetailTreeNode = FoundTreeNode;
				PropertyHandle.OriginalHandle.PropertyHandle = FoundTreeNode->CreatePropertyHandle();
			}
		}
	}

	return PropertyHandle;
}

TSharedPtr<IDetailTreeNode> FDMWidgetLibrary::SearchNodesForProperty(const TArray<TSharedRef<IDetailTreeNode>>& InNodes, FName InPropertyName)
{
	for (const TSharedRef<IDetailTreeNode>& ChildNode : InNodes)
	{
		switch (ChildNode->GetNodeType())
		{
			case EDetailNodeType::Category:
			{
				TArray<TSharedRef<IDetailTreeNode>> CategoryChildNodes;
				ChildNode->GetChildren(CategoryChildNodes);

				if (TSharedPtr<IDetailTreeNode> FoundNode = SearchNodesForProperty(CategoryChildNodes, InPropertyName))
				{
					return FoundNode;
				}

				break;
			}

			case EDetailNodeType::Item:
				if (ChildNode->GetNodeName() == InPropertyName)
				{
					return ChildNode;
				}
				break;

			default:
				// Do nothing
				break;
		}
	}

	return nullptr;
}

TSharedPtr<IDetailTreeNode> FDMWidgetLibrary::SearchGeneratorForNode(const TSharedRef<IPropertyRowGenerator>& InGenerator, FName InPropertyName)
{
	return SearchNodesForProperty(InGenerator->GetRootTreeNodes(), InPropertyName);
}

const FDMPropertyHandle* FDMWidgetLibrary::SearchForHandle(const TArray<FDMPropertyHandle>& InPropertyHandles, UObject* InObject)
{
	if (!InObject)
	{
		return nullptr;
	}

	for (const FDMPropertyHandle& PropertyHandle : InPropertyHandles)
	{
		if (PropertyHandle.PreviewHandle.PropertyRowGenerator.IsValid())
		{
			for (const TWeakObjectPtr<UObject>& WeakObject : PropertyHandle.PreviewHandle.PropertyRowGenerator->GetSelectedObjects())
			{
				if (WeakObject.Get() == InObject)
				{
					return &PropertyHandle;
				}
			}
		}
	}

	return nullptr;
}

void FDMWidgetLibrary::AddPropertyMetaData(UObject* InObject, FName InPropertyName, FDMPropertyHandle& InPropertyHandle)
{
	FProperty* Property = nullptr;

	if (InPropertyHandle.PreviewHandle.PropertyHandle.IsValid())
	{
		TSharedRef<IPropertyHandle> PropertyHandleRef = InPropertyHandle.PreviewHandle.PropertyHandle.ToSharedRef();
		InPropertyHandle.Priority = GetPriority(PropertyHandleRef);
		InPropertyHandle.bKeyframeable = IsKeyframeable(PropertyHandleRef);

		Property = InPropertyHandle.PreviewHandle.PropertyHandle->GetProperty();

		if (UDMMaterialValueFloat* FloatValue = Cast<UDMMaterialValueFloat>(InObject))
		{
			if (FloatValue->HasValueRange())
			{
				const FName UIMin = FName("UIMin");
				const FName UIMax = FName("UIMax");
				const FName ClampMin = FName("ClampMin");
				const FName ClampMax = FName("ClampMax");

				InPropertyHandle.PreviewHandle.PropertyHandle->SetInstanceMetaData(UIMin, FString::SanitizeFloat(FloatValue->GetValueRange().Min));
				InPropertyHandle.PreviewHandle.PropertyHandle->SetInstanceMetaData(ClampMin, FString::SanitizeFloat(FloatValue->GetValueRange().Min));
				InPropertyHandle.PreviewHandle.PropertyHandle->SetInstanceMetaData(UIMax, FString::SanitizeFloat(FloatValue->GetValueRange().Max));
				InPropertyHandle.PreviewHandle.PropertyHandle->SetInstanceMetaData(ClampMax, FString::SanitizeFloat(FloatValue->GetValueRange().Max));
			}
		}
	}
	else if (IsValid(InObject))
	{
		Property = InObject->GetClass()->FindPropertyByName(InPropertyName);
	}

	if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		uint8 ComponentCount = 1;

		if (StructProperty->Struct == TBaseStructure<FVector2D>::Get()
			|| StructProperty->Struct == TVariantStructure<FVector2f>::Get())
		{
			ComponentCount = 2;
		}

		if (StructProperty->Struct == TBaseStructure<FVector>::Get()
			|| StructProperty->Struct == TVariantStructure<FVector3f>::Get()
			|| StructProperty->Struct == TBaseStructure<FRotator>::Get())
		{
			ComponentCount = 3;
		}

		// FLinearColor doesn't need the extra space
		if (StructProperty->Struct == TBaseStructure<FVector4>::Get()
			|| StructProperty->Struct == TVariantStructure<FVector4f>::Get())
		{
			ComponentCount = 4;
		}

		switch (ComponentCount)
		{
			case 0:
			case 1:
				break;

			case 2:
				InPropertyHandle.MaxWidth = 200.f;
				break;

				// 3 and above
			default:
				InPropertyHandle.MaxWidth = 275.f;
				break;
		}
	}
}

EDMPropertyHandlePriority FDMWidgetLibrary::GetPriority(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	if (InPropertyHandle->HasMetaData("HighPriority"))
	{
		return EDMPropertyHandlePriority::High;
	}

	if (InPropertyHandle->HasMetaData("LowPriority"))
	{
		return EDMPropertyHandlePriority::Low;
	}

	return EDMPropertyHandlePriority::Normal;
}

bool FDMWidgetLibrary::IsKeyframeable(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	return !InPropertyHandle->HasMetaData("NotKeyframeable");
}
