// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DMEDefs.h"
#include "Templates/SharedPointerFwd.h"
#include "UI/Utils/IDMWidgetLibrary.h"
#include "UObject/ObjectKey.h"

class FName;
class SWidget;
class UObject;

class FDMWidgetLibrary : public IDMWidgetLibrary
{
public:
	static const FLazyName PropertyValueWidget;

	static FDMWidgetLibrary& Get();

	virtual ~FDMWidgetLibrary() = default;

	bool GetExpansionState(UObject* InOwner, FName InName, bool& bOutExpanded);

	void SetExpansionState(UObject* InOwner, FName InName, bool bInIsExpanded);

	void ClearPropertyHandles(const SWidget* InOwningWidget);

	TSharedPtr<SWidget> FindWidgetInHierarchy(const TSharedRef<SWidget>& InParent, const FName& InName);

	TSharedPtr<SWidget> GetInnerPropertyValueWidget(const TSharedRef<SWidget>& InWidget);

	void ClearData();

	//~ Begin IDMWidgetLibrary
	virtual FDMPropertyHandle GetPropertyHandle(const FDMPropertyHandleGenerateParams& InParams) override;
	//~ End IDMWidgetLibrary

private:
	struct FExpansionItem
	{
		TObjectKey<UObject> Owner;
		FName Name;

		bool operator==(const FExpansionItem& InOther) const
		{
			return Owner == InOther.Owner
				&& Name == InOther.Name;
		}

		friend uint32 GetTypeHash(const FExpansionItem& InItem)
		{
			return HashCombineFast(
				GetTypeHash(InItem.Owner),
				GetTypeHash(InItem.Name)
			);
		}
	};

	TMap<FExpansionItem, bool> ExpansionStates;

	TMap<const SWidget*, TArray<FDMPropertyHandle>> PropertyHandleMap;

	FDMPropertyHandle CreatePropertyHandle(const FDMPropertyHandleGenerateParams& InParams);

	static TSharedPtr<IDetailTreeNode> SearchNodesForProperty(const TArray<TSharedRef<IDetailTreeNode>>& InNodes, FName InPropertyName);

	static TSharedPtr<IDetailTreeNode> SearchGeneratorForNode(const TSharedRef<IPropertyRowGenerator>& InGenerator, FName InPropertyName);

	static const FDMPropertyHandle* SearchForHandle(const TArray<FDMPropertyHandle>& InPropertyHandles, UObject* InObject);

	static void AddPropertyMetaData(UObject* InObject, FName InPropertyName, FDMPropertyHandle& InPropertyHandle);

	/** Checks the meta data of a property meta data to check for high and low priority specifiers. */
	static EDMPropertyHandlePriority GetPriority(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	/** Checks for the NotKeyframeable meta data. */
	static bool IsKeyframeable(const TSharedRef<IPropertyHandle>& InPropertyHandle);
};
