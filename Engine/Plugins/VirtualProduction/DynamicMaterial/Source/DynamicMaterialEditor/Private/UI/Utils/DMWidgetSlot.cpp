// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/DMWidgetSlot.h"

#include "Layout/Children.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Widgets/SNullWidget.h"

FDMWidgetSlot::FDMWidgetSlot(FSlotBase* InSlot, const TSharedRef<SWidget>& InWidget)
{
	if (InSlot)
	{
		SetSlot(InSlot);
	}

	AssignWidget(InWidget);
}

FSlotBase* FDMWidgetSlot::GetSlot() const
{
	if (Owner.IsValid())
	{
		return Slot;
	}

	return nullptr;
}

void FDMWidgetSlot::SetSlot(FSlotBase* InSlot)
{
	if (FSlotBase* ValidSlot = GetSlot())
	{
		ValidSlot->DetachWidget();
	}

	Owner.Reset();

	Slot = InSlot;

	if (InSlot)
	{
		if (SWidget* SlotOwner = Slot->GetOwnerWidget())
		{
			Owner = SlotOwner->AsShared();
		}

		if (Widget.IsValid())
		{
			Slot->AttachWidget(Widget.ToSharedRef());
		}
	}
}

bool FDMWidgetSlot::IsValid() const
{
	return !bInvalidated && HasWidget();
}

bool FDMWidgetSlot::HasBeenInvalidated() const
{
	return bInvalidated;
}

void FDMWidgetSlot::Invalidate()
{
	bInvalidated = true;
}

bool FDMWidgetSlot::HasWidget() const
{
	return Widget.IsValid() && Widget != SNullWidget::NullWidget;
}

void FDMWidgetSlot::ClearWidget()
{
	Widget.Reset();
	bInvalidated = true;

	if (FSlotBase* ValidSlot = GetSlot())
	{
		ValidSlot->DetachWidget();
	}
}

FSlotBase* FDMWidgetSlot::FindSlot(const TSharedRef<SWidget>& InParentWidget, int32 InChildSlot) const
{
	ensure(InChildSlot >= 0);

	FChildren* ParentChildren = InParentWidget->GetChildren();
	ensure(ParentChildren->Num() > InChildSlot);

	return &const_cast<FSlotBase&>(ParentChildren->GetSlotAt(InChildSlot));
}

void FDMWidgetSlot::AssignWidget(const TSharedRef<SWidget>& InWidget)
{
	Widget = InWidget;
	bInvalidated = InWidget == SNullWidget::NullWidget;

	if (FSlotBase* ValidSlot = GetSlot())
	{
		ValidSlot->AttachWidget(InWidget);
	}
}

bool FDMWidgetSlot::operator==(const TSharedRef<SWidget>& InWidget) const
{
	return Widget == InWidget;
}
