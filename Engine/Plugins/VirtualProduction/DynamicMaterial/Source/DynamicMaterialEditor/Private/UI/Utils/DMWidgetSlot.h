// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FSlotBase;
class SWidget;

/**
 * Handles storing a widget, the slot it's in and its invalidation state.
 * Provides methods and operator overrides to simplify replacing widgets.
 */
struct FDMWidgetSlot
{
	FSlotBase* GetSlot() const;

	void SetSlot(FSlotBase* InSlot);

	/** Returns true if the widget is valid and hasn't been invalidated. */
	bool IsValid() const;

	bool HasBeenInvalidated() const;

	void Invalidate();

	/** Will return true even if the widget is invalidated. */
	bool HasWidget() const;

	void ClearWidget();

	bool operator==(const TSharedRef<SWidget>& InWidget) const;

protected:
	TWeakPtr<SWidget> Owner;
	FSlotBase* Slot = nullptr;
	TSharedPtr<SWidget> Widget;
	bool bInvalidated = true;

	FDMWidgetSlot() = default;
	FDMWidgetSlot(FSlotBase* InSlot, const TSharedRef<SWidget>& InWidget);

	FSlotBase* FindSlot(const TSharedRef<SWidget>& InParentWidget, int32 InChildSlot) const;

	void AssignWidget(const TSharedRef<SWidget>& InWidget);
};

template<typename InWidgetType>
struct TDMWidgetSlot : public FDMWidgetSlot
{
	using FWidgetType = InWidgetType;

	TDMWidgetSlot() = default;

	TDMWidgetSlot(FSlotBase* InSlot, const TSharedRef<FWidgetType>& InWidget)
		: FDMWidgetSlot(InSlot, InWidget)
	{
	}

	TDMWidgetSlot(const TSharedRef<SWidget>& InParentWidget, int32 InChildSlot, const TSharedRef<FWidgetType>& InWidget)
		: FDMWidgetSlot(FindSlot(InParentWidget, InChildSlot), InWidget)
	{
	}

	void operator<<(const TSharedRef<FWidgetType>& InWidget)
	{
		AssignWidget(InWidget);
	}

	TSharedPtr<FWidgetType> operator&() const
	{
		return StaticCastSharedPtr<FWidgetType>(Widget);
	}

	TSharedRef<FWidgetType> operator*() const
	{
		return StaticCastSharedPtr<FWidgetType>(Widget).ToSharedRef();
	}

	FWidgetType* operator->() const
	{
		return &*StaticCastSharedPtr<FWidgetType>(Widget);
	}
};

