// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/DynamicMaterialInstanceThumbnailRenderer.h"

#include "Engine/Texture2D.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Misc/App.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "ThumbnailHelpers.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UI/Utils/DynamicMaterialInstanceThumbnailScene.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMaterialInstanceThumbnailRenderer)

void UDynamicMaterialInstanceThumbnailRenderer::Draw(UObject* InObject, int32 InX, int32 InY, uint32 InWidth, uint32 InHeight, 
	FRenderTarget* InRenderTarget, FCanvas* InCanvas, bool bInAdditionalViewFamily)
{
	UDynamicMaterialInstance* MaterialInstance = Cast<UDynamicMaterialInstance>(InObject);

	if (IsValid(MaterialInstance))
	{
		if (ThumbnailScene == nullptr || ensure(ThumbnailScene->GetWorld() != nullptr) == false)
		{
			if (ThumbnailScene)
			{
				FlushRenderingCommands();
				delete ThumbnailScene;
			}

			ThumbnailScene = new FDynamicMaterialInstanceThumbnailScene();
		}

		ThumbnailScene->SetDynamicMaterialInstance(MaterialInstance);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(InRenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bInAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.SetSeparateTranslucency(false);
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.AntiAliasing = 0;

		RenderViewFamily(InCanvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, InX, InY, InWidth, InHeight));

		ThumbnailScene->SetDynamicMaterialInstance(nullptr);
		ThumbnailScene->SetStaticMesh(nullptr);
	}
}

bool UDynamicMaterialInstanceThumbnailRenderer::CanVisualizeAsset(UObject* InObject)
{
	return IsValid(InObject) && InObject->IsA<UDynamicMaterialInstance>();
}

void UDynamicMaterialInstanceThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
