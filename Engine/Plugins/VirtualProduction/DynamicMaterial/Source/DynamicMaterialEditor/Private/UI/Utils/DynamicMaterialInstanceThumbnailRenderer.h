// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"

#include "DynamicMaterialInstanceThumbnailRenderer.generated.h"

class FCanvas;
class FDynamicMaterialInstanceThumbnailScene;
class FRenderTarget;

/**
 * This thumbnail renderer displays a given Material Designer Material.
 */
UCLASS(config = Editor)
class UDynamicMaterialInstanceThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

	//~ Begin UThumbnailRenderer
	virtual void Draw(UObject* InObject, int32 InX, int32 InY, uint32 InWidth, uint32 InHeight, FRenderTarget* InRenderTarget, 
		FCanvas* InCanvas, bool bInAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* InObject) override;
	//~ End UThumbnailRenderer

	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

private:
	FDynamicMaterialInstanceThumbnailScene* ThumbnailScene;
};

