// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/DynamicMaterialInstanceThumbnailScene.h"

#include "Components/StaticMeshComponent.h"
#include "DynamicMaterialEditorSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Material/DynamicMaterialInstance.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "UnrealEdGlobals.h"

FDynamicMaterialInstanceThumbnailScene::FDynamicMaterialInstanceThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;

	UWorld* World = GetWorld();
	check(World);

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = World->SpawnActor<AStaticMeshActor>(SpawnInfo);

	PreviewActor->GetStaticMeshComponent()->SetCanEverAffectNavigation(false);
	PreviewActor->GetStaticMeshComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FDynamicMaterialInstanceThumbnailScene::SetStaticMesh(UStaticMesh* InStaticMesh)
{
	check(PreviewActor);

	UStaticMeshComponent* StaticMeshComponent = PreviewActor->GetStaticMeshComponent();
	check(StaticMeshComponent);

	StaticMeshComponent->SetStaticMesh(InStaticMesh);

	if (InStaticMesh)
	{
		FTransform MeshTransform = FTransform::Identity;

		PreviewActor->SetActorLocation(FVector(0, 0, 0), /* Sweep */ false);

		// Force LOD 0
		StaticMeshComponent->ForcedLodModel = 1;

		StaticMeshComponent->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(StaticMeshComponent->Bounds);

		PreviewActor->SetActorLocation(
			-StaticMeshComponent->Bounds.Origin + FVector(0, 0, BoundsZOffset),
			/* Sweep */ false
		);
	}

	StaticMeshComponent->RecreateRenderState_Concurrent();
}

void FDynamicMaterialInstanceThumbnailScene::SetDynamicMaterialInstance(UDynamicMaterialInstance* InMaterialInstance)
{
	check(PreviewActor);

	MaterialInstance = InMaterialInstance;

	UStaticMeshComponent* StaticMeshComponent = PreviewActor->GetStaticMeshComponent();
	check(StaticMeshComponent);

	if (!InMaterialInstance)
	{
		StaticMeshComponent->OverrideMaterials.Empty();
		return;
	}

	const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>();
	check(Settings);

	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(InMaterialInstance->ThumbnailInfo);
	EThumbnailPrimType PrimitiveType = TPT_Plane;

	if (ThumbnailInfo && ThumbnailInfo->bUserModifiedShape)
	{
		PrimitiveType = ThumbnailInfo->PrimitiveType;
	}
	else
	{
		switch (Settings->ContentBrowserThumbnail.PreviewMesh)
		{
			default:
			case EDMMaterialPreviewMesh::Plane:
				PrimitiveType = TPT_Plane;
				break;

			case EDMMaterialPreviewMesh::Cube:
				PrimitiveType = TPT_Cube;
				break;

			case EDMMaterialPreviewMesh::Sphere:
				PrimitiveType = TPT_Sphere;
				break;

			case EDMMaterialPreviewMesh::Cylinder:
				PrimitiveType = TPT_Cylinder;
				break;

			case EDMMaterialPreviewMesh::Custom:
				PrimitiveType = TPT_None;
				break;
		}
	}

	switch (PrimitiveType)
	{
		default:
		case TPT_Plane:
			SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorPlane);
			break;

		case TPT_Cube:
			SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorCube);
			break;

		case TPT_Sphere:
			SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorSphere);
			break;

		case TPT_Cylinder:
			SetStaticMesh(GUnrealEd->GetThumbnailManager()->EditorCylinder);
			break;

		case TPT_None:
		{
			SetStaticMesh(Settings->CustomPreviewMesh.LoadSynchronous());
			break;
		}
	}

	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

	if (!StaticMesh)
	{
		StaticMeshComponent->OverrideMaterials.Empty();
		return;
	}

	const int32 MaterialCount = StaticMesh->GetStaticMaterials().Num();

	StaticMeshComponent->OverrideMaterials.SetNumUninitialized(MaterialCount);

	for (int32 Index = 0; Index < MaterialCount; ++Index)
	{
		StaticMeshComponent->OverrideMaterials[Index] = InMaterialInstance;
	}

	StaticMeshComponent->MarkRenderStateDirty();
}

void FDynamicMaterialInstanceThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, 
	float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(MaterialInstance);
	check(PreviewActor);
	check(PreviewActor->GetStaticMeshComponent());
	check(PreviewActor->GetStaticMeshComponent()->GetStaticMesh());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = static_cast<float>(PreviewActor->GetStaticMeshComponent()->Bounds.SphereRadius * 1.15);
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetStaticMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(MaterialInstance->ThumbnailInfo);

	OutOrigin = FVector(0, 0, -BoundsZOffset);

	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}

		OutOrbitPitch = ThumbnailInfo->OrbitPitch;
		OutOrbitYaw = ThumbnailInfo->OrbitYaw;
		OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
	}
	else
	{
		const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>();
		check(Settings);

		switch (Settings->ContentBrowserThumbnail.PreviewMesh)
		{
			case EDMMaterialPreviewMesh::Plane:
			case EDMMaterialPreviewMesh::Cylinder:
			case EDMMaterialPreviewMesh::Sphere:
				OutOrbitPitch = 0.f;
				OutOrbitYaw = 0.f;
				OutOrbitZoom = TargetDistance;
				break;

			case EDMMaterialPreviewMesh::Cube:
				OutOrbitPitch = -30.f;
				OutOrbitYaw = 30.f;
				OutOrbitZoom = TargetDistance;
				break;

			case EDMMaterialPreviewMesh::Custom:
				OutOrbitPitch = -30.f;
				OutOrbitYaw = 152.f;
				OutOrbitZoom = TargetDistance - 409.f;
				break;
		}
	}
}