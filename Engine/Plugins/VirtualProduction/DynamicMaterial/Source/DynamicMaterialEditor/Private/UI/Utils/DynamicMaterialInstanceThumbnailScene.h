// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailHelpers.h"

#include "Math/MathFwd.h"

class AStaticMeshActor;
class UDynamicMaterialInstance;

class FDynamicMaterialInstanceThumbnailScene : public FThumbnailPreviewScene
{
public:
	FDynamicMaterialInstanceThumbnailScene();

	/** Sets the static mesh to render with. */
	void SetStaticMesh(UStaticMesh* InStaticMesh);

	/** Sets the Material Instance used on the mesh. */
	void SetDynamicMaterialInstance(UDynamicMaterialInstance* InMaterialInstance);

protected:
	//~ Begin FThumbnailPreviewScene
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, 
		float& OutOrbitYaw, float& OutOrbitZoom) const override;
	//~ End FThumbnailPreviewScene

private:
	AStaticMeshActor* PreviewActor;
	UDynamicMaterialInstance* MaterialInstance;
};
