// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/DMMaterialPreviewViewportClient.h"

#include "AdvancedPreviewScene.h"
#include "Components/StaticMeshComponent.h"
#include "UI/Widgets/Editor/SDMMaterialPreview.h"
#include "UnrealWidget.h"

FDMMaterialPreviewViewportClient::FDMMaterialPreviewViewportClient(const TSharedRef<SDMMaterialPreview>& InPreviewWidget,
	FAdvancedPreviewScene& InPreviewScene, TSharedRef<FEditorModeTools> InPreviewModeTools)
	: FEditorViewportClient(&*InPreviewModeTools, &InPreviewScene,
		StaticCastSharedRef<SEditorViewport>(InPreviewWidget))
{
	PreviewWidget = InPreviewWidget;
	PreviewModeTools = InPreviewModeTools;

	bDrawAxes = false;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = false;
	DrawHelper.GridColorAxis = FColor(80, 80, 80);
	DrawHelper.GridColorMajor = FColor(72, 72, 72);
	DrawHelper.GridColorMinor = FColor(64, 64, 64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;

	SetViewMode(VMI_Lit);
	SetRealtime(true);

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetSeparateTranslucency(true);
	EngineShowFlags.SetPivot(false);

	OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = true;

	// Don't want to display the widget in this viewport
	Widget->SetDefaultVisibility(false);

	AdvancedPreviewScene = &InPreviewScene;
}

void FDMMaterialPreviewViewportClient::FocusViewportOnBounds(const FBoxSphereBounds& InBounds, bool bInInstant)
{
	const FVector Position = InBounds.Origin;
	float Radius = InBounds.SphereRadius;

	float AspectToUse = AspectRatio;
	FIntPoint ViewportSize = Viewport->GetSizeXY();
	if (!bUseControllingActorViewInfo && ViewportSize.X > 0 && ViewportSize.Y > 0)
	{
		AspectToUse = Viewport->GetDesiredAspectRatio();
	}

	constexpr bool bEnable = false;
	ToggleOrbitCamera(bEnable);

	/**
	* We need to make sure we are fitting the sphere into the viewport completely, so if the height of the viewport is less
	* than the width of the viewport, we scale the radius by the aspect ratio in order to compensate for the fact that we have
	* less visible vertically than horizontally.
	*/
	if (AspectToUse > 1.0f)
	{
		Radius *= AspectToUse;
	}

	/**
	* Now that we have a adjusted radius, we are taking half of the viewport's FOV,
	* converting it to radians, and then figuring out the camera's distance from the center
	* of the bounding sphere using some simple trig.  Once we have the distance, we back up
	* along the camera's forward vector from the center of the sphere, and set our new view location.
	*/
	const float HalfFOVRadians = FMath::DegreesToRadians(ViewFOV / 2.0f);
	const float DistanceFromSphere = Radius / FMath::Sin(HalfFOVRadians);
	FViewportCameraTransform& ViewTransform = GetViewTransform();
	FVector CameraOffsetVector = ViewTransform.GetRotation().Vector() * -DistanceFromSphere;

	ViewTransform.SetLookAt(Position);
	ViewTransform.TransitionToLocation(Position + CameraOffsetVector, EditorViewportWidget, bInInstant);

	// Tell the viewport to redraw itself.
	Invalidate();
}

bool FDMMaterialPreviewViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	return FEditorViewportClient::InputKey(InEventArgs)
		|| AdvancedPreviewScene->HandleInputKey(InEventArgs);
}

bool FDMMaterialPreviewViewportClient::InputAxis(const FInputKeyEventArgs& InEventArgs)
{
	bool bResult = true;

	if (!bDisableInput)
	{
		bResult = AdvancedPreviewScene->HandleViewportInput(InEventArgs.Viewport, InEventArgs.InputDevice, InEventArgs.Key, InEventArgs.AmountDepressed, InEventArgs.DeltaTime, InEventArgs.NumSamples, InEventArgs.IsGamepad());

		if (bResult)
		{
			Invalidate();
		}
		else
		{
			bResult = FEditorViewportClient::InputAxis(InEventArgs);
		}
	}

	return bResult;
}

FLinearColor FDMMaterialPreviewViewportClient::GetBackgroundColor() const
{
	return AdvancedPreviewScene->GetBackgroundColor();
}

void FDMMaterialPreviewViewportClient::Tick(float InDeltaSeconds)
{
	FEditorViewportClient::Tick(InDeltaSeconds);

	// Tick the preview scene world.
	if (UWorld* World = PreviewScene->GetWorld())
	{
		if (IsValid(World))
		{
			World->Tick(LEVELTICK_All, InDeltaSeconds);
		}
	}
}

bool FDMMaterialPreviewViewportClient::ShouldOrbitCamera() const
{
	return true;
}
