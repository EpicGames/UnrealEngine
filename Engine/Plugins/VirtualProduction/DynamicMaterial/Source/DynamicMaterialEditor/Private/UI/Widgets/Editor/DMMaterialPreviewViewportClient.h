// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"

#include "Templates/SharedPointer.h"

class FAdvancedPreviewScene;
class FEditorModeTools;
class SDMMaterialPreview;

/** Based on FMaterialEditorViewportClient (private) */
class FDMMaterialPreviewViewportClient : public FEditorViewportClient
{
public:
	FDMMaterialPreviewViewportClient(const TSharedRef<SDMMaterialPreview>& InPreviewWidget, FAdvancedPreviewScene& InPreviewScene,
		TSharedRef<FEditorModeTools> InPreviewModeTools);

	/**
	* Focuses the viewport to the center of the bounding box/sphere ensuring that the entire bounds are in view
	*
	* @param InBounds   The bounds to focus
	* @param bInInstant Whether or not to focus the viewport instantly or over time
	*/
	void FocusViewportOnBounds(const FBoxSphereBounds& InBounds, bool bInInstant = false);

	//~ Begin FEditorViewportClient
	virtual bool InputKey(const FInputKeyEventArgs& InEventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& InEventArgs) override;
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float InDeltaSeconds) override;
	virtual bool ShouldOrbitCamera() const override;
	//~ End FEditorViewportClient

private:
	TWeakPtr<SDMMaterialPreview> PreviewWidget;
	FAdvancedPreviewScene* AdvancedPreviewScene;
	TSharedPtr<FEditorModeTools> PreviewModeTools;
};
