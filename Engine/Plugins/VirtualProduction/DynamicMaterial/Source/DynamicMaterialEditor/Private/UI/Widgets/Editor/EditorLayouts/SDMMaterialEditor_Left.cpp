// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_Left.h"

#include "Styling/AppStyle.h"
#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_Vertical.h"
#include "UI/Widgets/Editor/SDMMaterialPreview.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

void SDMMaterialEditor_Left::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor_LeftBase::Construct(
		SDMMaterialEditor_LeftBase::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

TSharedRef<SWidget> SDMMaterialEditor_Left::CreateSlot_Left()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SVerticalBox::FSlot* MaterialPreviewSlotPtr = nullptr;
	SVerticalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewLeft = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(5.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Expose(MaterialPreviewSlotPtr)
			.AutoHeight()
			.Padding(0.f)
			[
				SNullWidget::NullWidget
			]

			+ SVerticalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			.FillHeight(1.0f)
			.Padding(0.f, 5.f, 0.f, 0.f)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPreviewSlot = TDMWidgetSlot<SWidget>(MaterialPreviewSlotPtr, CreateSlot_Preview());
	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewLeft;
}

TSharedRef<SDMMaterialPropertySelector> SDMMaterialEditor_Left::CreateSlot_PropertySelector_Impl()
{
	return SNew(SDMMaterialPropertySelector_Vertical, SharedThis(this));
}
