// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_LeftBase.h"

class SDMMaterialEditor_Left : public SDMMaterialEditor_LeftBase
{
	SLATE_BEGIN_ARGS(SDMMaterialEditor_Left)
		: _MaterialModelBase(nullptr)
		, _MaterialProperty(TOptional<FDMObjectMaterialProperty>())
		, _PreviewMaterialModelBase(nullptr)
		{}
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, MaterialModelBase)
		SLATE_ARGUMENT(TOptional<FDMObjectMaterialProperty>, MaterialProperty)
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, PreviewMaterialModelBase)
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialEditor_Left() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget);

protected:
	//~ Begin SDMMaterialEditor_LeftBase
	virtual TSharedRef<SWidget> CreateSlot_Left() override;
	//~ End SDMMaterialEditor_LeftBase

	//~ Begin SDMMaterialEditor
	virtual TSharedRef<SDMMaterialPropertySelector> CreateSlot_PropertySelector_Impl() override;
	//~ End SDMMaterialEditor
};
