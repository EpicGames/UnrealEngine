// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_LeftBase.h"

#include "DynamicMaterialEditorSettings.h"
#include "Styling/AppStyle.h"
#include "UI/Utils/DMEditorSelectionContext.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/Editor/SDMMaterialGlobalSettingsEditor.h"
#include "UI/Widgets/Editor/SDMMaterialProperties.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"

void SDMMaterialEditor_LeftBase::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor::Construct(
		SDMMaterialEditor::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

void SDMMaterialEditor_LeftBase::EditSlot_Impl(UDMMaterialSlot* InSlot)
{
	SDMMaterialEditor::EditSlot_Impl(InSlot);

	RightSlot.Invalidate();
}

void SDMMaterialEditor_LeftBase::EditComponent_Impl(UDMMaterialComponent* InComponent)
{
	SDMMaterialEditor::EditComponent_Impl(InComponent);

	if (SelectionContext.bModeChanged)
	{
		RightSlot.Invalidate();
	}
}

void SDMMaterialEditor_LeftBase::EditGlobalSettings_Impl()
{
	SDMMaterialEditor::EditGlobalSettings_Impl();

	if (SelectionContext.bModeChanged)
	{
		RightSlot.Invalidate();
	}
}

void SDMMaterialEditor_LeftBase::EditProperties_Impl()
{
	SDMMaterialEditor::EditProperties_Impl();

	if (SelectionContext.bModeChanged)
	{
		RightSlot.Invalidate();
	}
}

void SDMMaterialEditor_LeftBase::ValidateSlots_Main()
{
	if (LeftSlot.HasBeenInvalidated())
	{
		LeftSlot << CreateSlot_Left();
	}

	if (RightSlot.HasBeenInvalidated())
	{
		RightSlot << CreateSlot_Right();
	}
}

void SDMMaterialEditor_LeftBase::ClearSlots_Main()
{
	LeftSlot.ClearWidget();
	RightSlot.ClearWidget();
}

TSharedRef<SWidget> SDMMaterialEditor_LeftBase::CreateSlot_Main()
{
	SHorizontalBox::FSlot* LeftSlotPtr = nullptr;
	SHorizontalBox::FSlot* RightSlotPtr = nullptr;

	TSharedRef<SHorizontalBox> NewMain = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Expose(LeftSlotPtr)
		.AutoWidth()
		[
			SNullWidget::NullWidget
		]

		+ SHorizontalBox::Slot()
		.Expose(RightSlotPtr)
		.FillWidth(1.0f)
		[
			SNullWidget::NullWidget
		];

	LeftSlot = TDMWidgetSlot<SWidget>(LeftSlotPtr, CreateSlot_Left());
	RightSlot = TDMWidgetSlot<SWidget>(RightSlotPtr, CreateSlot_Right());

	return NewMain;
}

TSharedRef<SWidget> SDMMaterialEditor_LeftBase::CreateSlot_Left()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SVerticalBox::FSlot* MaterialPreviewSlotPtr = nullptr;
	SVerticalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewLeft = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(5.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Expose(MaterialPreviewSlotPtr)
			.AutoHeight()
			.Padding(0.f)
			[
				SNullWidget::NullWidget
			]

			+ SVerticalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			.FillHeight(1.0f)
			.Padding(0.f, 5.f, 0.f, 0.f)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPreviewSlot = TDMWidgetSlot<SWidget>(MaterialPreviewSlotPtr, CreateSlot_Preview());
	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewLeft;
}

TSharedRef<SWidget> SDMMaterialEditor_LeftBase::CreateSlot_Right()
{
	using namespace UE::DynamicMaterialEditor::Private;

	const bool bHasSlotToEdit = SelectionContext.Slot.IsValid();

	if (SelectionContext.EditorMode == EDMMaterialEditorMode::EditSlot && !bHasSlotToEdit)
	{
		SelectionContext.EditorMode = EDMMaterialEditorMode::GlobalSettings;
	}
	else if (bHasSlotToEdit)
	{
		SelectionContext.EditorMode = EDMMaterialEditorMode::EditSlot;
	}

	TSharedPtr<SWidget> Content;

	switch (SelectionContext.EditorMode)
	{
		default:
		case EDMMaterialEditorMode::GlobalSettings:
			Content = CreateSlot_Right_GlobalSettings();
			break;

		case EDMMaterialEditorMode::Properties:
			Content = CreateSlot_Right_PropertyPreviews();
			break;

		case EDMMaterialEditorMode::EditSlot:
			Content = CreateSlot_Right_EditSlot();
			break;
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(FMargin(0.f, 5.f))
		[
			Content.ToSharedRef()
		];
}

TSharedRef<SWidget> SDMMaterialEditor_LeftBase::CreateSlot_Right_GlobalSettings()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SScrollBox::FSlot* GlobalSettingsSlotPtr = nullptr;

	TSharedRef<SBorder> NewRight = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
		.Padding(0.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.Expose(GlobalSettingsSlotPtr)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNullWidget::NullWidget
			]
		];

	GlobalSettingsEditorSlot = TDMWidgetSlot<SDMMaterialGlobalSettingsEditor>(GlobalSettingsSlotPtr, CreateSlot_GlobalSettingsEditor());

	return NewRight;
}

TSharedRef<SWidget> SDMMaterialEditor_LeftBase::CreateSlot_Right_PropertyPreviews()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SScrollBox::FSlot* PropertyPreviewsSlotPtr = nullptr;

	TSharedRef<SBorder> NewRight = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
		.Padding(0.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.Expose(PropertyPreviewsSlotPtr)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPropertiesSlot = TDMWidgetSlot<SDMMaterialProperties>(PropertyPreviewsSlotPtr, CreateSlot_MaterialProperties());

	return NewRight;
}

TSharedRef<SWidget> SDMMaterialEditor_LeftBase::CreateSlot_Right_EditSlot()
{
	using namespace UE::DynamicMaterialEditor::Private;

	float SplitterValue = 0.5;

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		SplitterValue = Settings->SplitterLocation;
	}

	TSharedPtr<SBorder> TopBox;
	TSharedPtr<SBorder> BottomBox;

	SSplitter::FSlot* ExplosedSlot = nullptr;

	TSharedRef<SSplitter> NewRight = SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.Orientation(Orient_Vertical)
		.ResizeMode(ESplitterResizeMode::Fill)
		.PhysicalSplitterHandleSize(5.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		.OnSplitterFinishedResizing(this, &SDMMaterialEditor_LeftBase::OnEditorSplitterResized)

		+ SSplitter::Slot()
		.Expose(ExplosedSlot)
		.Resizable(true)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.MinSize(165)
		.Value(SplitterValue)
		[
			SAssignNew(TopBox, SBorder)
			.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
			[
				SNullWidget::NullWidget
			]
		]

		+ SSplitter::Slot()
		.Resizable(true)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.MinSize(60)
		.Value(1.f - SplitterValue)
		[
			SAssignNew(BottomBox, SBorder)
			.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
			[
				SNullWidget::NullWidget
			]
		];

	SplitterSlot = ExplosedSlot;
	SlotEditorSlot = TDMWidgetSlot<SDMMaterialSlotEditor>(TopBox.ToSharedRef(), 0, CreateSlot_SlotEditor());
	ComponentEditorSlot = TDMWidgetSlot<SDMMaterialComponentEditor>(BottomBox.ToSharedRef(), 0, CreateSlot_ComponentEditor());

	return NewRight;
}
