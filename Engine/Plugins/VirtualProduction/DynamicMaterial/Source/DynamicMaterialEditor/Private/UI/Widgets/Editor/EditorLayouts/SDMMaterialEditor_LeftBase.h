// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/SDMMaterialEditor.h"

class SDMMaterialEditor_LeftBase : public SDMMaterialEditor
{
	SLATE_BEGIN_ARGS(SDMMaterialEditor_LeftBase)
		: _MaterialModelBase(nullptr)
		, _MaterialProperty(TOptional<FDMObjectMaterialProperty>())
		, _PreviewMaterialModelBase(nullptr)
		{}
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, MaterialModelBase)
		SLATE_ARGUMENT(TOptional<FDMObjectMaterialProperty>, MaterialProperty)
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, PreviewMaterialModelBase)
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialEditor_LeftBase() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget);

protected:
	TDMWidgetSlot<SWidget> LeftSlot;
	TDMWidgetSlot<SWidget> RightSlot;

	virtual TSharedRef<SWidget> CreateSlot_Left() = 0;

	TSharedRef<SWidget> CreateSlot_Right();

	TSharedRef<SWidget> CreateSlot_Right_GlobalSettings();

	TSharedRef<SWidget> CreateSlot_Right_PropertyPreviews();

	TSharedRef<SWidget> CreateSlot_Right_EditSlot();

	//~ Begin SDMMaterialEditor
	virtual void ValidateSlots_Main() override;
	virtual void ClearSlots_Main() override;
	virtual TSharedRef<SWidget> CreateSlot_Main() override;
	virtual void EditSlot_Impl(UDMMaterialSlot* InSlot) override;
	virtual void EditComponent_Impl(UDMMaterialComponent* InComponent) override;
	virtual void EditGlobalSettings_Impl() override;
	virtual void EditProperties_Impl() override;
	//~ End SDMMaterialEditor
};
