// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_LeftSlim.h"

#include "Styling/AppStyle.h"
#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_VerticalSlim.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

void SDMMaterialEditor_LeftSlim::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor_LeftBase::Construct(
		SDMMaterialEditor_LeftBase::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

TSharedRef<SWidget> SDMMaterialEditor_LeftSlim::CreateSlot_Left()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SVerticalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewLeft = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(5.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			[
				SNullWidget::NullWidget
			]
		];

	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewLeft;
}

TSharedRef<SDMMaterialPropertySelector> SDMMaterialEditor_LeftSlim::CreateSlot_PropertySelector_Impl()
{
	return SNew(SDMMaterialPropertySelector_VerticalSlim, SharedThis(this));
}
