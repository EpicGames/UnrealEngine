// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopBase.h"

#include "DynamicMaterialEditorSettings.h"
#include "UI/Utils/DMEditorSelectionContext.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/Editor/SDMMaterialGlobalSettingsEditor.h"
#include "UI/Widgets/Editor/SDMMaterialProperties.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"

void SDMMaterialEditor_TopBase::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor::Construct(
		SDMMaterialEditor::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

void SDMMaterialEditor_TopBase::EditSlot_Impl(UDMMaterialSlot* InSlot)
{
	SDMMaterialEditor::EditSlot_Impl(InSlot);

	BottomSlot.Invalidate();
}

void SDMMaterialEditor_TopBase::EditComponent_Impl(UDMMaterialComponent* InComponent)
{
	SDMMaterialEditor::EditComponent_Impl(InComponent);

	if (SelectionContext.bModeChanged)
	{
		BottomSlot.Invalidate();
	}
}

void SDMMaterialEditor_TopBase::EditGlobalSettings_Impl()
{
	SDMMaterialEditor::EditGlobalSettings_Impl();

	if (SelectionContext.bModeChanged)
	{
		BottomSlot.Invalidate();
	}
}

void SDMMaterialEditor_TopBase::EditProperties_Impl()
{
	SDMMaterialEditor::EditProperties_Impl();

	if (SelectionContext.bModeChanged)
	{
		BottomSlot.Invalidate();
	}
}

void SDMMaterialEditor_TopBase::ValidateSlots_Main()
{
	if (TopSlot.HasBeenInvalidated())
	{
		TopSlot << CreateSlot_Top();
	}

	if (BottomSlot.HasBeenInvalidated())
	{
		BottomSlot << CreateSlot_Bottom();
	}
}

void SDMMaterialEditor_TopBase::ClearSlots_Main()
{
	TopSlot.ClearWidget();
	BottomSlot.ClearWidget();
}

TSharedRef<SWidget> SDMMaterialEditor_TopBase::CreateSlot_Main()
{
	SVerticalBox::FSlot* TopSlotPtr = nullptr;
	SVerticalBox::FSlot* BottomSlotPtr = nullptr;

	TSharedRef<SVerticalBox> NewMain = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Expose(TopSlotPtr)
		.AutoHeight()
		[
			SNullWidget::NullWidget
		]

		+ SVerticalBox::Slot()
		.Expose(BottomSlotPtr)
		.FillHeight(1.0f)
		[
			SNullWidget::NullWidget
		];

	TopSlot = TDMWidgetSlot<SWidget>(TopSlotPtr, CreateSlot_Top());
	BottomSlot = TDMWidgetSlot<SWidget>(BottomSlotPtr, CreateSlot_Bottom());

	return NewMain;
}

TSharedRef<SWidget> SDMMaterialEditor_TopBase::CreateSlot_Top()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SVerticalBox::FSlot* MaterialPreviewSlotPtr = nullptr;
	SVerticalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewTop = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(5.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Expose(MaterialPreviewSlotPtr)
			.AutoHeight()
			.Padding(0.f)
			[
				SNullWidget::NullWidget
			]

			+ SVerticalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			.FillHeight(1.0f)
			.Padding(0.f, 5.f, 0.f, 0.f)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPreviewSlot = TDMWidgetSlot<SWidget>(MaterialPreviewSlotPtr, CreateSlot_Preview());
	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewTop;
}

TSharedRef<SWidget> SDMMaterialEditor_TopBase::CreateSlot_Bottom()
{
	using namespace UE::DynamicMaterialEditor::Private;

	const bool bHasSlotToEdit = SelectionContext.Slot.IsValid();

	if (SelectionContext.EditorMode == EDMMaterialEditorMode::EditSlot && !bHasSlotToEdit)
	{
		SelectionContext.EditorMode = EDMMaterialEditorMode::GlobalSettings;
	}
	else if (bHasSlotToEdit)
	{
		SelectionContext.EditorMode = EDMMaterialEditorMode::EditSlot;
	}

	TSharedPtr<SWidget> Content;

	switch (SelectionContext.EditorMode)
	{
		default:
		case EDMMaterialEditorMode::GlobalSettings:
			Content = CreateSlot_Bottom_GlobalSettings();
			break;

		case EDMMaterialEditorMode::Properties:
			Content = CreateSlot_Bottom_PropertyPreviews();
			break;

		case EDMMaterialEditorMode::EditSlot:
			Content = CreateSlot_Bottom_EditSlot();
			break;
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(FMargin(0.f, 5.f))
		[
			Content.ToSharedRef()
		];
}

TSharedRef<SWidget> SDMMaterialEditor_TopBase::CreateSlot_Bottom_GlobalSettings()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SScrollBox::FSlot* GlobalSettingsSlotPtr = nullptr;

	TSharedRef<SBorder> NewBottom = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
		.Padding(0.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.Expose(GlobalSettingsSlotPtr)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNullWidget::NullWidget
			]
		];

	GlobalSettingsEditorSlot = TDMWidgetSlot<SDMMaterialGlobalSettingsEditor>(GlobalSettingsSlotPtr, CreateSlot_GlobalSettingsEditor());

	return NewBottom;
}

TSharedRef<SWidget> SDMMaterialEditor_TopBase::CreateSlot_Bottom_PropertyPreviews()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SScrollBox::FSlot* PropertyPreviewsSlotPtr = nullptr;

	TSharedRef<SBorder> NewBottom = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
		.Padding(0.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.Expose(PropertyPreviewsSlotPtr)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPropertiesSlot = TDMWidgetSlot<SDMMaterialProperties>(PropertyPreviewsSlotPtr, CreateSlot_MaterialProperties());

	return NewBottom;
}

TSharedRef<SWidget> SDMMaterialEditor_TopBase::CreateSlot_Bottom_EditSlot()
{
	using namespace UE::DynamicMaterialEditor::Private;

	float SplitterValue = 0.5;

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		SplitterValue = Settings->SplitterLocation;
	}

	TSharedPtr<SBorder> TopBox;
	TSharedPtr<SBorder> BottomBox;

	SSplitter::FSlot* ExplosedSlot = nullptr;

	TSharedRef<SSplitter> NewBottom = SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.Orientation(Orient_Vertical)
		.ResizeMode(ESplitterResizeMode::Fill)
		.PhysicalSplitterHandleSize(5.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		.OnSplitterFinishedResizing(this, &SDMMaterialEditor_TopBase::OnEditorSplitterResized)

		+ SSplitter::Slot()
		.Expose(ExplosedSlot)
		.Resizable(true)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.MinSize(165)
		.Value(SplitterValue)
		[
			SAssignNew(TopBox, SBorder)
			.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
			[
				SNullWidget::NullWidget
			]
		]

		+ SSplitter::Slot()
		.Resizable(true)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.MinSize(60)
		.Value(1.f - SplitterValue)
		[
			SAssignNew(BottomBox, SBorder)
			.BorderImage(FAppStyle::GetBrush(EditorLightBackground))
			[
				SNullWidget::NullWidget
			]
		];

	SplitterSlot = ExplosedSlot;
	SlotEditorSlot = TDMWidgetSlot<SDMMaterialSlotEditor>(TopBox.ToSharedRef(), 0, CreateSlot_SlotEditor());
	ComponentEditorSlot = TDMWidgetSlot<SDMMaterialComponentEditor>(BottomBox.ToSharedRef(), 0, CreateSlot_ComponentEditor());

	return NewBottom;
}
