// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/SDMMaterialEditor.h"

class SDMMaterialEditor_TopBase : public SDMMaterialEditor
{
	SLATE_BEGIN_ARGS(SDMMaterialEditor_TopBase)
		: _MaterialModelBase(nullptr)
		, _MaterialProperty(TOptional<FDMObjectMaterialProperty>())
		, _PreviewMaterialModelBase(nullptr)
		{}
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, MaterialModelBase)
		SLATE_ARGUMENT(TOptional<FDMObjectMaterialProperty>, MaterialProperty)
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, PreviewMaterialModelBase)
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialEditor_TopBase() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget);

protected:
	TDMWidgetSlot<SWidget> TopSlot;
	TDMWidgetSlot<SWidget> BottomSlot;

	virtual TSharedRef<SWidget> CreateSlot_Top() = 0;

	TSharedRef<SWidget> CreateSlot_Bottom();

	TSharedRef<SWidget> CreateSlot_Bottom_GlobalSettings();

	TSharedRef<SWidget> CreateSlot_Bottom_PropertyPreviews();

	TSharedRef<SWidget> CreateSlot_Bottom_EditSlot();

	//~ Begin SDMMaterialEditor
	virtual void ValidateSlots_Main() override;
	virtual void ClearSlots_Main() override;
	virtual TSharedRef<SWidget> CreateSlot_Main() override;
	virtual void EditSlot_Impl(UDMMaterialSlot* InSlot) override;
	virtual void EditComponent_Impl(UDMMaterialComponent* InComponent) override;
	virtual void EditGlobalSettings_Impl() override;
	virtual void EditProperties_Impl() override;
	//~ End SDMMaterialEditor
};
