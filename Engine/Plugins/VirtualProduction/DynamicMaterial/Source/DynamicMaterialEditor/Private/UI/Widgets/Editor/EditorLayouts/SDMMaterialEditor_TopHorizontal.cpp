// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopHorizontal.h"

#include "Styling/AppStyle.h"
#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_Wrap.h"
#include "UI/Widgets/Editor/SDMMaterialPreview.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

void SDMMaterialEditor_TopHorizontal::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor_TopBase::Construct(
		SDMMaterialEditor_TopBase::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

TSharedRef<SWidget> SDMMaterialEditor_TopHorizontal::CreateSlot_Top()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SHorizontalBox::FSlot* MaterialPreviewSlotPtr = nullptr;
	SHorizontalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewLeft = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(5.f, 5.f, 5.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Expose(MaterialPreviewSlotPtr)
			.AutoWidth()
			.Padding(0.f)
			[
				SNullWidget::NullWidget
			]

			+ SHorizontalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			.FillWidth(1.0f)
			.Padding(5.f, 0.f, 0.f, 0.f)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPreviewSlot = TDMWidgetSlot<SWidget>(MaterialPreviewSlotPtr, CreateSlot_Preview());
	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewLeft;
}

TSharedRef<SDMMaterialPropertySelector> SDMMaterialEditor_TopHorizontal::CreateSlot_PropertySelector_Impl()
{
	return SNew(SDMMaterialPropertySelector_Wrap, SharedThis(this));
}
