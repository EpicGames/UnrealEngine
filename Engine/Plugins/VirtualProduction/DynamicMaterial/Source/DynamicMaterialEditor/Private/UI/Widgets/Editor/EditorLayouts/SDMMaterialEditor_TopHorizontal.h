// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopBase.h"

class SDMMaterialEditor_TopHorizontal : public SDMMaterialEditor_TopBase
{
	SLATE_BEGIN_ARGS(SDMMaterialEditor_TopHorizontal)
		: _MaterialModelBase(nullptr)
		, _MaterialProperty(TOptional<FDMObjectMaterialProperty>())
		, _PreviewMaterialModelBase(nullptr)
		{}
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, MaterialModelBase)
		SLATE_ARGUMENT(TOptional<FDMObjectMaterialProperty>, MaterialProperty)
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, PreviewMaterialModelBase)
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialEditor_TopHorizontal() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget);

protected:
	//~ Begin SDMMaterialEditor_TopBase
	virtual TSharedRef<SWidget> CreateSlot_Top() override;
	//~ End SDMMaterialEditor_TopBase

	//~ Begin SDMMaterialEditor
	virtual TSharedRef<SDMMaterialPropertySelector> CreateSlot_PropertySelector_Impl() override;
	//~ End SDMMaterialEditor
};
