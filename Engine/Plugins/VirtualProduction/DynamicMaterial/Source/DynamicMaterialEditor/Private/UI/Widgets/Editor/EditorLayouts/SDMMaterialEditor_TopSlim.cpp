// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopSlim.h"

#include "Styling/AppStyle.h"
#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_WrapSlim.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

void SDMMaterialEditor_TopSlim::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor_TopBase::Construct(
		SDMMaterialEditor_TopBase::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

TSharedRef<SWidget> SDMMaterialEditor_TopSlim::CreateSlot_Top()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SHorizontalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewLeft = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(5.f, 5.f, 5.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			.FillWidth(1.0f)
			[
				SNullWidget::NullWidget
			]
		];

	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewLeft;
}

TSharedRef<SDMMaterialPropertySelector> SDMMaterialEditor_TopSlim::CreateSlot_PropertySelector_Impl()
{
	return SNew(SDMMaterialPropertySelector_WrapSlim, SharedThis(this));
}
