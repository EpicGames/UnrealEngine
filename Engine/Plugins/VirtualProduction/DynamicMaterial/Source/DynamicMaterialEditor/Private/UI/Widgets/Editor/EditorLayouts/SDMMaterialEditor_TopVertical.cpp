// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopVertical.h"

#include "DynamicMaterialEditorSettings.h"
#include "Styling/AppStyle.h"
#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_Wrap.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"

SDMMaterialEditor_TopVertical::SDMMaterialEditor_TopVertical()
	: SplitterSlot_Top(nullptr)
{
}

void SDMMaterialEditor_TopVertical::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	SDMMaterialEditor_TopBase::Construct(
		SDMMaterialEditor_TopBase::FArguments()
			.MaterialModelBase(InArgs._MaterialModelBase)
			.MaterialProperty(InArgs._MaterialProperty)
			.PreviewMaterialModelBase(InArgs._PreviewMaterialModelBase),
		InDesignerWidget
	);
}

TSharedRef<SWidget> SDMMaterialEditor_TopVertical::CreateSlot_Top()
{
	using namespace UE::DynamicMaterialEditor::Private;

	SVerticalBox::FSlot* MaterialPreviewSlotPtr = nullptr;
	SVerticalBox::FSlot* PropertySelectorSlotPtr = nullptr;

	TSharedRef<SWidget> NewLeft = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(EditorDarkBackground))
		.Padding(FMargin(5.f, 5.f, 5.f, 0.f))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Expose(MaterialPreviewSlotPtr)
			.FillHeight(1.0f)
			.Padding(0.f)
			[
				SNullWidget::NullWidget
			]

			+ SVerticalBox::Slot()
			.Expose(PropertySelectorSlotPtr)
			.AutoHeight()
			.Padding(0.f, 5.f, 0.f, 0.f)
			[
				SNullWidget::NullWidget
			]
		];

	MaterialPreviewSlot = TDMWidgetSlot<SWidget>(MaterialPreviewSlotPtr, CreateSlot_Preview());
	PropertySelectorSlot = TDMWidgetSlot<SDMMaterialPropertySelector>(PropertySelectorSlotPtr, CreateSlot_PropertySelector());

	return NewLeft;
}

TSharedRef<SWidget> SDMMaterialEditor_TopVertical::CreateSlot_Main()
{
	using namespace UE::DynamicMaterialEditor::Private;

	float SplitterValue = 0.333f;

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		SplitterValue = Settings->PreviewSplitterLocation;
	}

	SSplitter::FSlot* SplitterSlotPtr = nullptr;
	SSplitter::FSlot* BottomSlotPtr = nullptr;
	TSharedPtr<SBox> TopBox;

	TSharedRef<SSplitter> NewMain = SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.Orientation(Orient_Vertical)
		.ResizeMode(ESplitterResizeMode::Fill)
		.PhysicalSplitterHandleSize(5.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		.OnSplitterFinishedResizing(this, &SDMMaterialEditor_TopVertical::OnTopSplitterResized)

		+ SSplitter::Slot()
		.Expose(SplitterSlotPtr)
		.Resizable(true)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.MinSize(100)
		.Value(SplitterValue)
		[
			SAssignNew(TopBox, SBox)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			.Clipping(EWidgetClipping::ClipToBoundsAlways)
			[
				SNullWidget::NullWidget
			]
		]

		+ SSplitter::Slot()
		.Expose(BottomSlotPtr)
		.Resizable(true)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.MinSize(250)
		.Value(1.f - SplitterValue)
		[
			SNullWidget::NullWidget
		];

	SplitterSlot_Top = SplitterSlotPtr;
	TopSlot = TDMWidgetSlot<SWidget>(TopBox.ToSharedRef(), 0, CreateSlot_Top());
	BottomSlot = TDMWidgetSlot<SWidget>(BottomSlotPtr, CreateSlot_Bottom());

	return NewMain;
}

TSharedRef<SDMMaterialPropertySelector> SDMMaterialEditor_TopVertical::CreateSlot_PropertySelector_Impl()
{
	return SNew(SDMMaterialPropertySelector_Wrap, SharedThis(this));
}

void SDMMaterialEditor_TopVertical::OnTopSplitterResized()
{
	if (SplitterSlot_Top)
	{
		if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
		{
			const float SplitterLocation = static_cast<SSplitter::FSlot*>(SplitterSlot_Top)->GetSizeValue();
			Settings->PreviewSplitterLocation = SplitterLocation;
			Settings->SaveConfig();
		}
	}
}
