// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_Vertical.h"

#include "DetailLayoutBuilder.h"
#include "DMDefs.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialPropertySelector_Vertical"

void SDMMaterialPropertySelector_Vertical::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget)
{
	SDMMaterialPropertySelector_VerticalBase::Construct(
		SDMMaterialPropertySelector_VerticalBase::FArguments(),
		InEditorWidget
	);
}

TSharedRef<SWidget> SDMMaterialPropertySelector_Vertical::CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage)
{
	const FText ButtonText = GetSelectButtonText(InPage, /* Short Name */ false);
	const FText ToolTip = GetButtonToolTip(InPage);

	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(0.f)
		.IsEnabled(this, &SDMMaterialPropertySelector_Vertical::GetPropertySelectEnabled, InPage)
		.IsChecked(this, &SDMMaterialPropertySelector_Vertical::GetPropertySelectState, InPage)
		.OnCheckStateChanged(this, &SDMMaterialPropertySelector_Vertical::OnPropertySelectStateChanged, InPage)
		.ToolTipText(ToolTip)
		.Content()
		[
			SNew(SBox)
			.WidthOverride(135.f)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FilterBar.FilterImage"))
					.ColorAndOpacity(this, &SDMMaterialPropertySelector_Vertical::GetPropertySelectButtonChipColor, InPage)
						.DesiredSizeOverride(FVector2D(8, 17))
				]
				+SHorizontalBox::Slot()
				.Padding(5.f, 4.f)
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(ButtonText)
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
