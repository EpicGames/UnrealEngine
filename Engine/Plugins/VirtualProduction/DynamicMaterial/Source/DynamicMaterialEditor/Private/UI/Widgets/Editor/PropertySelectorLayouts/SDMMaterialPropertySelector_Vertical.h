// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_VerticalBase.h"

struct FDMMaterialEditorPage;

class SDMMaterialPropertySelector_Vertical : public SDMMaterialPropertySelector_VerticalBase
{
	SLATE_BEGIN_ARGS(SDMMaterialPropertySelector_Vertical) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialPropertySelector_Vertical() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget);

protected:
	//~ Begin SDMMaterialPropertySelector
	virtual TSharedRef<SWidget> CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage) override;
	//~ End SDMMaterialPropertySelector
};
