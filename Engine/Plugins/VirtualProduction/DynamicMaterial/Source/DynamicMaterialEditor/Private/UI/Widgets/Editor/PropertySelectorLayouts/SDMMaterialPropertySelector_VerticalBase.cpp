// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_VerticalBase.h"

#include "Components/DMMaterialProperty.h"
#include "DMDefs.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SDMMaterialPropertySelector_VerticalBase"

void SDMMaterialPropertySelector_VerticalBase::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget)
{
	SDMMaterialPropertySelector::Construct(
		SDMMaterialPropertySelector::FArguments(),
		InEditorWidget
	);
}

TSharedRef<SWidget> SDMMaterialPropertySelector_VerticalBase::CreateSlot_PropertyList()
{
	using namespace UE::DynamicMaterialEditor::Private;

	TSharedRef<SGridPanel> NewSlotList = SNew(SGridPanel)
		.FillColumn(PropertySelectorColumns::Select, 1.f);

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return NewSlotList;
	}

	const FMargin Padding = FMargin(0.f, 1.f);

	int32 Row = 0;

	NewSlotList->AddSlot(PropertySelectorColumns::Select, Row)
		.Padding(Padding)
		[
			CreateSlot_SelectButton(FDMMaterialEditorPage::GlobalSettings)
		];

	++Row;

	NewSlotList->AddSlot(PropertySelectorColumns::Select, Row)
		.Padding(Padding)
		[
			CreateSlot_SelectButton(FDMMaterialEditorPage::Properties)
		];

	++Row;

	// Valid model properties
	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || IsCustomMaterialProperty(PropertyPair.Key))
		{
			continue;
		}

		// Skip invalid
		if (!PropertyPair.Value->IsValidForModel(*EditorOnlyData))
		{
			continue;
		}

		// Skip inactive
		if (!PropertyPair.Value->IsEnabled() || !EditorOnlyData->GetSlotForMaterialProperty(PropertyPair.Value->GetMaterialProperty()))
		{
			continue;
		}

		NewSlotList->AddSlot(PropertySelectorColumns::Enable, Row)
			[
				CreateSlot_EnabledButton(PropertyPair.Key)
			];

		NewSlotList->AddSlot(PropertySelectorColumns::Select, Row)
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];

		++Row;
	}

	// Disabled model properties
	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || IsCustomMaterialProperty(PropertyPair.Key))
		{
			continue;
		}

		// Skip invalid
		if (!PropertyPair.Value->IsValidForModel(*EditorOnlyData))
		{
			continue;
		}

		// Skip active
		if (PropertyPair.Value->IsEnabled() && EditorOnlyData->GetSlotForMaterialProperty(PropertyPair.Value->GetMaterialProperty()))
		{
			continue;
		}

		NewSlotList->AddSlot(PropertySelectorColumns::Enable, Row)
			[
				CreateSlot_EnabledButton(PropertyPair.Key)
			];

		NewSlotList->AddSlot(PropertySelectorColumns::Select, Row)
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];

		++Row;
	}

	// Invalid model properties
	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || IsCustomMaterialProperty(PropertyPair.Key))
		{
			continue;
		}

		// Skip valid
		if (PropertyPair.Value->IsValidForModel(*EditorOnlyData))
		{
			continue;
		}

		NewSlotList->AddSlot(PropertySelectorColumns::Enable, Row)
			[
				CreateSlot_EnabledButton(PropertyPair.Key)
			];

		NewSlotList->AddSlot(PropertySelectorColumns::Select, Row)
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];

		++Row;
	}

	return NewSlotList;
}

#undef LOCTEXT_NAMESPACE
