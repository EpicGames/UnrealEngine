// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"

class SDMMaterialPropertySelector_VerticalBase : public SDMMaterialPropertySelector
{
	SLATE_BEGIN_ARGS(SDMMaterialPropertySelector_VerticalBase) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialPropertySelector_VerticalBase() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget);

protected:
	//~ Begin SDMMaterialPropertySelector
	virtual TSharedRef<SWidget> CreateSlot_PropertyList() override;
	//~ End SDMMaterialPropertySelector
};
