// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_VerticalSlim.h"

#include "Components/DMMaterialProperty.h"
#include "DetailLayoutBuilder.h"
#include "DMDefs.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialPropertySelector_VerticalSlim"

void SDMMaterialPropertySelector_VerticalSlim::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget)
{
	SDMMaterialPropertySelector_VerticalBase::Construct(
		SDMMaterialPropertySelector_VerticalBase::FArguments(),
		InEditorWidget
	);
}

TSharedRef<SWidget> SDMMaterialPropertySelector_VerticalSlim::CreateSlot_PropertyList()
{
	using namespace UE::DynamicMaterialEditor::Private;

	TSharedRef<SVerticalBox> NewSlotList = SNew(SVerticalBox);

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return NewSlotList;
	}

	const FMargin Padding = FMargin(0.f, 1.f);

	TSharedRef<SWidget> PreviewButton = CreateSlot_SelectButton(FDMMaterialEditorPage::Preview);
	SetupMaterialPreviewButton(PreviewButton);

	NewSlotList->AddSlot()
		.AutoHeight()
		.Padding(Padding)
		[
			PreviewButton
		];

	NewSlotList->AddSlot()
		.AutoHeight()
		.Padding(Padding)
		[
			CreateSlot_SelectButton(FDMMaterialEditorPage::GlobalSettings)
		];

	NewSlotList->AddSlot()
		.AutoHeight()
		.Padding(Padding)
		[
			CreateSlot_SelectButton(FDMMaterialEditorPage::Properties)
		];

	// Valid model properties
	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || IsCustomMaterialProperty(PropertyPair.Key))
		{
			continue;
		}

		// Skip invalid
		if (!PropertyPair.Value->IsValidForModel(*EditorOnlyData))
		{
			continue;
		}

		// Skip inactive
		if (!PropertyPair.Value->IsEnabled() || !EditorOnlyData->GetSlotForMaterialProperty(PropertyPair.Value->GetMaterialProperty()))
		{
			continue;
		}

		NewSlotList->AddSlot()
			.AutoHeight()
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];
	}

	// Disabled model properties
	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || IsCustomMaterialProperty(PropertyPair.Key))
		{
			continue;
		}

		// Skip invalid
		if (!PropertyPair.Value->IsValidForModel(*EditorOnlyData))
		{
			continue;
		}

		// Skip active
		if (PropertyPair.Value->IsEnabled() && EditorOnlyData->GetSlotForMaterialProperty(PropertyPair.Value->GetMaterialProperty()))
		{
			continue;
		}

		NewSlotList->AddSlot()
			.AutoHeight()
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];
	}

	// Invalid model properties
	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || IsCustomMaterialProperty(PropertyPair.Key))
		{
			continue;
		}

		// Skip valid
		if (PropertyPair.Value->IsValidForModel(*EditorOnlyData))
		{
			continue;
		}

		NewSlotList->AddSlot()
			.AutoHeight()
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];
	}

	return NewSlotList;
}

TSharedRef<SWidget> SDMMaterialPropertySelector_VerticalSlim::CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage)
{
	const FText ButtonText = GetSelectButtonText(InPage, /* Short Name */ true);
	const FText ToolTip = GetButtonToolTip(InPage);

	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(0.f)
		.IsEnabled(this, &SDMMaterialPropertySelector_VerticalSlim::GetPropertySelectEnabled, InPage)
		.IsChecked(this, &SDMMaterialPropertySelector_VerticalSlim::GetPropertySelectState, InPage)
		.OnCheckStateChanged(this, &SDMMaterialPropertySelector_VerticalSlim::OnPropertySelectStateChanged, InPage)
		.ToolTipText(ToolTip)
		.Content()
		[
			SNew(SBox)
			.WidthOverride(45.f)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FilterBar.FilterImage"))
					.ColorAndOpacity(this, &SDMMaterialPropertySelector_VerticalSlim::GetPropertySelectButtonChipColor, InPage)
					.DesiredSizeOverride(FVector2D(8, 17))
				]
				+SHorizontalBox::Slot()
				.Padding(2.f, 4.f)
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(ButtonText)
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
