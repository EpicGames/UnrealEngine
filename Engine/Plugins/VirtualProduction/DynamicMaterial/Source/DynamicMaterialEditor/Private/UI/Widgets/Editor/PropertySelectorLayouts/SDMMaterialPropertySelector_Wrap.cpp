// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_Wrap.h"

#define LOCTEXT_NAMESPACE "SDMMaterialPropertySelector_Wrap"

void SDMMaterialPropertySelector_Wrap::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget)
{
	SDMMaterialPropertySelector_WrapBase::Construct(
		SDMMaterialPropertySelector_WrapBase::FArguments(),
		InEditorWidget
	);
}

#undef LOCTEXT_NAMESPACE
