// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_WrapBase.h"

class SDMMaterialPropertySelector_Wrap : public SDMMaterialPropertySelector_WrapBase
{
	SLATE_BEGIN_ARGS(SDMMaterialPropertySelector_Wrap) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialPropertySelector_Wrap() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget);
};
