// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"

struct FDMMaterialEditorPage;

class SDMMaterialPropertySelector_WrapBase : public SDMMaterialPropertySelector
{
	SLATE_BEGIN_ARGS(SDMMaterialPropertySelector_WrapBase) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialPropertySelector_WrapBase() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget);

protected:
	//~ Begin SDMMaterialPropertySelector
	virtual TSharedRef<SWidget> CreateSlot_PropertyList() override;
	virtual TSharedRef<SWidget> CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage) override;
	//~ End SDMMaterialPropertySelector
};
