// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_WrapSlim.h"

#include "Components/DMMaterialProperty.h"
#include "DetailLayoutBuilder.h"
#include "DynamicMaterialEditorSettings.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialPropertySelector_WrapSlim"

void SDMMaterialPropertySelector_WrapSlim::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget)
{
	SDMMaterialPropertySelector_WrapBase::Construct(
		SDMMaterialPropertySelector_WrapBase::FArguments(),
		InEditorWidget
	);
}

TSharedRef<SWidget> SDMMaterialPropertySelector_WrapSlim::CreateSlot_PropertyList()
{
	using namespace UE::DynamicMaterialEditor::Private;

	TSharedRef<SWrapBox> NewSlotList = SNew(SWrapBox)
		.InnerSlotPadding(FVector2D(6.f, 3.f))
		.UseAllottedSize(true);

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return NewSlotList;
	}

	const FMargin Padding = FMargin(0.f, 1.f);

	TSharedRef<SWidget> PreviewButton = CreateSlot_SelectButton(FDMMaterialEditorPage::Preview);
	SetupMaterialPreviewButton(PreviewButton);

	NewSlotList->AddSlot()
		.Padding(Padding)
		[
			PreviewButton
		];

	NewSlotList->AddSlot()
		.Padding(Padding)
		[
			CreateSlot_SelectButton(FDMMaterialEditorPage::GlobalSettings)
		];

	NewSlotList->AddSlot()
		.Padding(Padding)
		[
			CreateSlot_SelectButton(FDMMaterialEditorPage::Properties)
		];

	for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
	{
		if (!PropertyPair.Value || !PropertyPair.Value->IsEnabled() || IsCustomMaterialProperty(PropertyPair.Key)
			|| !PropertyPair.Value->IsValidForModel(*EditorOnlyData)
			|| !EditorOnlyData->GetSlotForMaterialProperty(PropertyPair.Value->GetMaterialProperty()))
		{
			continue;
		}

		NewSlotList->AddSlot()
			.Padding(Padding)
			[
				CreateSlot_SelectButton({EDMMaterialEditorMode::EditSlot, PropertyPair.Key})
			];
	}

	return NewSlotList;
}

TSharedRef<SWidget> SDMMaterialPropertySelector_WrapSlim::CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage)
{
	if (const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>())
	{
		if (Settings->bUseFullChannelNamesInTopSlimLayout)
		{
			return SDMMaterialPropertySelector_WrapBase::CreateSlot_SelectButton(InPage);
		}
	}

	const FText ButtonText = GetSelectButtonText(InPage, /* Short Name */ true);
	const FText ToolTip = GetButtonToolTip(InPage);


	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(0.f)
		.IsEnabled(this, &SDMMaterialPropertySelector_WrapSlim::GetPropertySelectEnabled, InPage)
		.IsChecked(this, &SDMMaterialPropertySelector_WrapSlim::GetPropertySelectState, InPage)
		.OnCheckStateChanged(this, &SDMMaterialPropertySelector_WrapSlim::OnPropertySelectStateChanged, InPage)
		.ToolTipText(ToolTip)
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("FilterBar.FilterImage"))
				.ColorAndOpacity(this, &SDMMaterialPropertySelector_WrapSlim::GetPropertySelectButtonChipColor, InPage)
				.DesiredSizeOverride(FVector2D(8, 17))
			]
			+SHorizontalBox::Slot()
			.Padding(4.f, 4.f)
			.VAlign(VAlign_Center)
			.FillWidth(1.f)
			[
				SNew(SBox)
				.WidthOverride(32.f)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(ButtonText)
					.Justification(ETextJustify::Center)
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
