// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/PropertySelectorLayouts/SDMMaterialPropertySelector_WrapBase.h"

class SDMMaterialPropertySelector_WrapSlim : public SDMMaterialPropertySelector_WrapBase
{
	SLATE_BEGIN_ARGS(SDMMaterialPropertySelector_WrapSlim) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialPropertySelector_WrapSlim() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget);

protected:
	//~ Begin SDMMaterialPropertySelector
	virtual TSharedRef<SWidget> CreateSlot_PropertyList() override;
	virtual TSharedRef<SWidget> CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage) override;
	//~ End SDMMaterialPropertySelector
};
