// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialValue.h"
#include "CustomDetailsViewArgs.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialModule.h"
#include "ICustomDetailsView.h"
#include "Items/ICustomDetailsViewCustomCategoryItem.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Model/DynamicMaterialModelBase.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SDMMaterialComponentEditor"

void SDMMaterialComponentEditor::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialComponentEditor::~SDMMaterialComponentEditor()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDMMaterialComponent* Component = GetComponent())
	{
		Component->GetOnUpdate().RemoveAll(this);
	}
}

void SDMMaterialComponentEditor::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, 
	UDMMaterialComponent* InMaterialComponent)
{
	SetCanTick(false);

	SDMObjectEditorWidgetBase::Construct(
		SDMObjectEditorWidgetBase::FArguments(), 
		InEditorWidget, 
		InMaterialComponent
	);

	if (InMaterialComponent)
	{
		InMaterialComponent->GetOnUpdate().AddSP(this, &SDMMaterialComponentEditor::OnComponentUpdated);
	}
}

UDMMaterialComponent* SDMMaterialComponentEditor::GetComponent() const
{
	return Cast<UDMMaterialComponent>(ObjectWeak.Get());
}

UDMMaterialComponent* SDMMaterialComponentEditor::GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const
{
	if (!InPreviewComponent)
	{
		return nullptr;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	return EditorWidget->GetOriginalComponent(InPreviewComponent);
}

void SDMMaterialComponentEditor::NotifyPreChange(FProperty* InPropertyAboutToChange)
{
	SDMObjectEditorWidgetBase::NotifyPreChange(InPropertyAboutToChange);

	if (UDMMaterialComponent* Component = GetComponent())
	{
		Component->NotifyPreChange(InPropertyAboutToChange);

		if (UDMMaterialComponent* OriginalComponent = GetOriginalComponent(Component))
		{
			OriginalComponent->NotifyPreChange(InPropertyAboutToChange);
		}
	}
}

void SDMMaterialComponentEditor::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	SDMObjectEditorWidgetBase::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (UDMMaterialComponent* Component = GetComponent())
	{
		Component->NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

		if (UDMMaterialComponent* OriginalComponent = GetOriginalComponent(Component))
		{			
			OriginalComponent->NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
		}
	}
}

void SDMMaterialComponentEditor::OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure | EDMUpdateType::RefreshDetailView))
	{
		if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
		{
			EditorWidget->EditComponent(GetComponent(), /* Force refresh */ true);
		}
	}

	// Don't copy parameters if it's just a details panel refresh.
	if (InUpdateType != EDMUpdateType::RefreshDetailView)
	{
		if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
		{
			if (Settings->ShouldAutomaticallyCopyParametersToSourceMaterial())
			{
				if (InComponent)
				{
					if (UDMMaterialComponent* OriginalComponent = GetOriginalComponent(InComponent))
					{
						if (GUndo)
						{
							OriginalComponent->Modify();
						}

						IDMParameterContainer::CopyParametersBetween(InComponent, OriginalComponent);
						return;
					}
				}
			}
		}
	}

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		if (UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetOriginalMaterialModelBase())
		{
			MaterialModelBase->MarkPreviewModified();
		}
	}	
}

TSharedRef<ICustomDetailsViewItem> SDMMaterialComponentEditor::GetDefaultCategory(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const FCustomDetailsViewItemId& InRootId)
{
	UDMMaterialComponent* Component = GetComponent();

	if (!Component)
	{
		return SDMObjectEditorWidgetBase::GetDefaultCategory(InDetailsView, InRootId);
	}

	if (!DefaultCategoryItem.IsValid())
	{
		const FText ComponentCategoryFormat = LOCTEXT("ComponantCategoryFormat", "{0} Settings");
		const FText ComponentCategoryText = FText::Format(ComponentCategoryFormat, Component->GetComponentDescription());
		DefaultCategoryItem = InDetailsView->CreateCustomCategoryItem(InDetailsView->GetRootItem(), DefaultCategoryName,
			ComponentCategoryText)->AsItem();
		InDetailsView->ExtendTree(InRootId, ECustomDetailsTreeInsertPosition::Child, DefaultCategoryItem.ToSharedRef());

		bool bExpansionState = true;
		FDMWidgetLibrary::Get().GetExpansionState(ObjectWeak.Get(), DefaultCategoryName, bExpansionState);

		InDetailsView->SetItemExpansionState(
			DefaultCategoryItem->GetItemId(),
			bExpansionState ? ECustomDetailsViewExpansion::SelfExpanded : ECustomDetailsViewExpansion::Collapsed
		);

		Categories.Add(DefaultCategoryName);
	}

	return DefaultCategoryItem.ToSharedRef();
}

TArray<FDMPropertyHandle> SDMMaterialComponentEditor::GetPropertyRows()
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget)
	{
		return {};
	}

	TArray<FDMPropertyHandle> PropertyRows;
	TSet<UObject*> ProcessedObjects;

	FDMComponentPropertyRowGeneratorParams Params(PropertyRows, ProcessedObjects);
	Params.NotifyHook = this;
	Params.Owner = this;
	Params.Object = GetComponent();
	Params.PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();
	Params.OriginalMaterialModelBase = EditorWidget->GetOriginalMaterialModelBase();

	FDynamicMaterialEditorModule::GeneratorComponentPropertyRows(Params);

	BindPropertyRowUpdateDelegates(PropertyRows);

	return PropertyRows;
}

void SDMMaterialComponentEditor::BindPropertyRowUpdateDelegates(TConstArrayView<FDMPropertyHandle> InPropertyRows)
{
	for (const FDMPropertyHandle& PropertyRow : InPropertyRows)
	{
		TArray<UObject*> Outers;

		if (PropertyRow.PreviewHandle.PropertyHandle.IsValid())
		{
			PropertyRow.PreviewHandle.PropertyHandle->GetOuterObjects(Outers);
		}
		else if (PropertyRow.PreviewHandle.PropertyRowGenerator.IsValid())
		{
			for (const TWeakObjectPtr<UObject>& WeakObject : PropertyRow.PreviewHandle.PropertyRowGenerator->GetSelectedObjects())
			{
				if (UObject* Object = WeakObject.Get())
				{
					Outers.Add(Object);
				}
			}
		}

		for (UObject* Outer : Outers)
		{
			UDMMaterialComponent* Component = Cast<UDMMaterialComponent>(Outer);

			if (!Component)
			{
				Component = Outer->GetTypedOuter<UDMMaterialComponent>();
			}

			if (Component)
			{
				if (!Component->GetOnUpdate().IsBoundToObject(this))
				{
					Component->GetOnUpdate().AddSP(this, &SDMMaterialComponentEditor::OnComponentUpdated);
				}
			}
		}
	}
}

void SDMMaterialComponentEditor::AddDetailTreeRowExtensionWidgets(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const FDMPropertyHandle& InPropertyRow, const TSharedRef<ICustomDetailsViewItem>& InItem, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	TArray<FPropertyRowExtensionButton> ExtensionButtons;

	// Reset to Default
	if (InPropertyRow.ResetToDefaultOverride.IsSet())
	{
		FPropertyRowExtensionButton ResetToDefaultButton;

		if (InItem->CreateResetToDefaultButton(ResetToDefaultButton))
		{
			ExtensionButtons.Add(MoveTemp(ResetToDefaultButton));
		}
	}

	// Global Extensions
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FOnGenerateGlobalRowExtensionArgs RowExtensionArgs;
	RowExtensionArgs.OwnerTreeNode = InPropertyRow.OriginalHandle.DetailTreeNode;
	RowExtensionArgs.PropertyHandle = InPropertyRow.OriginalHandle.PropertyHandle;
	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(RowExtensionArgs, ExtensionButtons);

	bool bValidPropertyHandles = false;

	if (InPropertyRow.PreviewHandle.PropertyHandle.IsValid()
		&& InPropertyRow.OriginalHandle.PropertyHandle.IsValid())
	{
		TArray<UObject*> OuterPreview;
		TArray<UObject*> OuterOriginal;

		InPropertyRow.PreviewHandle.PropertyHandle->GetOuterObjects(OuterPreview);
		InPropertyRow.OriginalHandle.PropertyHandle->GetOuterObjects(OuterOriginal);

		if (OuterPreview.Num() == 1 && OuterOriginal.Num() == 1
			&& IsValid(OuterPreview[0]) && IsValid(OuterOriginal[0])
			&& OuterPreview[0]->GetClass() == OuterOriginal[0]->GetClass())
		{
			bValidPropertyHandles = true;

			// Sequencer relies on getting the Keyframe Handler via the Details View of the IDetailTreeNode, but is null since there's no Details View here
			// instead add it manually here
			if (InPropertyRow.bKeyframeable)
			{
				if (TOptional<FPropertyRowExtensionButton> CreateKeyButton = CreateKeyframeButton(
					InPropertyRow.PreviewHandle.PropertyHandle,
					InPropertyRow.OriginalHandle.PropertyHandle))
				{
					ExtensionButtons.Add(*CreateKeyButton);
				}
			}
		}
	}

	if (!bValidPropertyHandles)
	{
		ExtensionButtons.Add(CreateNeedsApplyButton());
	}		

	if (ExtensionButtons.IsEmpty())
	{
		InItem->SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, SNullWidget::NullWidget);
		return;
	}

	InItem->SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, InItem->CreateExtensionButtonWidget(ExtensionButtons));
}

#undef LOCTEXT_NAMESPACE
