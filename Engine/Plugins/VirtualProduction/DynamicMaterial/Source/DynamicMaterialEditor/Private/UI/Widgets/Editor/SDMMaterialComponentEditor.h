// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/SDMObjectEditorWidgetBase.h"

class UDMMaterialComponent;
enum class EDMUpdateType : uint8;
struct FPropertyRowExtensionButton;

/** Extends the object editor to provide component-specific events and properties. */
class SDMMaterialComponentEditor : public SDMObjectEditorWidgetBase
{
	SLATE_DECLARE_WIDGET(SDMMaterialComponentEditor, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialComponentEditor) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialComponentEditor() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialComponent* InMaterialComponent);

	UDMMaterialComponent* GetComponent() const;

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	void OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	UDMMaterialComponent* GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const;

	void BindPropertyRowUpdateDelegates(TConstArrayView<FDMPropertyHandle> InPropertyRows);

	//~ Begin SDMObjectEditorWidgetBase
	virtual TSharedRef<ICustomDetailsViewItem> GetDefaultCategory(const TSharedRef<ICustomDetailsView>& InDetailsView,
		const FCustomDetailsViewItemId& InRootId) override;
	virtual TArray<FDMPropertyHandle> GetPropertyRows() override;
	virtual void AddDetailTreeRowExtensionWidgets(const TSharedRef<ICustomDetailsView>& InDetailsView, const FDMPropertyHandle& InPropertyRow,
		const TSharedRef<ICustomDetailsViewItem>& InItem, const TSharedRef<IPropertyHandle>& InPropertyHandle) override;
	//~ End SDMObjectEditorWidgetBase
};
