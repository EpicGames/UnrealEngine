// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMMaterialGlobalSettingsEditor.h"

#include "Components/DMMaterialComponent.h"
#include "Components/DMMaterialValue.h"
#include "DMEDefs.h"
#include "DynamicMaterialEditorSettings.h"
#include "ICustomDetailsView.h"
#include "IDynamicMaterialEditorModule.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelBase.h"
#include "UI/PropertyGenerators/DMMaterialModelPropertyRowGenerator.h"
#include "UI/Widgets/SDMMaterialEditor.h"

#define LOCTEXT_NAMESPACE "SDMMaterialGlobalSettingsEditor"

void SDMMaterialGlobalSettingsEditor::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SDMMaterialGlobalSettingsEditor::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDynamicMaterialModelBase* InMaterialModelBase)
{
	SetCanTick(false);

	SDMObjectEditorWidgetBase::Construct(
		SDMObjectEditorWidgetBase::FArguments(), 
		InEditorWidget, 
		InMaterialModelBase
	);
}

UDynamicMaterialModelBase* SDMMaterialGlobalSettingsEditor::GetMaterialModelBase() const
{
	return Cast<UDynamicMaterialModelBase>(ObjectWeak.Get());
}

UDynamicMaterialModelBase* SDMMaterialGlobalSettingsEditor::GetOriginalMaterialModelBase() const
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	return EditorWidget->GetOriginalMaterialModelBase();
}

void SDMMaterialGlobalSettingsEditor::NotifyPreChange(FProperty* InPropertyAboutToChange)
{
	SDMObjectEditorWidgetBase::NotifyPreChange(InPropertyAboutToChange);

	UDynamicMaterialModel* PreviewMaterialModel = Cast<UDynamicMaterialModel>(GetMaterialModelBase());

	if (!PreviewMaterialModel)
	{
		return;
	}

	// We can't know which component it is, so call this on all of them.
	TArray<UDMMaterialValue*> Values = {
		PreviewMaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOffsetValueName),
		PreviewMaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalTilingValueName),
		PreviewMaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalRotationValueName)
	};

	for (UDMMaterialValue* Value : Values)
	{
		Value->NotifyPreChange(InPropertyAboutToChange);

		if (UDMMaterialValue* OriginalValue = Cast<UDMMaterialValue>(GetOriginalComponent(Value)))
		{
			OriginalValue->NotifyPreChange(InPropertyAboutToChange);
		}
	}
}

void SDMMaterialGlobalSettingsEditor::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	SDMObjectEditorWidgetBase::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	UDynamicMaterialModel* PreviewMaterialModel = Cast<UDynamicMaterialModel>(GetMaterialModelBase());

	if (!PreviewMaterialModel)
	{
		return;
	}

	// We called pre-update on all of them, so call post change on all of them (or things break).
	TArray<UDMMaterialValue*> Values = {
		PreviewMaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOffsetValueName),
		PreviewMaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalTilingValueName),
		PreviewMaterialModel->GetGlobalParameterValue(UDynamicMaterialModel::GlobalRotationValueName)
	};

	for (UDMMaterialValue* Value : Values)
	{
		Value->NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

		if (UDMMaterialValue* OriginalValue = Cast<UDMMaterialValue>(GetOriginalComponent(Value)))
		{
			OriginalValue->NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
		}
	}
}

UDMMaterialComponent* SDMMaterialGlobalSettingsEditor::GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const
{
	if (!InPreviewComponent)
	{
		return nullptr;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	return EditorWidget->GetOriginalComponent(InPreviewComponent);
}

void SDMMaterialGlobalSettingsEditor::OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource,
	EDMUpdateType InUpdateType)
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->ShouldAutomaticallyCopyParametersToSourceMaterial())
		{
			if (InComponent)
			{
				if (UDMMaterialComponent* OriginalComponent = GetOriginalComponent(InComponent))
				{
					IDMParameterContainer::CopyParametersBetween(InComponent, OriginalComponent);
					return;
				}
			}
		}
	}

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		if (UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetOriginalMaterialModelBase())
		{
			MaterialModelBase->MarkPreviewModified();
		}
	}
}

TArray<FDMPropertyHandle> SDMMaterialGlobalSettingsEditor::GetPropertyRows()
{
	TArray<FDMPropertyHandle> PropertyRows;
	TSet<UObject*> ProcessedObjects;

	FDMComponentPropertyRowGeneratorParams Params(PropertyRows, ProcessedObjects);
	Params.Owner = this;
	Params.NotifyHook = this;
	Params.Object = GetMaterialModelBase();
	Params.PreviewMaterialModelBase = GetMaterialModelBase();
	Params.OriginalMaterialModelBase = GetOriginalMaterialModelBase();

	FDMMaterialModelPropertyRowGenerator::AddMaterialModelProperties(Params);

	return PropertyRows;
}

void SDMMaterialGlobalSettingsEditor::AddDetailTreeRowExtensionWidgets(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const FDMPropertyHandle& InPropertyRow, const TSharedRef<ICustomDetailsViewItem>& InItem, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	if (!InPropertyRow.PreviewHandle.PropertyHandle.IsValid())
	{
		return;
	}

	TArray<UObject*> OuterPreview;
	InPropertyRow.PreviewHandle.PropertyHandle->GetOuterObjects(OuterPreview);

	if (OuterPreview.Num() != 1 || !IsValid(OuterPreview[0]))
	{
		return;
	}

	TArray<FPropertyRowExtensionButton> ExtensionButtons;

	// Reset to Default
	if (InPropertyRow.ResetToDefaultOverride.IsSet())
	{
		FPropertyRowExtensionButton ResetToDefaultButton;

		if (InItem->CreateResetToDefaultButton(ResetToDefaultButton))
		{
			ExtensionButtons.Add(MoveTemp(ResetToDefaultButton));
		}
	}

	// Global Extensions
	if (!OuterPreview[0]->IsA<UDynamicMaterialModelBase>())
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FOnGenerateGlobalRowExtensionArgs RowExtensionArgs;
		RowExtensionArgs.OwnerTreeNode = InPropertyRow.OriginalHandle.DetailTreeNode;
		RowExtensionArgs.PropertyHandle = InPropertyRow.OriginalHandle.PropertyHandle;
		PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(RowExtensionArgs, ExtensionButtons);
	}

	bool bValidPropertyHandles = false;
	TArray<UObject*> OuterOriginal;

	if (InPropertyRow.OriginalHandle.PropertyHandle.IsValid())
	{
		InPropertyRow.OriginalHandle.PropertyHandle->GetOuterObjects(OuterOriginal);
	}

	if (OuterOriginal.Num() == 1 && IsValid(OuterOriginal[0]) && OuterPreview[0]->GetClass() == OuterOriginal[0]->GetClass())
	{
		bValidPropertyHandles = true;

		// Sequencer relies on getting the Keyframe Handler via the Details View of the IDetailTreeNode, but is null since there's no Details View here
		// instead add it manually here
		if (InPropertyRow.bKeyframeable)
		{
			if (TOptional<FPropertyRowExtensionButton> CreateKeyButton = CreateKeyframeButton(
						InPropertyRow.PreviewHandle.PropertyHandle,
						InPropertyRow.OriginalHandle.PropertyHandle))
			{
				ExtensionButtons.Add(*CreateKeyButton);
			}

			if (UDMMaterialComponent* PreviewComponent = Cast<UDMMaterialComponent>(OuterPreview[0]))
			{
				if (!PreviewComponent->GetOnUpdate().IsBoundToObject(this))
				{
					PreviewComponent->GetOnUpdate().AddSP(this, &SDMMaterialGlobalSettingsEditor::OnComponentUpdated);
				}
			}
		}
	}

	if (!bValidPropertyHandles)
	{
		ExtensionButtons.Add(CreateNeedsApplyButton());
	}

	if (ExtensionButtons.IsEmpty())
	{
		InItem->SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, SNullWidget::NullWidget);
		return;
	}

	InItem->SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, InItem->CreateExtensionButtonWidget(ExtensionButtons));
}

#undef LOCTEXT_NAMESPACE
