// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Widgets/Editor/SDMObjectEditorWidgetBase.h"

class UDMMaterialComponent;
class UDynamicMaterialModelBase;
enum class EDMUpdateType : uint8;

class SDMMaterialGlobalSettingsEditor : public SDMObjectEditorWidgetBase
{
	SLATE_DECLARE_WIDGET(SDMMaterialGlobalSettingsEditor, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialGlobalSettingsEditor) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialGlobalSettingsEditor() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, 
		UDynamicMaterialModelBase* InMaterialModelBase);

	UDynamicMaterialModelBase* GetMaterialModelBase() const;

	UDynamicMaterialModelBase* GetOriginalMaterialModelBase() const;

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	UDMMaterialComponent* GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const;

	void OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	//~ Begin SDMObjectEditorWidgetBase
	virtual TArray<FDMPropertyHandle> GetPropertyRows() override;
	virtual void AddDetailTreeRowExtensionWidgets(const TSharedRef<ICustomDetailsView>& InDetailsView, const FDMPropertyHandle& InPropertyRow,
		const TSharedRef<ICustomDetailsViewItem>& InItem, const TSharedRef<IPropertyHandle>& InPropertyHandle) override;
	//~ End SDMObjectEditorWidgetBase
};
