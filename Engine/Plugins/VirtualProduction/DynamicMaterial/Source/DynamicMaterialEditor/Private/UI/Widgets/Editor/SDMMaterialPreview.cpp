// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMMaterialPreview.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "AssetViewerSettings.h"
#include "Components/StaticMeshComponent.h"
#include "DynamicMaterialEditorCommands.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialModule.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "Engine/Engine.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Material/DynamicMaterialInstance.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "MaterialEditorActions.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Slate/SceneViewport.h"
#include "Styling/SlateIconFinder.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UI/Widgets/Editor/DMMaterialPreviewViewportClient.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UnrealEdGlobals.h"
#include "Widgets/SNullWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SDMMaterialPreview)

#define LOCTEXT_NAMESPACE "SDMMaterialPreview"

void SDMMaterialPreview::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialPreview::~SDMMaterialPreview()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (PreviewMeshComponent != nullptr)
	{
		PreviewMeshComponent->OverrideMaterials.Empty();
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (UEditorEngine* Editor = CastChecked<UEditorEngine>(GEngine))
	{
		Editor->OnPreviewShaderPlatformChanged().RemoveAll(this);
	}

	if (UDynamicMaterialEditorSettings* EditorSettings = UDynamicMaterialEditorSettings::Get())
	{
		EditorSettings->GetOnSettingsChanged().RemoveAll(this);
	}
}

void SDMMaterialPreview::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, 
	UDynamicMaterialModelBase* InMaterialModelBase)
{
	EditorWidgetWeak = InEditorWidget;
	bShowMenu = InArgs._ShowMenu;
	bIsPopout = InArgs._IsPopout;
	PreviewScene = MakeShareable(new FAdvancedPreviewScene(
		FPreviewScene::ConstructionValues()
		.SetCreatePhysicsScene(false)
		.ShouldSimulatePhysics(false)
	));

	PreviewMaterial = nullptr;
	PreviewMeshComponent = nullptr;
	PostProcessVolumeActor = nullptr;

	SetCanTick(false);

	SEditorViewport::Construct(SEditorViewport::FArguments().ViewportSize(FVector2D(135.f)));

	// restore last used feature level
	UWorld* PreviewWorld = PreviewScene->GetWorld();

	if (PreviewWorld != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		PreviewWorld->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}

	UDynamicMaterialEditorSettings* EditorSettings = UDynamicMaterialEditorSettings::Get();

	if (EditorSettings)
	{
		EditorSettings->GetOnSettingsChanged().AddSP(this, &SDMMaterialPreview::OnEditorSettingsChanged);
		SetPreviewType(EditorSettings->PreviewMesh);
	}

	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(PreviewScene, EditorViewportClient);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SDMMaterialPreview::OnPropertyChanged);

	if (UEditorEngine* Editor = Cast<UEditorEngine>(GEngine))
	{
		Editor->OnPreviewShaderPlatformChanged().AddSP(this, &SDMMaterialPreview::OnFeatureLevelChanged);
	}

	if (UDynamicMaterialInstance* MaterialInstance = InMaterialModelBase->GetDynamicMaterialInstance())
	{
		SetPreviewMaterial(MaterialInstance);
	}

	if (EditorSettings)
	{
		SetShowPreviewBackground(EditorSettings->bShowPreviewBackground);
	}
}

void SDMMaterialPreview::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		[
			MakeViewportToolbarOverlay()
		];
}

TSharedRef<SWidget> SDMMaterialPreview::MakeViewportToolbarOverlay()
{
	if (!bShowMenu)
	{
		return SNullWidget::NullWidget;
	}

	FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None, {});

	const FName ToolBarStyle = TEXT("EditorViewportToolBar");
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SDMMaterialPreview::GenerateToolbarMenu),
		TAttribute<FText>(),
		LOCTEXT("ToolbarToolTip", "Material preview options"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.OptionsDropdown"),
		/* Simple combo box */ true
	);

	return ToolbarBuilder.MakeWidget();
}

void SDMMaterialPreview::RefreshViewport()
{
	// Reregister the preview components, so if the preview material changed it will be propagated to the render thread
	if (PreviewMeshComponent != nullptr)
	{
		PreviewMeshComponent->MarkRenderStateDirty();
	}

	SceneViewport->InvalidateDisplay();

	if (EditorViewportClient.IsValid() && PreviewScene.IsValid())
	{
		UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
		const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();

		if (Settings->Profiles.IsValidIndex(ProfileIndex))
		{
			PreviewScene->UpdateScene(Settings->Profiles[ProfileIndex]);

			if (Settings->Profiles[ProfileIndex].bRotateLightingRig && !EditorViewportClient->IsRealtime())
			{
				EditorViewportClient->SetRealtime(true);
			}
		}
	}
}

void SDMMaterialPreview::SetPreviewType(EDMMaterialPreviewMesh InPrimitiveType)
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	Settings->PreviewMesh = InPrimitiveType;
	Settings->SaveConfig();

	if (!SceneViewport.IsValid())
	{
		return;
	}

	UStaticMesh* Primitive = nullptr;

	switch (InPrimitiveType)
	{
		case EDMMaterialPreviewMesh::Plane: Primitive = GUnrealEd->GetThumbnailManager()->EditorPlane; break;
		case EDMMaterialPreviewMesh::Cube: Primitive = GUnrealEd->GetThumbnailManager()->EditorCube; break;
		case EDMMaterialPreviewMesh::Sphere: Primitive = GUnrealEd->GetThumbnailManager()->EditorSphere; break;
		case EDMMaterialPreviewMesh::Cylinder: Primitive = GUnrealEd->GetThumbnailManager()->EditorCylinder; break;
		case EDMMaterialPreviewMesh::Custom: Primitive = Settings->CustomPreviewMesh.LoadSynchronous(); break;
		default: return;
	}

	SetPreviewAsset(Primitive);

	RefreshViewport();
}

ECheckBoxState SDMMaterialPreview::IsPreviewTypeSet(EDMMaterialPreviewMesh InPrimitiveType) const
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		return Settings->PreviewMesh == InPrimitiveType
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}

void SDMMaterialPreview::SetPreviewAsset(UObject* InAsset)
{
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(InAsset);

	if (!StaticMesh)
	{
		return;
	}

	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	// Unregister the current component
	if (PreviewMeshComponent != nullptr)
	{
		PreviewScene->RemoveComponent(PreviewMeshComponent);
		PreviewMeshComponent = nullptr;
	}

	FTransform Transform = FTransform::Identity;

	// Special case handling for static meshes, to use more accurate bounds via a subclass
	UStaticMeshComponent* NewSMComponent = NewObject<UMaterialEditorMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	NewSMComponent->SetStaticMesh(StaticMesh);

	PreviewMeshComponent = NewSMComponent;

	using namespace UE::DynamicMaterialEditor::Private;

	// Update the rotation of the plane mesh so that it is front facing to the viewport camera's default forward view.
	if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorPlane)
	{
		const FRotator PlaneRotation(0.0f, 180.0f, 0.0f);
		Transform.SetRotation(FQuat(PlaneRotation));
	}
	else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorCube)
	{
		const FRotator PlaneRotation(0.0f, -90.0f, 0.0f);
		Transform.SetRotation(FQuat(PlaneRotation));
		Transform.SetScale3D(FVector(0.75));
	}
	else if (StaticMesh == Settings->CustomPreviewMesh)
	{
		const FRotator PlaneRotation(0.0f, -90.0f, 0.0f);
		Transform.SetRotation(FQuat(PlaneRotation));
		Transform.SetLocation(FVector(0, 0, -StaticMesh->GetBounds().BoxExtent.Z));
	}

	// Add the new component to the scene
	if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
	{
		PreviewMeshComponent->SetMobility(EComponentMobility::Static);
	}

	PreviewScene->AddComponent(PreviewMeshComponent, Transform);

	PreviewScene->SetFloorOffset(-PreviewMeshComponent->Bounds.Origin.Z + PreviewMeshComponent->Bounds.BoxExtent.Z);

	// Make sure the preview material is applied to the component
	SetPreviewMaterial(PreviewMaterial);
}

void SDMMaterialPreview::SetPreviewMaterial(UMaterialInterface* InMaterialInterface)
{
	PreviewMaterial = InMaterialInterface;

	if (!PreviewMaterial)
	{
		return;
	}

	if (PreviewMaterial->GetMaterial()->IsPostProcessMaterial())
	{
		ApplyPreviewMaterial_PostProcess();
	}
	else
	{
		ApplyPreviewMaterial_Default();
	}
}

void SDMMaterialPreview::ApplyPreviewMaterial_Default()
{
	if (PostProcessVolumeActor)
	{
		PostProcessVolumeActor->Destroy();
		PostProcessVolumeActor = nullptr;
	}

	GetViewportClient()->EngineShowFlags.SetPostProcessing(false);
	GetViewportClient()->EngineShowFlags.SetPostProcessMaterial(false);

	if (PreviewMeshComponent == nullptr)
	{
		return;
	}

	PreviewMeshComponent->OverrideMaterials.Empty(3);

	if (PreviewMaterial)
	{
		PreviewMeshComponent->OverrideMaterials.Add(PreviewMaterial);
		PreviewMeshComponent->OverrideMaterials.Add(PreviewMaterial);
		PreviewMeshComponent->OverrideMaterials.Add(PreviewMaterial);
	}

	PreviewMeshComponent->MarkRenderStateDirty();
}

void SDMMaterialPreview::ApplyPreviewMaterial_PostProcess()
{
	if (PreviewMeshComponent == nullptr)
	{
		return;
	}

	if (PostProcessVolumeActor == nullptr)
	{
		PostProcessVolumeActor = GetWorld()->SpawnActor<APostProcessVolume>(APostProcessVolume::StaticClass(), FTransform::Identity);

		GetViewportClient()->EngineShowFlags.SetPostProcessing(true);
		GetViewportClient()->EngineShowFlags.SetPostProcessMaterial(true);
	}

	// Clear blendables, and re-add them (cleans up any post process materials with UserSceneTextures that are no longer used or loaded)
	PostProcessVolumeActor->Settings.WeightedBlendables.Array.Empty(3);

	check(PreviewMaterial != nullptr);
	PostProcessVolumeActor->AddOrUpdateBlendable(PreviewMaterial);
	PostProcessVolumeActor->AddOrUpdateBlendable(PreviewMaterial);
	PostProcessVolumeActor->AddOrUpdateBlendable(PreviewMaterial);
	PostProcessVolumeActor->bEnabled = true;
	PostProcessVolumeActor->BlendWeight = 1.0f;
	PostProcessVolumeActor->bUnbound = true;

	// Setting this forces this post process material to write to SceneColor instead of any UserSceneTexture it may have assigned, for preview purposes
	PostProcessVolumeActor->Settings.PreviewBlendable = PreviewMaterial;

	// Remove preview material from the preview mesh.
	if (PreviewMeshComponent != nullptr)
	{
		PreviewMeshComponent->OverrideMaterials.Empty();
		PreviewMeshComponent->MarkRenderStateDirty();
	}

	GetViewportClient()->RedrawRequested(GetSceneViewport().Get());
}

void SDMMaterialPreview::SetShowPreviewBackground(bool bInShowBackground)
{
	UDynamicMaterialEditorSettings* EditorSettings = UDynamicMaterialEditorSettings::Get();

	if (!EditorSettings)
	{
		return;
	}

	EditorSettings->bShowPreviewBackground = bInShowBackground;
	EditorSettings->SaveConfig();

	UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
	const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();

	if (Settings->Profiles.IsValidIndex(ProfileIndex))
	{
		PreviewScene->SetEnvironmentVisibility(bInShowBackground);
	}

	RefreshViewport();
}

void SDMMaterialPreview::TogglePreviewBackground()
{
	SetShowPreviewBackground(IsPreviewBackgroundEnabled() != ECheckBoxState::Checked);
}

ECheckBoxState SDMMaterialPreview::IsPreviewBackgroundEnabled() const
{
	if (UAssetViewerSettings* Settings = UAssetViewerSettings::Get())
	{
		const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();

		if (Settings->Profiles.IsValidIndex(ProfileIndex))
		{
			return Settings->Profiles[ProfileIndex].bShowEnvironment
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Unchecked;
}

void SDMMaterialPreview::OnPropertyChanged(UObject* InObjectBeingModified, FPropertyChangedEvent& InPropertyChangedEvent)
{
}

void SDMMaterialPreview::OnFeatureLevelChanged(EShaderPlatform NewShaderPlatform)
{
	if (UWorld* World = PreviewScene->GetWorld())
	{
		World->ShaderPlatformChanged(NewShaderPlatform);
	}

}

TSharedRef<SWidget> SDMMaterialPreview::GenerateToolbarMenu()
{
	const FName MenuName = TEXT("DMMaterialPreview");

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		const FMaterialEditorCommands& MaterialEditorCommands = FMaterialEditorCommands::Get();
		const FDynamicMaterialEditorCommands& DMEditorCommands = FDynamicMaterialEditorCommands::Get();

		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);

		FToolMenuSection& MeshSection = Menu->AddSection(TEXT("PreviewMesh"), LOCTEXT("PreviewMesh", "Preview Mesh"));
		MeshSection.AddEntry(FToolMenuEntry::InitMenuEntry(MaterialEditorCommands.SetPlanePreview));
		MeshSection.AddEntry(FToolMenuEntry::InitMenuEntry(MaterialEditorCommands.SetCubePreview));
		MeshSection.AddEntry(FToolMenuEntry::InitMenuEntry(MaterialEditorCommands.SetSpherePreview));
		MeshSection.AddEntry(FToolMenuEntry::InitMenuEntry(MaterialEditorCommands.SetCylinderPreview));

		MeshSection.AddEntry(FToolMenuEntry::InitMenuEntry(
			DMEditorCommands.SetCustomPreviewMesh, 
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIconFinder::FindIcon(TEXT("GraphEditor.SpawnActor_16x"))
		));

		FToolMenuSection& SettingsSection = Menu->AddSection(TEXT("Settings"), LOCTEXT("Settings", "Settings"));
		SettingsSection.AddEntry(FToolMenuEntry::InitMenuEntry(MaterialEditorCommands.TogglePreviewBackground));

		Menu->AddDynamicSection(
			TEXT("Actions"),
			FNewSectionConstructChoice(FNewToolMenuDelegate::CreateStatic(&SDMMaterialPreview::AddActionMenu))		
		);
	}

	UDMMaterialPreviewContext* PreviewContext = NewObject<UDMMaterialPreviewContext>();
	PreviewContext->SetPreviewWidget(SharedThis(this));

	FToolMenuContext Context;
	Context.AppendCommandList(CommandList);
	Context.AddObject(PreviewContext);

	return ToolMenus->GenerateWidget(MenuName, Context);
}

void SDMMaterialPreview::AddActionMenu(UToolMenu* InMenu)
{
	const UDMMaterialPreviewContext* Context = InMenu->FindContext<UDMMaterialPreviewContext>();

	if (!Context)
	{
		return;
	}

	TSharedPtr<SDMMaterialPreview> PreviewWidget = Context->GetPreviewWidget();

	if (!PreviewWidget.IsValid() || PreviewWidget->bIsPopout)
	{
		return;
	}

	FToolMenuSection& ActionsSection = InMenu->AddSection(TEXT("Actions"), LOCTEXT("Actions", "Actions"));

	FUIAction OpenPreviewTabAction;
	OpenPreviewTabAction.ExecuteAction.BindSP(PreviewWidget.Get(), &SDMMaterialPreview::OpenMaterialPreviewTab);

	ActionsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		TEXT("PopoutMaterialPreviewTab"),
		LOCTEXT("OpenPreview", "Open Preview"),
		LOCTEXT("OpenPreviewToolTip", "Open a tab with a preview of the material."),
		TAttribute<FSlateIcon>(),
		OpenPreviewTabAction
	));
}

void SDMMaterialPreview::OnEditorSettingsChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, PreviewMesh))
	{
		SetPreviewType(Settings->PreviewMesh);
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, bShowPreviewBackground))
	{
		SetShowPreviewBackground(Settings->bShowPreviewBackground);
	}
}

void SDMMaterialPreview::OpenMaterialPreviewTab()
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	EditorWidget->OpenMaterialPreviewTab();
}

TSharedRef<FEditorViewportClient> SDMMaterialPreview::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShared<FDMMaterialPreviewViewportClient>(SharedThis(this), *PreviewScene.Get(), MakeShared<FEditorModeTools>());
	EditorViewportClient->SetViewLocation(FVector::ZeroVector);
	EditorViewportClient->SetViewRotation(FRotator(-15.0f, -90.0f, 0.0f));
	EditorViewportClient->SetViewLocationForOrbiting(FVector::ZeroVector);
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->EngineShowFlags.EnableAdvancedFeatures();
	EditorViewportClient->EngineShowFlags.SetLighting(true);
	EditorViewportClient->EngineShowFlags.SetIndirectLightingCache(true);
	EditorViewportClient->EngineShowFlags.SetPostProcessing(false);
	EditorViewportClient->Invalidate();
	EditorViewportClient->VisibilityDelegate.BindSP(this, &SDMMaterialPreview::IsVisible);

	return EditorViewportClient.ToSharedRef();
}

EVisibility SDMMaterialPreview::OnGetViewportContentVisibility() const
{
	return SEditorViewport::OnGetViewportContentVisibility();
}

void SDMMaterialPreview::BindCommands()
{
	SEditorViewport::BindCommands();

	const FMaterialEditorCommands& MaterialEditorCommands = FMaterialEditorCommands::Get();
	const FDynamicMaterialEditorCommands& DMEditorCommands = FDynamicMaterialEditorCommands::Get();

	check(CommandList.IsValid());

	// Add the commands to the toolkit command list so that the toolbar buttons can find them
	CommandList->MapAction(
		MaterialEditorCommands.SetPlanePreview,
		FExecuteAction::CreateSP(this, &SDMMaterialPreview::SetPreviewType, EDMMaterialPreviewMesh::Plane),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SDMMaterialPreview::IsPreviewTypeSet, EDMMaterialPreviewMesh::Plane));

	CommandList->MapAction(
		MaterialEditorCommands.SetCubePreview,
		FExecuteAction::CreateSP(this, &SDMMaterialPreview::SetPreviewType, EDMMaterialPreviewMesh::Cube),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SDMMaterialPreview::IsPreviewTypeSet, EDMMaterialPreviewMesh::Cube));

	CommandList->MapAction(
		MaterialEditorCommands.SetSpherePreview,
		FExecuteAction::CreateSP(this, &SDMMaterialPreview::SetPreviewType, EDMMaterialPreviewMesh::Sphere),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SDMMaterialPreview::IsPreviewTypeSet, EDMMaterialPreviewMesh::Sphere));

	CommandList->MapAction(
		MaterialEditorCommands.SetCylinderPreview,
		FExecuteAction::CreateSP(this, &SDMMaterialPreview::SetPreviewType, EDMMaterialPreviewMesh::Cylinder),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SDMMaterialPreview::IsPreviewTypeSet, EDMMaterialPreviewMesh::Cylinder));

	CommandList->MapAction(
		DMEditorCommands.SetCustomPreviewMesh,
		FExecuteAction::CreateSP(this, &SDMMaterialPreview::SetPreviewType, EDMMaterialPreviewMesh::Custom),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SDMMaterialPreview::IsPreviewTypeSet, EDMMaterialPreviewMesh::Custom));

	CommandList->MapAction(
		MaterialEditorCommands.TogglePreviewBackground,
		FExecuteAction::CreateSP(this, &SDMMaterialPreview::TogglePreviewBackground),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SDMMaterialPreview::IsPreviewBackgroundEnabled));
}

void SDMMaterialPreview::OnFocusViewportToSelection()
{
	if (PreviewMeshComponent != nullptr)
	{
		EditorViewportClient->FocusViewportOnBounds(PreviewMeshComponent->Bounds);
	}
}

void SDMMaterialPreview::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(PreviewMeshComponent);
	InCollector.AddReferencedObject(PreviewMaterial);
	InCollector.AddReferencedObject(PostProcessVolumeActor);
}

FString SDMMaterialPreview::GetReferencerName() const
{
	return TEXT("SDMMaterialPreview");
}

#undef LOCTEXT_NAMESPACE
