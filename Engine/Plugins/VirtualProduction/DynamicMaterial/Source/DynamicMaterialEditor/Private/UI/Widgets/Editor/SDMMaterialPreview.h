// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"
#include "RHIShaderPlatform.h"
#include "UObject/GCObject.h"

#include "Delegates/IDelegateInstance.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "SDMMaterialPreview.generated.h"

class APostProcessVolume;
class FAdvancedPreviewScene;
class FDMMaterialPreviewViewportClient;
class FEditorViewportClient;
class SDMMaterialEditor;
class SDMMaterialPreview;
class UDynamicMaterialModelBase;
class UMaterialInterface;
class UMeshComponent;
class UToolMenu;
enum class EDMMaterialPreviewMesh : uint8;
struct FPropertyChangedEvent;

UCLASS(MinimalAPI)
class UDMMaterialPreviewContext : public UObject
{
	GENERATED_BODY()

public:
	void SetPreviewWidget(const TSharedRef<SDMMaterialPreview>& InPreviewWidget)
	{
		PreviewWidgetWeak = InPreviewWidget;
	}

	TSharedPtr<SDMMaterialPreview> GetPreviewWidget() const
	{
		return PreviewWidgetWeak.Pin();
	}

private:
	TWeakPtr<SDMMaterialPreview> PreviewWidgetWeak;
};


/** Based on SMaterialEditor3DPreviewViewport (private) */
class SDMMaterialPreview : public SEditorViewport, public FGCObject
{
	SLATE_DECLARE_WIDGET(SDMMaterialPreview, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialPreview)
		: _ShowMenu(true)
		, _IsPopout(false)
		{}
		SLATE_ARGUMENT(bool, ShowMenu)
		SLATE_ARGUMENT(bool, IsPopout)
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialPreview() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, 
		UDynamicMaterialModelBase* InMaterialModelBase);

	//~ Begin SEditorViewport
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	//~ End SEditorViewport

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	bool bShowMenu;
	bool bIsPopout;

	TSharedPtr<FDMMaterialPreviewViewportClient> EditorViewportClient;
	TSharedPtr<FAdvancedPreviewScene> PreviewScene;

	TObjectPtr<UMeshComponent> PreviewMeshComponent;
	TObjectPtr<UMaterialInterface> PreviewMaterial;
	TObjectPtr<APostProcessVolume> PostProcessVolumeActor;

	TSharedRef<SWidget> MakeViewportToolbarOverlay();
	void RefreshViewport();

	void SetPreviewType(EDMMaterialPreviewMesh InPrimitiveType);

	ECheckBoxState IsPreviewTypeSet(EDMMaterialPreviewMesh InPrimitiveType) const;

	void SetPreviewAsset(UObject* InAsset);

	void SetPreviewMaterial(UMaterialInterface* InMaterialInterface);

	void ApplyPreviewMaterial_Default();

	/** Spawn post processing volume actor if the material has post processing as domain. */
	void ApplyPreviewMaterial_PostProcess();

	void SetShowPreviewBackground(bool bInShowBackground);

	void TogglePreviewBackground();

	ECheckBoxState IsPreviewBackgroundEnabled() const;

	void OnPropertyChanged(UObject* InObjectBeingModified, FPropertyChangedEvent& InPropertyChangedEvent);

	void OnFeatureLevelChanged(EShaderPlatform NewShaderPlatform);

	TSharedRef<SWidget> GenerateToolbarMenu();

	static void AddActionMenu(UToolMenu* InMenu);

	void OnEditorSettingsChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	void OpenMaterialPreviewTab();

	//~ Begin SEditorViewport
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	//~ End SEditorViewport
};
