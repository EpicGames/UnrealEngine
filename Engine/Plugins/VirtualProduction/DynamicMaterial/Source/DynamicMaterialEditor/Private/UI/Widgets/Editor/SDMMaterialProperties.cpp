// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMMaterialProperties.h"

#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"
#include "DetailLayoutBuilder.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialModule.h"
#include "Engine/World.h"
#include "ICustomDetailsView.h"
#include "Items/ICustomDetailsViewCustomCategoryItem.h"
#include "Items/ICustomDetailsViewCustomItem.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Misc/MessageDialog.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "SAssetDropTarget.h"
#include "Styling/StyleColors.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/Visualizers/SDMMaterialComponentPreview.h"
#include "Utils/DMMaterialSlotFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialProperties"

void SDMMaterialProperties::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialProperties::~SDMMaterialProperties()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		Settings->GetOnSettingsChanged().RemoveAll(this);
	}
}

void SDMMaterialProperties::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget)
{
	EditorWidgetWeak = InEditorWidget;

	Content = TDMWidgetSlot<SWidget>(SharedThis(this), 0, CreateSlot_Content());

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		Settings->GetOnSettingsChanged().AddSP(this, &SDMMaterialProperties::OnSettingsUpdated);
	}
}

void SDMMaterialProperties::Validate()
{
	if (Content.HasBeenInvalidated())
	{
		GlobalItems.Empty();
		PropertyPreviewContainers.Empty();
		PropertyEmptyContainers.Empty();
		PropertyPreviews.Empty();
		SliderItems.Empty();
		Content << CreateSlot_Content();
	}
}

void SDMMaterialProperties::NotifyPreChange(FProperty* InPropertyAboutToChange)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (!MaterialModelBase)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	// Impossible to know which property changed, so notify all of them.
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[EditorOnlyData, InPropertyAboutToChange](EDMMaterialPropertyType InMaterialProperty)
		{
			if (UE::DynamicMaterialEditor::Private::IsCustomMaterialProperty(InMaterialProperty))
			{
				return EDMIterationResult::Continue;
			}

			if (UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty))
			{
				UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForEnabledMaterialProperty(InMaterialProperty);

				const bool bIsActive = Slot && MaterialProperty->IsEnabled() && MaterialProperty->IsValidForModel(*EditorOnlyData);

				if (bIsActive)
				{
					if (UDMMaterialValue* Value = Cast<UDMMaterialValue>(MaterialProperty->GetComponent(UDynamicMaterialModelEditorOnlyData::AlphaValueName)))
					{
						Value->NotifyPreChange(InPropertyAboutToChange);
					}
				}
			}

			return EDMIterationResult::Continue;
		}
	);
}

void SDMMaterialProperties::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (!MaterialModelBase)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	// All of them were notified of the pre-change, so notify them of the post change.
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[EditorOnlyData, &InPropertyChangedEvent, InPropertyThatChanged](EDMMaterialPropertyType InMaterialProperty)
		{
			if (UE::DynamicMaterialEditor::Private::IsCustomMaterialProperty(InMaterialProperty))
			{
				return EDMIterationResult::Continue;
			}

			if (UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty))
			{
				UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForEnabledMaterialProperty(InMaterialProperty);

				const bool bIsActive = Slot && MaterialProperty->IsEnabled() && MaterialProperty->IsValidForModel(*EditorOnlyData);

				if (bIsActive)
				{
					if (UDMMaterialValue* Value = Cast<UDMMaterialValue>(MaterialProperty->GetComponent(UDynamicMaterialModelEditorOnlyData::AlphaValueName)))
					{
						Value->NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
					}
				}
			}

			return EDMIterationResult::Continue;
		}
	);
}

void SDMMaterialProperties::OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->ShouldAutomaticallyCopyParametersToSourceMaterial())
		{
			if (InComponent)
			{
				if (UDMMaterialComponent* OriginalComponent = GetOriginalComponent(InComponent))
				{
					IDMParameterContainer::CopyParametersBetween(InComponent, OriginalComponent);
					return;
				}
			}
		}
	}

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		if (UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetOriginalMaterialModelBase())
		{
			MaterialModelBase->MarkPreviewModified();
		}
	}
}

UDMMaterialComponent* SDMMaterialProperties::GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const
{
	if (!InPreviewComponent)
	{
		return nullptr;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	return EditorWidget->GetOriginalComponent(InPreviewComponent);
}

TSharedRef<SWidget> SDMMaterialProperties::CreateSlot_Content()
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (!MaterialModelBase)
	{
		return SNullWidget::NullWidget;
	}

	UDynamicMaterialModel* MaterialModel = MaterialModelBase->ResolveMaterialModel();

	if (!MaterialModel)
	{
		return SNullWidget::NullWidget;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return SNullWidget::NullWidget;
	}

	TGuardValue<bool> ConstructGuard(bConstructing, true);

	FCustomDetailsViewArgs Args;
	Args.bAllowGlobalExtensions = false;
	Args.bAllowResetToDefault = false;
	Args.bShowCategories = true;
	Args.OnExpansionStateChanged.AddSP(this, &SDMMaterialProperties::OnExpansionStateChanged);

	TSharedRef<ICustomDetailsView> DetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(Args);
	TSharedRef<ICustomDetailsViewItem> RootItem = DetailsView->GetRootItem();

	auto CreateCategory = [&DetailsView, &RootItem, MaterialModelBase](FName InName, const FText& InDisplayName)
		{
			TSharedRef<ICustomDetailsViewItem> CategoryItem = DetailsView->CreateCustomCategoryItem(RootItem, InName, InDisplayName)->AsItem();
			DetailsView->ExtendTree(RootItem->GetItemId(), ECustomDetailsTreeInsertPosition::Child, CategoryItem);

			bool bExpansionState = true;
			FDMWidgetLibrary::Get().GetExpansionState(MaterialModelBase, InName, bExpansionState);

			DetailsView->SetItemExpansionState(
				CategoryItem->GetItemId(),
				bExpansionState ? ECustomDetailsViewExpansion::SelfExpanded : ECustomDetailsViewExpansion::Collapsed
			);

			return CategoryItem;
		};

	TSharedRef<ICustomDetailsViewItem> ActiveCategory = CreateCategory(TEXT("Active"), LOCTEXT("ActiveCategory", "Active Channels"));
	TSharedRef<ICustomDetailsViewItem> InactiveCategory = CreateCategory(TEXT("Inactive"), LOCTEXT("InactiveCategory", "Inactive Channels"));
	TSharedRef<ICustomDetailsViewItem> IncompatibleCategory = CreateCategory(TEXT("Incompatible"), LOCTEXT("IncompatibleCategory", "Incompatible Channels"));

	TSharedRef<SVerticalBox> List = SNew(SVerticalBox);

	// Active properties first.
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &DetailsView, &ActiveCategory, EditorOnlyData](EDMMaterialPropertyType InMaterialProperty)
		{
			if (UE::DynamicMaterialEditor::Private::IsCustomMaterialProperty(InMaterialProperty))
			{
				return EDMIterationResult::Continue;
			}

			if (UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty))
			{
				UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForEnabledMaterialProperty(InMaterialProperty);

				const bool bIsActive = Slot && MaterialProperty->IsEnabled() && MaterialProperty->IsValidForModel(*EditorOnlyData);

				if (bIsActive)
				{
					AddProperty(DetailsView, ActiveCategory, MaterialProperty);
				}
			}

			return EDMIterationResult::Continue;
		}
	);

	// Now inactive properties
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &DetailsView, &InactiveCategory, EditorOnlyData](EDMMaterialPropertyType InMaterialProperty)
		{
			if (UE::DynamicMaterialEditor::Private::IsCustomMaterialProperty(InMaterialProperty))
			{
				return EDMIterationResult::Continue;
			}

			if (UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty))
			{
				UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForEnabledMaterialProperty(InMaterialProperty);

				const bool bIsActive = Slot && MaterialProperty->IsEnabled();
				const bool bIsValid = MaterialProperty->IsValidForModel(*EditorOnlyData);

				if (!bIsActive && bIsValid)
				{
					AddProperty(DetailsView, InactiveCategory, MaterialProperty);
				}
			}

			return EDMIterationResult::Continue;
		}
	);

	// Now invalid properties
	UE::DynamicMaterial::ForEachMaterialPropertyType(
		[this, &DetailsView, &IncompatibleCategory, EditorOnlyData](EDMMaterialPropertyType InMaterialProperty)
		{
			if (UE::DynamicMaterialEditor::Private::IsCustomMaterialProperty(InMaterialProperty))
			{
				return EDMIterationResult::Continue;
			}

			if (UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty))
			{
				UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForEnabledMaterialProperty(InMaterialProperty);

				const bool bIsActive = Slot && MaterialProperty->IsEnabled();
				const bool bIsValid = MaterialProperty->IsValidForModel(*EditorOnlyData);

				if (!bIsActive && !bIsValid)
				{
					AddProperty(DetailsView, IncompatibleCategory, MaterialProperty);
				}
			}

			return EDMIterationResult::Continue;
		}
	);

	DetailsView->RebuildTree(ECustomDetailsViewBuildType::InstantBuild);

	return DetailsView;
}

void SDMMaterialProperties::AddProperty(const TSharedRef<ICustomDetailsView>& InDetailsView, const TSharedRef<ICustomDetailsViewItem>& InCategory,
	UDMMaterialProperty* InProperty)
{
	if (!InProperty)
	{
		return;
	}

	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	TSharedPtr<ICustomDetailsViewCustomItem> Item = InDetailsView->CreateCustomItem(InCategory, InProperty->GetClass()->GetFName());

	if (!Item.IsValid())
	{
		return;
	}

	Item->SetWholeRowWidget(CreatePropertyRow(InProperty));

	InDetailsView->ExtendTree(InCategory->GetItemId(), ECustomDetailsTreeInsertPosition::Child, Item->AsItem());
}

TSharedRef<SWidget> SDMMaterialProperties::CreatePropertyRow(UDMMaterialProperty* InProperty)
{
	// There are all ensured to be valid by the caller of this method
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();
	UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModelBase);
	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();
	const EDMMaterialPropertyType MaterialProperty = InProperty->GetMaterialProperty();

	TSharedRef<SBox> PreviewWidgetContainer = SNew(SBox)
		.WidthOverride(Settings->PropertyPreviewSize + 4.f)
		.HeightOverride(18.f);

	TSharedRef<SWidget> PropertyName = CreateSlot_PropertyName(MaterialProperty);

	TSharedRef<SWidget> Slider = SNullWidget::NullWidget;

	const bool bValidProperty = InProperty->IsValidForModel(*EditorOnlyData);
	const bool bPropertyEnabled = bValidProperty && InProperty->IsEnabled() && EditorOnlyData->GetSlotForMaterialProperty(MaterialProperty);;

	if (bPropertyEnabled)
	{
		TSharedPtr<SDMMaterialComponentPreview> PreviewImage;

		PreviewWidgetContainer = SNew(SBox)
			.WidthOverride(Settings->PropertyPreviewSize + 4.f)
			.HeightOverride(Settings->PropertyPreviewSize + 4.f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(FLinearColor(1, 1, 1, 1.f))
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush(UE::DynamicMaterialEditor::Private::EditorDarkBackground))
				[
					SAssignNew(PreviewImage, SDMMaterialComponentPreview, EditorWidget.ToSharedRef(), InProperty)
					.PreviewSize(FVector2D(Settings->PropertyPreviewSize))
				]
			];

		PropertyPreviewContainers.Add(PreviewWidgetContainer);
		PropertyPreviews.Add(PreviewImage.ToSharedRef());

		PreviewWidgetContainer->SetCursor(EMouseCursor::Hand);
		PreviewWidgetContainer->SetOnMouseButtonUp(FPointerEventHandler::CreateSP(this, &SDMMaterialProperties::OnPropertyClicked, MaterialProperty));

		PropertyName->SetCursor(EMouseCursor::Hand);
		PropertyName->SetOnMouseButtonUp(FPointerEventHandler::CreateSP(this, &SDMMaterialProperties::OnPropertyClicked, MaterialProperty));

		Slider = CreateGlobalSlider(InProperty);
	}
	else
	{
		PropertyEmptyContainers.Add(PreviewWidgetContainer);
	}

	// If the property is just disabled, leave the slider widget blank.

	TSharedRef<SHorizontalBox> PropertyWidget = SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(0.f, 5.f, 0.f, 5.f)
		[
			PreviewWidgetContainer
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.Padding(5.f, 5.f, 0.f, 5.f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					CreateSlot_EnabledButton(MaterialProperty)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5.f, 0.f, 0.f, 0.f)
				[
					PropertyName
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				Slider
			]
		];

	if (!bPropertyEnabled || !MaterialModelBase->IsA<UDynamicMaterialModel>())
	{
		return PropertyWidget;
	}

	return SNew(SAssetDropTarget)
		.OnAreAssetsAcceptableForDrop(this, &SDMMaterialProperties::OnAssetDraggedOver, MaterialProperty)
		.OnAssetsDropped(this, &SDMMaterialProperties::OnAssetsDropped, MaterialProperty)
		[
			PropertyWidget
		];
}

TSharedRef<SWidget> SDMMaterialProperties::CreateSlot_EnabledButton(EDMMaterialPropertyType InMaterialProperty)
{
	const FDMMaterialEditorPage Page = {EDMMaterialEditorMode::EditSlot, InMaterialProperty};
	const FText Format = LOCTEXT("PropertyEnableFormat", "Toggle the {0} property.\n\nProperty must be valid for the Material Type.");
	const FText ToolTip = FText::Format(Format, SDMMaterialPropertySelector::GetSelectButtonText(Page, /* Short Name */ false));

	return SNew(SCheckBox)
		.IsEnabled(this, &SDMMaterialProperties::GetPropertyEnabledEnabled, InMaterialProperty)
		.IsChecked(this, &SDMMaterialProperties::GetPropertyEnabledState, InMaterialProperty)
		.OnCheckStateChanged(this, &SDMMaterialProperties::OnPropertyEnabledStateChanged, InMaterialProperty)
		.ToolTipText(ToolTip);
}

TSharedRef<SWidget> SDMMaterialProperties::CreateSlot_PropertyName(EDMMaterialPropertyType InMaterialProperty)
{
	const FDMMaterialEditorPage Page = {EDMMaterialEditorMode::EditSlot, InMaterialProperty};

	return SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(SDMMaterialPropertySelector::GetSelectButtonText(Page, /* Short Name */ false))
		.ToolTipText(SDMMaterialPropertySelector::GetSelectButtonText(Page, /* Short Name */ false));
}

bool SDMMaterialProperties::GetPropertyEnabledEnabled(EDMMaterialPropertyType InMaterialProperty) const
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return false;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return false;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return false;
	}

	UDMMaterialProperty* Property = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!Property)
	{
		return false;
	}

	return Property->IsValidForModel(*EditorOnlyData);
}

ECheckBoxState SDMMaterialProperties::GetPropertyEnabledState(EDMMaterialPropertyType InMaterialProperty) const
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return ECheckBoxState::Unchecked;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return ECheckBoxState::Unchecked;
	}

	UDMMaterialProperty* Property = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!Property)
	{
		return ECheckBoxState::Unchecked;
	}

	return (Property->IsEnabled() && EditorOnlyData->GetSlotForMaterialProperty(InMaterialProperty))
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SDMMaterialProperties::OnPropertyEnabledStateChanged(ECheckBoxState InState, EDMMaterialPropertyType InMaterialProperty)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!MaterialProperty)
	{
		return;
	}

	const bool bEnabled = InState == ECheckBoxState::Checked;

	MaterialProperty->SetEnabled(bEnabled);

	if (bEnabled)
	{
		UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForMaterialProperty(InMaterialProperty);

		if (!Slot)
		{
			EditorOnlyData->AddSlotForMaterialProperty(InMaterialProperty);
		}
	}

	Content.Invalidate();

	// Make sure we go back to the property previews
	EditorWidget->EditProperties();
}

FReply SDMMaterialProperties::OnPropertyClicked(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent,
	EDMMaterialPropertyType InMaterialProperty)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return FReply::Handled();
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return FReply::Handled();
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return FReply::Handled();
	}

	UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForMaterialProperty(InMaterialProperty);

	if (!Slot)
	{
		return FReply::Handled();
	}

	EditorWidget->SelectProperty(InMaterialProperty);

	return FReply::Handled();
}

TSharedRef<SWidget> SDMMaterialProperties::CreateGlobalSlider(UDMMaterialProperty* InProperty)
{
	if (!InProperty)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (!PreviewMaterialModelBase)
	{
		return SNullWidget::NullWidget;
	}

	UObject* AlphaObject = InProperty->GetComponent(UDynamicMaterialModelEditorOnlyData::AlphaValueName);

	if (!AlphaObject)
	{
		return SNullWidget::NullWidget;
	}

	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(PreviewMaterialModelBase))
	{
		AlphaObject = MaterialModelDynamic->GetComponentDynamic(AlphaObject->GetFName());

		if (!AlphaObject)
		{
			return SNullWidget::NullWidget;
		}
	}

	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = nullptr;

	if (UDynamicMaterialModelBase* OriginalMaterialModelBase = EditorWidget->GetOriginalMaterialModelBase())
	{
		if (UWorld* World = OriginalMaterialModelBase->GetWorld())
		{
			if (const UDMWorldSubsystem* const WorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>())
			{
				KeyframeHandler = WorldSubsystem->GetKeyframeHandler();
			}
		}
	}

	FCustomDetailsViewArgs Args;
	Args.KeyframeHandler = KeyframeHandler;
	Args.bAllowGlobalExtensions = true;
	Args.bAllowResetToDefault = true;
	Args.bShowCategories = false;

	TSharedRef<ICustomDetailsView> DetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(Args);
	FCustomDetailsViewItemId RootId = DetailsView->GetRootItem()->GetItemId();

	FDMPropertyHandleGenerateParams Params;
	Params.Widget = this;
	Params.NotifyHook = this;
	Params.Object = AlphaObject;
	Params.PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();
	Params.OriginalMaterialModelBase = EditorWidget->GetOriginalMaterialModelBase();
	Params.PropertyName = UDMMaterialValue::ValueName;

	FDMPropertyHandle PropertyHandle = FDMWidgetLibrary::Get().GetPropertyHandle(Params);

	TSharedRef<ICustomDetailsViewItem> AlphaItem = DetailsView->CreateDetailTreeItem(DetailsView->GetRootItem(), PropertyHandle.PreviewHandle.DetailTreeNode.ToSharedRef());

	if (UDMMaterialValue* AlphaValue = Cast<UDMMaterialValue>(AlphaObject))
	{
		AlphaItem->SetResetToDefaultOverride(FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(AlphaValue, &UDMMaterialValue::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(AlphaValue, &UDMMaterialValue::ResetToDefault)
		));

		if (!AlphaValue->GetOnUpdate().IsBoundToObject(this))
		{
			AlphaValue->GetOnUpdate().AddSP(this, &SDMMaterialProperties::OnComponentUpdated);
		}
	}
	else if (UDMMaterialValueDynamic* AlphaValueDynamic = Cast<UDMMaterialValueDynamic>(AlphaObject))
	{
		AlphaItem->SetResetToDefaultOverride(FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(AlphaValueDynamic, &UDMMaterialValueDynamic::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(AlphaValueDynamic, &UDMMaterialValueDynamic::ResetToDefault)
		));

		if (!AlphaValueDynamic->GetOnUpdate().IsBoundToObject(this))
		{
			AlphaValueDynamic->GetOnUpdate().AddSP(this, &SDMMaterialProperties::OnComponentUpdated);
		}
	}

	AlphaItem->MakeWidget(nullptr, SharedThis(this));

	SliderItems.Add(AlphaItem);

	TSharedPtr<SWidget> ValueWidget = AlphaItem->GetWidget(ECustomDetailsViewWidgetType::Value);
	TSharedPtr<SWidget> ExtensionWidget = AlphaItem->GetWidget(ECustomDetailsViewWidgetType::Extensions);

	if (ValueWidget.IsValid())
	{
		if (TSharedPtr<SWidget> FoundPropertyValueWidget = FDMWidgetLibrary::Get().FindWidgetInHierarchy(ValueWidget.ToSharedRef(), FDMWidgetLibrary::PropertyValueWidget))
		{
			if (TSharedPtr<SWidget> InnerPropertyWidget = FDMWidgetLibrary::Get().GetInnerPropertyValueWidget(FoundPropertyValueWidget.ToSharedRef()))
			{
				ValueWidget = InnerPropertyWidget;
			}
		}
	}

	const FText PropertyGlobalSliderToolTipFormat = LOCTEXT("PropertyGlobalSliderToolTipFormat", "Change the global {0} value.");

	const FText PropertyGlobalSliderToolTip = FText::Format(
		PropertyGlobalSliderToolTipFormat,
		UE::DynamicMaterialEditor::Private::GetMaterialPropertyLongDisplayName(InProperty->GetMaterialProperty())
	);

	return SNew(SHorizontalBox)
		.ToolTipText(PropertyGlobalSliderToolTip)
		
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SBox)
			.HeightOverride(32.f)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				ValueWidget.IsValid() ? ValueWidget.ToSharedRef() : SNullWidget::NullWidget
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.HeightOverride(32.f)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				ExtensionWidget.IsValid() ? ExtensionWidget.ToSharedRef() : SNullWidget::NullWidget
			]
		];
}

void SDMMaterialProperties::OnExpansionStateChanged(const TSharedRef<ICustomDetailsViewItem>& InItem, bool bInExpansionState)
{
	if (bConstructing)
	{
		return;
	}

	const FCustomDetailsViewItemId& ItemId = InItem->GetItemId();

	if (ItemId.GetItemType() != static_cast<uint32>(EDetailNodeType::Category))
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (!MaterialModelBase)
	{
		return;
	}

	FDMWidgetLibrary::Get().SetExpansionState(MaterialModelBase, *ItemId.GetItemName(), bInExpansionState);
}

void SDMMaterialProperties::OnSettingsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, PropertyPreviewSize))
	{
		for (const TSharedRef<SBox>& PropertyPreviewContainer : PropertyPreviewContainers)
		{
			PropertyPreviewContainer->SetWidthOverride(Settings->PropertyPreviewSize + 4.f);
			PropertyPreviewContainer->SetHeightOverride(Settings->PropertyPreviewSize + 4.f);
		}

		for (const TSharedRef<SBox>& PropertyEmptyContainer : PropertyEmptyContainers)
		{
			PropertyEmptyContainer->SetWidthOverride(Settings->PropertyPreviewSize + 4.f);
		}

		for (const TSharedRef<SDMMaterialComponentPreview>& PropertyPreview : PropertyPreviews)
		{
			PropertyPreview->SetPreviewSize(FVector2D(Settings->PropertyPreviewSize));
		}
	}
}

bool SDMMaterialProperties::OnAssetDraggedOver(TArrayView<FAssetData> InAssets, EDMMaterialPropertyType InMaterialProperty)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return false;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return false;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return false;
	}

	UDMMaterialProperty* Property = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!Property || !Property->IsEnabled())
	{
		return false;
	}

	const TArray<UClass*> AllowedClasses = {
		UTexture::StaticClass()
	};

	TArray<FAssetData> Textures;

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		for (UClass* AllowedClass : AllowedClasses)
		{
			if (AssetClass->IsChildOf(AllowedClass))
			{
				Textures.Add(Asset);
			}
		}
	}

	return Textures.Num() == 1;
}

void SDMMaterialProperties::OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets, 
	EDMMaterialPropertyType InMaterialProperty)
{
	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			HandleDrop_Texture(Cast<UTexture>(Asset.GetAsset()), InMaterialProperty);
			break;
		}
	}
}

void SDMMaterialProperties::HandleDrop_Texture(UTexture* InTexture, EDMMaterialPropertyType InMaterialProperty)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	UDMMaterialProperty* Property = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!Property || !Property->IsEnabled())
	{
		return;
	}

	UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForMaterialProperty(InMaterialProperty);

	if (!Slot)
	{
		return;
	}

	const EAppReturnType::Type Result = FMessageDialog::Open(
		EAppMsgType::YesNoCancel,
		LOCTEXT("ReplaceSlotsTextureSet",
			"Material Designer Channel.\n\n"
			"Replace Slot?\n\n"
			"- Yes: Delete Layers.\n"
			"- No: Add Layer.\n"
			"- Cancel")
	);

	FDMScopedUITransaction Transaction(LOCTEXT("DropTexture", "Drop Texture On Channel"));

	switch (Result)
	{
		case EAppReturnType::Yes:
			UDMMaterialSlotFunctionLibrary::AddTextureLayer(Slot, InTexture, InMaterialProperty, /* Replace Slot */ true);
			break;

		case EAppReturnType::No:
			UDMMaterialSlotFunctionLibrary::AddTextureLayer(Slot, InTexture, InMaterialProperty, /* Replace Slot */ false);
			break;

		default:
			Transaction.Transaction.Cancel();
			break;
	}
}

#undef LOCTEXT_NAMESPACE
