// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

#include "Containers/Array.h"
#include "UI/Utils/DMWidgetSlot.h"

class ICustomDetailsView;
class ICustomDetailsViewItem;
class SBox;
class SDMMaterialComponentPreview;
class SDMMaterialEditor;
class SVerticalBox;
class UDMMaterialComponent;
class UDMMaterialProperty;
class UDMMaterialSlot;
class UTexture;
enum class ECheckBoxState : uint8;
enum class EDMMaterialPropertyType : uint8;
enum class EDMUpdateType : uint8;
struct FPropertyChangedEvent;

class SDMMaterialProperties : public SCompoundWidget, public FNotifyHook
{
	SLATE_DECLARE_WIDGET(SDMMaterialProperties, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialProperties) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialProperties() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget);

	void Validate();

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;

	TDMWidgetSlot<SWidget> Content;

	TArray<TSharedRef<ICustomDetailsViewItem>> GlobalItems;
	TArray<TSharedRef<SBox>> PropertyPreviewContainers;
	TArray<TSharedRef<SBox>> PropertyEmptyContainers;
	TArray<TSharedRef<SDMMaterialComponentPreview>> PropertyPreviews;
	TArray<TSharedRef<ICustomDetailsViewItem>> SliderItems;

	bool bConstructing = false;

	TSharedRef<SWidget> CreateSlot_Content();

	void AddProperty(const TSharedRef<ICustomDetailsView>& InDetailsView, const TSharedRef<ICustomDetailsViewItem>& InCategory, 
		UDMMaterialProperty* InProperty);

	TSharedRef<SWidget> CreatePropertyRow(UDMMaterialProperty* InProperty);

	TSharedRef<SWidget> CreateSlot_EnabledButton(EDMMaterialPropertyType InMaterialProperty);

	TSharedRef<SWidget> CreateSlot_PropertyName(EDMMaterialPropertyType InMaterialProperty);

	bool GetPropertyEnabledEnabled(EDMMaterialPropertyType InMaterialProperty) const;
	ECheckBoxState GetPropertyEnabledState(EDMMaterialPropertyType InMaterialProperty) const;
	void OnPropertyEnabledStateChanged(ECheckBoxState InState, EDMMaterialPropertyType InMaterialProperty);

	FReply OnPropertyClicked(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, EDMMaterialPropertyType InMaterialProperty);

	TSharedRef<SWidget> CreateGlobalSlider(UDMMaterialProperty* InProperty);

	void OnExpansionStateChanged(const TSharedRef<ICustomDetailsViewItem>& InItem, bool bInExpansionState);

	void OnSettingsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent);

	bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets, EDMMaterialPropertyType InMaterialProperty);

	void OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets, EDMMaterialPropertyType InMaterialProperty);

	void HandleDrop_Texture(UTexture* InTexture, EDMMaterialPropertyType InMaterialProperty);

	void OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	UDMMaterialComponent* GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const;
};
