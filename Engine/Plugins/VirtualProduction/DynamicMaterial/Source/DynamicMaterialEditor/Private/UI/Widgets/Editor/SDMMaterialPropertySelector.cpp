// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"

#include "Components/DMMaterialProperty.h"
#include "DynamicMaterialEditorSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Styling/StyleColors.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UObject/Class.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialPropertySelector"

void SDMMaterialPropertySelector::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SDMMaterialPropertySelector::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget)
{
	EditorWidgetWeak = InEditorWidget;

	SetCanTick(false);

	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)
		+ SScrollBox::Slot()
		[
			CreateSlot_PropertyList()
		]
	];
}

TSharedPtr<SDMMaterialEditor> SDMMaterialPropertySelector::GetEditorWidget() const
{
	return EditorWidgetWeak.Pin();
}

void SDMMaterialPropertySelector::SetSelectedProperty(const FDMMaterialEditorPage& InPage)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	switch (InPage.EditorMode)
	{
		case EDMMaterialEditorMode::MaterialPreview:
			OpenMaterialPreviewTab();
			break;

		default:
			EditorWidget->SetActivePage(InPage);
			break;
	}
}

void SDMMaterialPropertySelector::SetupMaterialPreviewButton(const TSharedRef<SWidget>& InSelectButton)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	TSharedPtr<IToolTip> PreviewToolTip = EditorWidget->GetMaterialPreviewToolTip();

	if (!PreviewToolTip.IsValid())
	{
		return;
	}

	InSelectButton->SetToolTip(PreviewToolTip);
}

void SDMMaterialPropertySelector::OpenMaterialPreviewTab()
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	EditorWidget->OpenMaterialPreviewTab();
}

UDynamicMaterialModelEditorOnlyData* SDMMaterialPropertySelector::GetEditorOnlyData() const
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!IsValid(MaterialModel))
	{
		return nullptr;
	}

	return UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);
}

TSharedPtr<SDMMaterialSlotEditor> SDMMaterialPropertySelector::GetSlotEditorWidget() const
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	return EditorWidget->GetSlotEditorWidget();
}

TSharedRef<SWidget> SDMMaterialPropertySelector::CreateSlot_EnabledButton(EDMMaterialPropertyType InMaterialProperty)
{
	const FText Format = LOCTEXT("PropertyEnableFormat", "Toggle the {0} property.\n\nProperty must be valid for the Material Type.");
	const FText ToolTip = FText::Format(Format, GetSelectButtonText({EDMMaterialEditorMode::EditSlot, InMaterialProperty}, /* Short Name */ false));

	return SNew(SCheckBox)
		.IsEnabled(this, &SDMMaterialPropertySelector::GetPropertyEnabledEnabled, InMaterialProperty)
		.IsChecked(this, &SDMMaterialPropertySelector::GetPropertyEnabledState, InMaterialProperty)
		.OnCheckStateChanged(this, &SDMMaterialPropertySelector::OnPropertyEnabledStateChanged, InMaterialProperty)
		.ToolTipText(ToolTip);
}

FText SDMMaterialPropertySelector::GetSelectButtonText(const FDMMaterialEditorPage& InPage, 
	bool bInShortName)
{
	switch (InPage.EditorMode)
	{
		case EDMMaterialEditorMode::MaterialPreview:
			return bInShortName
				? LOCTEXT("MaterialPreviewShort", "Prev")
				: LOCTEXT("MaterialPreview", "Material Preview");

		case EDMMaterialEditorMode::GlobalSettings:
			return bInShortName
				? LOCTEXT("GlobalSettingsShort", "Global")
				: LOCTEXT("GlobalSettings", "Global Settings");

		case EDMMaterialEditorMode::Properties:
			return bInShortName
				? LOCTEXT("ChannelsShort", "Chans")
				: LOCTEXT("Channels", "Channels");

		case EDMMaterialEditorMode::EditSlot:
			return bInShortName
				? UE::DynamicMaterialEditor::Private::GetMaterialPropertyShortDisplayName(InPage.MaterialProperty)
				: UE::DynamicMaterialEditor::Private::GetMaterialPropertyLongDisplayName(InPage.MaterialProperty);

		default:
			return FText::GetEmpty();
	}
}

FText SDMMaterialPropertySelector::GetButtonToolTip(const FDMMaterialEditorPage& InPage)
{
	switch (InPage.EditorMode)
	{
		case EDMMaterialEditorMode::MaterialPreview:
			return LOCTEXT("MaterialPreviewToolTip", "Show a preview of the Material.");

		case EDMMaterialEditorMode::GlobalSettings:
			return LOCTEXT("GeneralSettingsToolTip", "Edit the Material Global Settings.");

		case EDMMaterialEditorMode::Properties:
			return LOCTEXT("PropertyPreviewsToolTip", "Preview and toggle the Material Channels.");

		case EDMMaterialEditorMode::EditSlot:
		{
			const FText Format = LOCTEXT("PropertySelectFormat", "Edit the {0} channel.");
			return FText::Format(Format, GetSelectButtonText(InPage, /* Short Name */ false));
		}

		default:
			return FText::GetEmpty();
	}
}

bool SDMMaterialPropertySelector::IsPropertyEnabled(EDMMaterialPropertyType InMaterialProperty) const
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return false;
	}

	return !!EditorOnlyData->GetMaterialProperty(InMaterialProperty);
}

bool SDMMaterialPropertySelector::SetPropertyEnabled(EDMMaterialPropertyType InMaterialProperty, bool bInEnabled)
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return false;
	}

	UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!MaterialProperty)
	{
		return false;
	}

	MaterialProperty->SetEnabled(bInEnabled);

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		if (InMaterialProperty == EditorWidget->GetSelectedPropertyType())
		{
			SetSelectedProperty(FDMMaterialEditorPage::GlobalSettings);
		}
	}

	if (!bInEnabled)
	{
		return true;
	}

	if (EditorOnlyData->GetSlotForMaterialProperty(InMaterialProperty))
	{
		return true;
	}

	// Return true if the slot was successfully added
	return !!EditorOnlyData->AddSlotForMaterialProperty(InMaterialProperty);
}

bool SDMMaterialPropertySelector::DoesPropertySlotExist(EDMMaterialPropertyType InMaterialProperty) const
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return false;
	}

	UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!MaterialProperty)
	{
		return false;
	}

	if (!MaterialProperty->IsEnabled())
	{
		return false;
	}

	return !!EditorOnlyData->GetSlotForMaterialProperty(InMaterialProperty);
}

bool SDMMaterialPropertySelector::GetPropertyEnabledEnabled(EDMMaterialPropertyType InMaterialProperty) const
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = GetEditorOnlyData();

	if (!EditorOnlyData)
	{
		return false;
	}

	UDMMaterialProperty* MaterialProperty = EditorOnlyData->GetMaterialProperty(InMaterialProperty);

	if (!MaterialProperty)
	{
		return false;
	}

	return MaterialProperty->IsValidForModel(*EditorOnlyData);
}

ECheckBoxState SDMMaterialPropertySelector::GetPropertyEnabledState(EDMMaterialPropertyType InMaterialProperty) const
{
	return (GetPropertyEnabledEnabled(InMaterialProperty) && DoesPropertySlotExist(InMaterialProperty))
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SDMMaterialPropertySelector::OnPropertyEnabledStateChanged(ECheckBoxState InState, EDMMaterialPropertyType InMaterialProperty)
{
	const bool bSetEnabled = InState == ECheckBoxState::Checked;

	if (SetPropertyEnabled(InMaterialProperty, /* Enabled */ bSetEnabled))
	{
		if (bSetEnabled)
		{
			SetSelectedProperty({EDMMaterialEditorMode::EditSlot, InMaterialProperty});
		}
	}
}

bool SDMMaterialPropertySelector::GetPropertySelectEnabled(FDMMaterialEditorPage InPage) const
{
	switch (InPage.EditorMode)
	{
		case EDMMaterialEditorMode::EditSlot:
			return GetPropertyEnabledEnabled(InPage.MaterialProperty) && DoesPropertySlotExist(InPage.MaterialProperty);

		default:
			return true;
	}	
}

ECheckBoxState SDMMaterialPropertySelector::GetPropertySelectState(FDMMaterialEditorPage InPage) const
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return ECheckBoxState::Undetermined;
	}

	switch (InPage.EditorMode)
	{
		default:
			return EditorWidget->GetEditMode() == InPage.EditorMode
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;

		case EDMMaterialEditorMode::MaterialPreview:
			return ECheckBoxState::Unchecked;

		case EDMMaterialEditorMode::EditSlot:
			return InPage.MaterialProperty == EditorWidget->GetSelectedPropertyType()
				? ECheckBoxState::Checked
				: ECheckBoxState::Unchecked;
	}
}

void SDMMaterialPropertySelector::OnPropertySelectStateChanged(ECheckBoxState InState, FDMMaterialEditorPage InPage)
{
	if (InState != ECheckBoxState::Checked)
	{
		return;
	}

	SetSelectedProperty(InPage);
}

FSlateColor SDMMaterialPropertySelector::GetPropertySelectButtonChipColor(FDMMaterialEditorPage InPage) const
{
	switch (InPage.EditorMode)
	{
		case EDMMaterialEditorMode::MaterialPreview:
		case EDMMaterialEditorMode::GlobalSettings:
		case EDMMaterialEditorMode::Properties:
			return FStyleColors::AccentGreen;

		case EDMMaterialEditorMode::EditSlot:
			if (GetPropertySelectEnabled({EDMMaterialEditorMode::EditSlot, InPage.MaterialProperty}))
			{
				return FStyleColors::Primary;
			}

			// Fall through

		default:
			return FStyleColors::Panel;
	}
}

#undef LOCTEXT_NAMESPACE
