// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class SDMMaterialEditor;
class SDMMaterialSlotEditor;
class SWidget;
class UDMMaterialProperty;
class UDMMaterialSlot;
class UDynamicMaterialModelEditorOnlyData;
enum class ECheckBoxState : uint8;
enum class EDMMaterialEditorMode : uint8;
enum class EDMMaterialPropertyType : uint8;
struct FDMMaterialEditorPage;

namespace UE::DynamicMaterialEditor::Private
{
	namespace PropertySelectorColumns
	{
		constexpr int32 Enable = 0;
		constexpr int32 Select = 1;
	}
}

class SDMMaterialPropertySelector : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SDMMaterialPropertySelector, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialPropertySelector) {}
	SLATE_END_ARGS()

public:
	static FText GetSelectButtonText(const FDMMaterialEditorPage& InPage, bool bInShortName);

	static FText GetButtonToolTip(const FDMMaterialEditorPage& InPage);

	virtual ~SDMMaterialPropertySelector() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor> InEditorWidget);

	TSharedPtr<SDMMaterialEditor> GetEditorWidget() const;

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;

	UDynamicMaterialModelEditorOnlyData* GetEditorOnlyData() const;

	TSharedPtr<SDMMaterialSlotEditor> GetSlotEditorWidget() const;

	virtual TSharedRef<SWidget> CreateSlot_PropertyList() = 0;

	TSharedRef<SWidget> CreateSlot_EnabledButton(EDMMaterialPropertyType InMaterialProperty);

	virtual TSharedRef<SWidget> CreateSlot_SelectButton(const FDMMaterialEditorPage& InPage) = 0;

	bool IsPropertyEnabled(EDMMaterialPropertyType InMaterialProperty) const;

	bool SetPropertyEnabled(EDMMaterialPropertyType InMaterialProperty, bool bInEnabled);

	bool DoesPropertySlotExist(EDMMaterialPropertyType InMaterialProperty) const;

	bool GetPropertyEnabledEnabled(EDMMaterialPropertyType InMaterialProperty) const;
	ECheckBoxState GetPropertyEnabledState(EDMMaterialPropertyType InMaterialProperty) const;
	void OnPropertyEnabledStateChanged(ECheckBoxState InState, EDMMaterialPropertyType InMaterialProperty);

	bool GetPropertySelectEnabled(FDMMaterialEditorPage InPage) const;
	ECheckBoxState GetPropertySelectState(FDMMaterialEditorPage InPage) const;
	void OnPropertySelectStateChanged(ECheckBoxState InState, FDMMaterialEditorPage InPage);
	FSlateColor GetPropertySelectButtonChipColor(FDMMaterialEditorPage InPage) const;

	void SetSelectedProperty(const FDMMaterialEditorPage& InPage);

	void SetupMaterialPreviewButton(const TSharedRef<SWidget>& InSelectButton);

	void OpenMaterialPreviewTab();
};
