// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageBlends/DMMSBNormal.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIFunction.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"
#include "DetailLayoutBuilder.h"
#include "DMTextureSet.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "DynamicMaterialModule.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDynamicMaterialEditorModule.h"
#include "IDetailPropertyRow.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "SAssetDropTarget.h"
#include "UI/DragDrop/DMLayerEffectsDragDropOperation.h"
#include "UI/DragDrop/DMSlotLayerDragDropOperation.h"
#include "UI/Menus/DMMaterialSlotLayerAddEffectMenus.h"
#include "UI/Menus/DMMaterialSlotLayerMenus.h"
#include "UI/Utils/DMDropTargetPrivateSetter.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMMaterialSlotFunctionLibrary.h"
#include "Utils/DMMaterialStageFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialSlotEditor"

namespace UE::DynamicMaterialEditor::Private
{
class SDMLayerOpacityEditor : public SDMMaterialComponentEditor
{
	SLATE_DECLARE_WIDGET(SDMLayerOpacityEditor, SDMMaterialComponentEditor)

	SLATE_BEGIN_ARGS(SDMLayerOpacityEditor)
		{}
	SLATE_END_ARGS()

public:
	SDMLayerOpacityEditor()
	{
		bShowCategories = false;
	}

	virtual ~SDMLayerOpacityEditor() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialComponent* InMaterialComponent)
	{
		SDMMaterialComponentEditor::Construct(
			SDMMaterialComponentEditor::FArguments(),
			InEditorWidget,
			InMaterialComponent
		);
	}

protected:
	//~ Begin SDMObjectEditorWidgetBase
	virtual TArray<FDMPropertyHandle> GetPropertyRows() override
	{
		TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

		if (!EditorWidget)
		{
			return {};
		}

		TArray<FDMPropertyHandle> PropertyRows;
		TSet<UObject*> ProcessedObjects;

		UDMMaterialValue* Value = Cast<UDMMaterialValue>(GetComponent());

		FDMComponentPropertyRowGeneratorParams Params(PropertyRows, ProcessedObjects);
		Params.NotifyHook = this;
		Params.Owner = this;
		Params.Object = Value;
		Params.PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();
		Params.OriginalMaterialModelBase = EditorWidget->GetOriginalMaterialModelBase();

		FDMPropertyHandle& Handle = Params.PropertyRows->Add_GetRef(
			FDMWidgetLibrary::Get().GetPropertyHandle(Params.CreatePropertyHandleParams(UDMMaterialValue::ValueName))
		);

		Handle.NameOverride = LOCTEXT("LayerOpacity", "Layer Opacity");

		Handle.ResetToDefaultOverride = FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateUObject(Value, &UDMMaterialValue::CanResetToDefault),
			FResetToDefaultHandler::CreateUObject(Value, &UDMMaterialValue::ResetToDefault)
		);

		BindPropertyRowUpdateDelegates(PropertyRows);

		return PropertyRows;
	}
	//~ End SDMObjectEditorWidgetBase
};

void SDMLayerOpacityEditor::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}
}

void SDMMaterialSlotEditor::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialSlotEditor::~SDMMaterialSlotEditor()
{
	FDMWidgetLibrary::Get().ClearPropertyHandles(this);

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDMMaterialSlot* Slot = GetSlot())
	{
		Slot->GetOnPropertiesUpdateDelegate().RemoveAll(this);
		Slot->GetOnLayersUpdateDelegate().RemoveAll(this);
	}
}

void SDMMaterialSlotEditor::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialSlot* InSlot)
{
	EditorWidgetWeak = InEditorWidget;
	MaterialSlotWeak = InSlot;

	SetCanTick(false);

	bIsDynamic = !Cast<UDynamicMaterialModel>(InEditorWidget->GetPreviewMaterialModelBase());

	ContentSlot = TDMWidgetSlot<SWidget>(SharedThis(this), 0, SNullWidget::NullWidget);

	if (!IsValid(InSlot))
	{
		return;
	}

	InSlot->GetOnPropertiesUpdateDelegate().AddSP(this, &SDMMaterialSlotEditor::OnSlotPropertiesUpdated);
	InSlot->GetOnLayersUpdateDelegate().AddSP(this, &SDMMaterialSlotEditor::OnSlotLayersUpdated);

	ContentSlot << CreateSlot_Container();
}

void SDMMaterialSlotEditor::ValidateSlots()
{
	if (!MaterialSlotWeak.IsValid())
	{
		if (ContentSlot.HasWidget())
		{
			ContentSlot.ClearWidget();
		}

		return;
	}

	if (ContentSlot.HasBeenInvalidated())
	{
		ContentSlot << CreateSlot_Container();
	}
	else
	{
		if (SlotSettingsSlot.HasBeenInvalidated())
		{
			SlotSettingsSlot << CreateSlot_SlotSettings();
		}

		if (LayerViewSlot.HasBeenInvalidated())
		{
			LayerViewSlot << CreateSlot_LayerView();
		}

		if (LayerSettingsSlot.HasBeenInvalidated())
		{
			LayerSettingsSlot << CreateSlot_LayerSettings();
		}
	}
}

TSharedPtr<SDMMaterialEditor> SDMMaterialSlotEditor::GetEditorWidget() const
{
	return EditorWidgetWeak.Pin();
}

UDMMaterialSlot* SDMMaterialSlotEditor::GetSlot() const
{
	return MaterialSlotWeak.Get();
}

void SDMMaterialSlotEditor::ClearSelection()
{
	LayerViewSlot->ClearSelection();
}

bool SDMMaterialSlotEditor::CanAddNewLayer() const
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return false;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = Slot->GetMaterialModelEditorOnlyData();

	if (!EditorOnlyData)
	{
		return false;
	}

	return !EditorOnlyData->GetMaterialPropertiesForSlot(Slot).IsEmpty();
}

void SDMMaterialSlotEditor::AddNewLayer()
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = Slot->GetMaterialModelEditorOnlyData();
	TArray<EDMMaterialPropertyType> SlotProperties = EditorOnlyData->GetMaterialPropertiesForSlot(Slot);

	FDMScopedUITransaction Transaction(LOCTEXT("AddNewLayer", "Add New Layer"));
	Slot->Modify();

	UDMMaterialLayerObject* NewLayer = Slot->AddDefaultLayer(SlotProperties[0]);

	if (!NewLayer)
	{
		return;
	}

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		EditorWidget->EditSlot(Slot);

		if (UDMMaterialStage* Stage = NewLayer->GetFirstValidStage(EDMMaterialLayerStage::All))
		{
			EditorWidget->EditComponent(Stage);
		}
	}
}

bool SDMMaterialSlotEditor::CanInsertNewLayer() const
{
	return !!LayerViewSlot->GetSelectedLayer();
}

void SDMMaterialSlotEditor::InsertNewLayer()
{
	UDMMaterialLayerObject* SelectedLayer = LayerViewSlot->GetSelectedLayer();

	if (!SelectedLayer)
	{
		return;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("InsertNewLayer", "Insert New Layer"));
	Slot->Modify();

	UDMMaterialLayerObject* NewLayer = Slot->AddDefaultLayer(SelectedLayer->GetMaterialProperty());

	if (!NewLayer)
	{
		Transaction.Transaction.Cancel();
		return;
	}

	Slot->MoveLayerAfter(SelectedLayer, NewLayer);

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		EditorWidget->EditSlot(Slot);

		if (UDMMaterialStage* Stage = NewLayer->GetFirstValidStage(EDMMaterialLayerStage::All))
		{
			EditorWidget->EditComponent(Stage);
		}
	}
}

bool SDMMaterialSlotEditor::CanCopySelectedLayer() const
{
	return !!LayerViewSlot->GetSelectedLayer();
}

void SDMMaterialSlotEditor::CopySelectedLayer()
{
	UDMMaterialLayerObject* SelectedLayer = LayerViewSlot->GetSelectedLayer();

	FPlatformApplicationMisc::ClipboardCopy(*SelectedLayer->SerializeToString());
}

bool SDMMaterialSlotEditor::CanCutSelectedLayer() const
{
	return CanCopySelectedLayer() && CanDeleteSelectedLayer();
}

void SDMMaterialSlotEditor::CutSelectedLayer()
{
	FDMScopedUITransaction Transaction(LOCTEXT("CutLayer", "Cut Layer"));

	CopySelectedLayer();
	DeleteSelectedLayer();
}

bool SDMMaterialSlotEditor::CanPasteLayer() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return !ClipboardContent.IsEmpty();
}

void SDMMaterialSlotEditor::PasteLayer()
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	UDMMaterialLayerObject* PastedLayer = UDMMaterialLayerObject::DeserializeFromString(Slot, ClipboardContent);

	if (!PastedLayer)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("PasteLayer", "Paste Layer"));
	Slot->Modify();

	Slot->PasteLayer(PastedLayer);

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
	{
		EditorWidget->EditSlot(Slot);

		if (UDMMaterialStage* Stage = PastedLayer->GetFirstValidStage(EDMMaterialLayerStage::All))
		{
			EditorWidget->EditComponent(Stage);
		}
	}
}

bool SDMMaterialSlotEditor::CanDuplicateSelectedLayer() const
{
	// There's no "can add" check, so only copy is tested.
	return CanCopySelectedLayer();
}

void SDMMaterialSlotEditor::DuplicateSelectedLayer()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	// Added here to set the transaction description
	FDMScopedUITransaction Transaction(LOCTEXT("DuplicateLayer", "Duplicate Layer"));

	CopySelectedLayer();
	PasteLayer();

	FPlatformApplicationMisc::ClipboardCopy(*PastedText);
}

bool SDMMaterialSlotEditor::CanDeleteSelectedLayer() const
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return false;
	}

	UDMMaterialLayerObject* SelectedLayer = LayerViewSlot->GetSelectedLayer();

	if (!SelectedLayer)
	{
		return false;
	}

	return Slot->CanRemoveLayer(SelectedLayer);
}

void SDMMaterialSlotEditor::DeleteSelectedLayer()
{
	UDMMaterialSlot* Slot = GetSlot();
	UDMMaterialLayerObject* SelectedLayer = LayerViewSlot->GetSelectedLayer();

	FDMScopedUITransaction Transaction(LOCTEXT("DeleteLayer", "Delete Layer"));
	Slot->Modify();
	SelectedLayer->Modify();

	Slot->RemoveLayer(SelectedLayer);
}

bool SDMMaterialSlotEditor::SelectLayer_CanExecute(int32 InIndex) const
{
	return LayerViewSlot.HasWidget() && LayerViewSlot->GetItems().IsValidIndex(InIndex);
}

void SDMMaterialSlotEditor::SelectLayer_Execute(int32 InIndex)
{
	if (!SelectLayer_CanExecute(InIndex))
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	TArray<UDMMaterialLayerObject*> Layers = Slot->GetLayers();

	// Layer order is reversed.
	const int32 LayerIndex = Layers.Num() - 1 - InIndex;

	if (!Layers.IsValidIndex(InIndex))
	{
		return;
	}

	UDMMaterialLayerObject* Layer = Layers[LayerIndex];

	// Switch between stages
	if (LayerViewSlot->GetSelectedLayer() == Layer)
	{
		if (TSharedPtr<SDMMaterialComponentEditor> ComponentEditorWidget = EditorWidget->GetComponentEditorWidget())
		{
			UDMMaterialStage* BaseStage = Layer->GetFirstEnabledStage(EDMMaterialLayerStage::Base);
			UDMMaterialStage* MaskStage = Layer->GetFirstEnabledStage(EDMMaterialLayerStage::Mask);

			if (MaskStage && ComponentEditorWidget->GetObject() == BaseStage)
			{
				EditorWidget->EditComponent(MaskStage);
			}
			else if (BaseStage && ComponentEditorWidget->GetObject() == MaskStage)
			{
				EditorWidget->EditComponent(BaseStage);
			}
		}
	}
	// Select new layer
	else
	{
		LayerViewSlot->SetSelectedLayer(Layer);

		if (UDMMaterialStage* Stage = Layer->GetFirstEnabledStage(EDMMaterialLayerStage::All))
		{
			EditorWidget->EditComponent(Stage, /* Force Refersh */ false);
		}
		else
		{
			EditorWidget->EditComponent(nullptr);
		}
	}
}

bool SDMMaterialSlotEditor::SetOpacity_CanExecute()
{
	return LayerOpacityValueWeak.IsValid();
}

void SDMMaterialSlotEditor::SetOpacity_Execute(float InOpacity)
{
	if (UDMMaterialValueFloat1* OpacityValue = LayerOpacityValueWeak.Get())
	{
		OpacityValue->SetValue(InOpacity);
	}
}

TSharedRef<SDMMaterialSlotLayerView> SDMMaterialSlotEditor::GetLayerView() const
{
	return *LayerViewSlot;
}

void SDMMaterialSlotEditor::InvalidateSlotSettings()
{
	SlotSettingsSlot.Invalidate();
}

void SDMMaterialSlotEditor::InvalidateLayerView()
{
	LayerViewSlot.Invalidate();
}

void SDMMaterialSlotEditor::InvalidateLayerSettings()
{
	LayerSettingsSlot.Invalidate();
}

void SDMMaterialSlotEditor::SetSelectedLayer(UDMMaterialLayerObject* InLayer)
{
	if (LayerViewSlot.HasWidget())
	{
		LayerViewSlot->SetSelectedLayer(InLayer);
	}
}

void SDMMaterialSlotEditor::TriggerLayerSelectionChange(const TSharedRef<SDMMaterialSlotLayerView>& InLayerView,
	const TSharedPtr<FDMMaterialLayerReference>& InLayerReference)
{
	SlotSettingsSlot.Invalidate();

	OnLayerSelectionChanged.Broadcast(InLayerView, InLayerReference);
}

void SDMMaterialSlotEditor::TriggerStageSelectionChange(const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItem, UDMMaterialStage* InStage)
{
	if (UDMMaterialLayerObject* Layer = InStage->GetLayer())
	{
		SetSelectedLayer(Layer);
	}

	OnStageSelectionChanged.Broadcast(InLayerItem, InStage);
}

void SDMMaterialSlotEditor::TriggerEffectSelectionChange(const TSharedRef<SDMMaterialSlotLayerEffectView>& InEffectView, UDMMaterialEffect* InEffect)
{
	OnEffectSelectionChanged.Broadcast(InEffectView, InEffect);
}

void SDMMaterialSlotEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	if (UDMMaterialValue* OpacityValue = LayerOpacityValueWeak.Get())
	{
		OpacityValue->NotifyPreChange(PropertyAboutToChange);
	}
}

void SDMMaterialSlotEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (UDMMaterialValue* OpacityValue = LayerOpacityValueWeak.Get())
	{
		OpacityValue->NotifyPostChange(PropertyChangedEvent, PropertyThatChanged);
	}
}

TSharedRef<SWidget> SDMMaterialSlotEditor::CreateSlot_Container()
{
	SVerticalBox::FSlot* SettingsSlotPtr = nullptr;
	SScrollBox::FSlot* LayerViewSlotPtr = nullptr;
	SVerticalBox::FSlot* LayerSettingsSlotPtr = nullptr;

	TSharedPtr<SAssetDropTarget> DropTarget;

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(EOrientation::Orient_Vertical)
		.HideWhenNotInUse(true)
		.Style(&FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"));

	TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(EOrientation::Orient_Horizontal)
		.HideWhenNotInUse(true)
		.Style(&FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"));

	TSharedRef<SVerticalBox> NewContainer = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.HeightOverride(32.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Expose(SettingsSlotPtr)
				.AutoHeight()
				[
					SNullWidget::NullWidget
				]
			]			
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FDynamicMaterialEditorStyle::Get().GetBrush("LayerView.Background"))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				 SAssignNew(DropTarget, SAssetDropTarget)
				.OnAreAssetsAcceptableForDrop(this, &SDMMaterialSlotEditor::OnAreAssetsAcceptableForDrop)
				.OnAssetsDropped(this, &SDMMaterialSlotEditor::OnAssetsDropped)
				.bSupportsMultiDrop(true)
				//.bPlaceDropTargetOnTop(false)
				[
					SNew(SBox)
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.f)
							[
								SNew(SScrollBox)
								.Orientation(Orient_Horizontal)
								.ExternalScrollbar(HorizontalScrollBar)
								+ SScrollBox::Slot()
								.FillSize(1.f)
								[
									SNew(SScrollBox)
									.Orientation(EOrientation::Orient_Vertical)
									.ExternalScrollbar(VerticalScrollBar)
									+ SScrollBox::Slot()
									.Expose(LayerViewSlotPtr)
									.VAlign(EVerticalAlignment::VAlign_Fill)
									.Padding(0.f, 0.f, 0.f, 20.f)
									[
										SNullWidget::NullWidget
									]
								]
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								VerticalScrollBar
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.f)
							[
								HorizontalScrollBar
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBox)
								.WidthOverride(12.f)
								.HeightOverride(12.f)
							]
						]
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.Expose(LayerSettingsSlotPtr)
		.AutoHeight()
		[
			SNullWidget::NullWidget
		];

	SlotSettingsSlot = TDMWidgetSlot<SWidget>(SettingsSlotPtr, CreateSlot_SlotSettings());
	LayerViewSlot = TDMWidgetSlot<SDMMaterialSlotLayerView>(LayerViewSlotPtr, CreateSlot_LayerView());
	LayerSettingsSlot = TDMWidgetSlot<SWidget>(LayerSettingsSlotPtr, CreateSlot_LayerSettings());

	if (UDMMaterialSlot* Slot = GetSlot())
	{
		const TArray<UDMMaterialLayerObject*> Layers = Slot->GetLayers();

		if (!Layers.IsEmpty())
		{
			LayerViewSlot->SetSelectedLayer(Layers[0]);
		}
	}

	// Swap position of first and second child, so the drop border goes behind the list view.
	TSharedRef<SWidget> DropTargetFirstChild = DropTarget->GetChildren()->GetChildAt(0);
	check(DropTargetFirstChild->GetWidgetClass().GetWidgetType() == SOverlay::StaticWidgetClass().GetWidgetType());

	FChildren* DropTargetOverlayChildren = DropTargetFirstChild->GetChildren();

	TSharedRef<SWidget> FirstChild = DropTargetOverlayChildren->GetSlotAt(0).GetWidget();
	TSharedRef<SWidget> SecondChild = DropTargetOverlayChildren->GetSlotAt(1).GetWidget();

	const_cast<FSlotBase&>(DropTargetOverlayChildren->GetSlotAt(0)).DetachWidget();
	const_cast<FSlotBase&>(DropTargetOverlayChildren->GetSlotAt(1)).DetachWidget();

	const_cast<FSlotBase&>(DropTargetOverlayChildren->GetSlotAt(0)).AttachWidget(SecondChild);
	const_cast<FSlotBase&>(DropTargetOverlayChildren->GetSlotAt(1)).AttachWidget(FirstChild);

	using namespace UE::DynamicMaterialEditor::Private;
	DropTarget::SetInvalidColor(DropTarget.Get(), FStyleColors::Transparent);

	return NewContainer;
}

TSharedRef<SWidget> SDMMaterialSlotEditor::CreateSlot_SlotSettings()
{
	FDMWidgetLibrary::Get().ClearPropertyHandles(this);

	return CreateSlot_LayerOpacity();
}

TSharedRef<SWidget> SDMMaterialSlotEditor::CreateSlot_LayerOpacity()
{
	LayerOpacityValueWeak.Reset();
	LayerOpacityItem = nullptr;

	if (TSharedPtr<SDMMaterialEditor> MaterialEditor = EditorWidgetWeak.Pin())
	{
		if (LayerViewSlot.IsValid())
		{
			if (const UDMMaterialLayerObject* SelectedLayer = LayerViewSlot->GetSelectedLayer())
			{
				if (UDMMaterialStage* ValidStage = SelectedLayer->GetFirstValidStage(EDMMaterialLayerStage::All))
				{
					if (UDMMaterialStageInputValue* SelectedOpacityStageInputValue = UDMMaterialStageFunctionLibrary::FindDefaultStageOpacityInputValue(ValidStage))
					{
						if (UDMMaterialValueFloat1* OpacityValue = Cast<UDMMaterialValueFloat1>(SelectedOpacityStageInputValue->GetValue()))
						{
							using namespace UE::DynamicMaterialEditor::Private;

							LayerOpacityValueWeak = OpacityValue;
							LayerOpacityItem = SNew(SDMLayerOpacityEditor, MaterialEditor.ToSharedRef(), OpacityValue);
						}
					}
				}
			}
		}
	}

	if (!LayerOpacityItem.IsValid())
	{
		return SNullWidget::NullWidget;
	}


	return SNew(SBox)
		.HAlign(HAlign_Fill)
		.ToolTipText(LOCTEXT("MaterialDesignerInstanceLayerOpacityTooltip", "Change the Opacity of the selected Material Layer."))
		[
			SNew(SBox)
			.HeightOverride(32.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(0.f, 3.f, 0.f, 3.f)
			[
				LayerOpacityItem.ToSharedRef()
			]
		];
}

TSharedRef<SDMMaterialSlotLayerView> SDMMaterialSlotEditor::CreateSlot_LayerView()
{
	TSharedRef<SDMMaterialSlotLayerView> NewLayerView = SNew(SDMMaterialSlotLayerView, SharedThis(this));
	NewLayerView->EnsureSelectedStage();

	return NewLayerView;
}

TSharedRef<SWidget> SDMMaterialSlotEditor::CreateSlot_LayerSettings()
{
	TSharedPtr<SDropTarget> DropTarget;

	TSharedRef<SHorizontalBox> NewLayerSettings = SNew(SHorizontalBox)
		.IsEnabled(!bIsDynamic)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(5.0f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "SlotLayerInfo")
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Text(GetLayerButtonsDescription())
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.IsFocusable(true)
			.ContentPadding(4.0f)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly.Bordered.Dark")
			.ToolTipText(LOCTEXT("AddLayerEffecTooltip", "Add Layer Effect"))
			.IsEnabled(this, &SDMMaterialSlotEditor::GetLayerCanAddEffect)
			.OnGetMenuContent(this, &SDMMaterialSlotEditor::GetLayerEffectsMenuContent)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FDynamicMaterialEditorStyle::Get().GetBrush("EffectsView.Row.Fx"))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.DesiredSizeOverride(FVector2D(16.0f))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.IsFocusable(true)
			.ContentPadding(4.0f)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly.Bordered.Dark")
			.ToolTipText(LOCTEXT("AddLayerTooltip", "Add New Layer"))
			.OnGetMenuContent(this, &SDMMaterialSlotEditor::GetLayerButtonsMenuContent)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.DesiredSizeOverride(FVector2D(16.0f))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(5.0f, 2.0f, 0.0f, 2.0f)
		[
			SAssignNew(DropTarget, SDropTarget)
			.OnIsRecognized(this, &SDMMaterialSlotEditor::IsValidLayerDropForDelete)
			.OnAllowDrop(this, &SDMMaterialSlotEditor::CanDropLayerForDelete)
			.OnDropped(this, &SDMMaterialSlotEditor::OnLayerDroppedForDelete)
			[
				SNew(SButton)
				.ContentPadding(4.0f)
				.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly.Bordered.Dark")
				.ToolTipText(LOCTEXT("RemoveLayerTooltip", "Remove Selected Layer\n\nThe last layer cannot be removed."))
				.IsEnabled(this, &SDMMaterialSlotEditor::GetLayerRowsButtonsCanRemove)
				.OnClicked(this, &SDMMaterialSlotEditor::OnLayerRowButtonsRemoveClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.DesiredSizeOverride(FVector2D(16.0f))
				]
			]
		];

	return NewLayerSettings;
}

void SDMMaterialSlotEditor::OnSlotLayersUpdated(UDMMaterialSlot* InSlot)
{
	if (InSlot != GetSlot())
	{
		return;
	}
}

void SDMMaterialSlotEditor::OnSlotPropertiesUpdated(UDMMaterialSlot* InSlot)
{
	if (InSlot != GetSlot())
	{
		return;
	}
}

FText SDMMaterialSlotEditor::GetLayerButtonsDescription() const
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return FText::GetEmpty();
	}

	const int32 SlotLayerCount = Slot->GetLayers().Num();

	return SlotLayerCount == 1
		? LOCTEXT("SlotLayerInfo_OneLayer", "1 Layer")
		: FText::Format(LOCTEXT("SlotLayerInfo", "{0}|plural(one=Layer, other=Layers)"), SlotLayerCount);
}

TSharedRef<SWidget> SDMMaterialSlotEditor::GetLayerButtonsMenuContent()
{
	return FDMMaterialSlotLayerMenus::GenerateSlotLayerMenu(SharedThis(this), nullptr);
}

bool SDMMaterialSlotEditor::GetLayerCanAddEffect() const
{
	return !!LayerViewSlot->GetSelectedLayer();
}

TSharedRef<SWidget> SDMMaterialSlotEditor::GetLayerEffectsMenuContent()
{
	if (UDMMaterialLayerObject* LayerObject = LayerViewSlot->GetSelectedLayer())
	{
		return FDMMaterialSlotLayerAddEffectMenus::OpenAddEffectMenu(EditorWidgetWeak.Pin(), LayerObject);
	}

	return SNullWidget::NullWidget;
}

bool SDMMaterialSlotEditor::GetLayerRowsButtonsCanDuplicate() const
{
	return CanDuplicateSelectedLayer();
}

FReply SDMMaterialSlotEditor::OnLayerRowButtonsDuplicateClicked()
{
	DuplicateSelectedLayer();

	return FReply::Handled();
}

bool SDMMaterialSlotEditor::GetLayerRowsButtonsCanRemove() const
{
	return CanDeleteSelectedLayer();
}

FReply SDMMaterialSlotEditor::OnLayerRowButtonsRemoveClicked()
{
	DeleteSelectedLayer();

	return FReply::Handled();
}

void SDMMaterialSlotEditor::OnOpacityUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->ShouldAutomaticallyCopyParametersToSourceMaterial())
		{
			if (InComponent)
			{
				if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
				{
					if (UDMMaterialComponent* OriginalComponent = EditorWidget->GetOriginalComponent(InComponent))
					{
						IDMParameterContainer::CopyParametersBetween(InComponent, OriginalComponent);
					}
				}
			}
		}
	}
}

bool SDMMaterialSlotEditor::OnAreAssetsAcceptableForDrop(TArrayView<FAssetData> InAssets)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return false;
	}

	UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (PreviewMaterialModelBase->IsA<UDynamicMaterialModelDynamic>())
	{
		return false;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return false;
	}

	const TArray<UClass*> AllowedClasses = {
		UMaterialFunctionInterface::StaticClass()
	};

	TArray<FAssetData> DroppedTextures;

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		for (UClass* AllowedClass : AllowedClasses)
		{
			if (AssetClass->IsChildOf(AllowedClass))
			{
				return true;
			}
		}

		if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			DroppedTextures.Add(Asset);
		}
	}

	if (DroppedTextures.Num() == 1)
	{
		return true;
	}

	return false;
}

void SDMMaterialSlotEditor::OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (PreviewMaterialModelBase->IsA<UDynamicMaterialModelDynamic>())
	{
		return;
	}

	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	TArray<FAssetData> DroppedTextures;

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			DroppedTextures.Add(Asset);
			continue;
		}

		if (AssetClass->IsChildOf(UDMTextureSet::StaticClass()))
		{
			HandleDrop_TextureSet(Cast<UDMTextureSet>(Asset.GetAsset()));
			return;
		}

		if (AssetClass->IsChildOf(UMaterialFunctionInterface::StaticClass()))
		{
			HandleDrop_MaterialFunction(Cast<UMaterialFunctionInterface>(Asset.GetAsset()));
			return;
		}
	}

	if (DroppedTextures.Num() == 1)
	{
		HandleDrop_Texture(Cast<UTexture>(DroppedTextures[0].GetAsset()));
	}
	else if (DroppedTextures.Num() > 1)
	{
		HandleDrop_CreateTextureSet(DroppedTextures);
	}
}

void SDMMaterialSlotEditor::HandleDrop_Texture(UTexture* InTexture)
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("DropTexture", "Drop Texture"));
	Slot->Modify();

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialSlotFunctionLibrary::AddNewLayer(Slot, NewStage);

	UDMMaterialStageInputExpression* InputExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		NewStage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	UDMMaterialSubStage* SubStage = InputExpression->GetSubStage();

	if (ensure(SubStage))
	{
		UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			SubStage,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			EDMValueType::VT_Texture,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		if (ensure(InputValue))
		{
			UDMMaterialValueTexture* InputTexture = Cast<UDMMaterialValueTexture>(InputValue->GetValue());

			if (ensure(InputTexture))
			{
				InputTexture->SetValue(InTexture);
			}
		}
	}
}

void SDMMaterialSlotEditor::HandleDrop_CreateTextureSet(const TArray<FAssetData>& InTextureAssets)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	EditorWidget->HandleDrop_CreateTextureSet(InTextureAssets);
}

void SDMMaterialSlotEditor::HandleDrop_TextureSet(UDMTextureSet* InTextureSet)
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	EditorWidget->HandleDrop_TextureSet(InTextureSet);
}

void SDMMaterialSlotEditor::HandleDrop_MaterialFunction(UMaterialFunctionInterface* InMaterialFunction)
{
	UDMMaterialSlot* Slot = GetSlot();

	if (!Slot)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("DropFunction", "Drop Material Function"));
	Slot->Modify();

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* Layer = UDMMaterialSlotFunctionLibrary::AddNewLayer(Slot, NewStage);

	if (ensure(Layer))
	{
		UDMMaterialStageInputFunction* NewFunction = UDMMaterialStageInputFunction::ChangeStageInput_Function(
			NewStage,
			InMaterialFunction,
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		// The function was invalid and was removed. Remove the layer.
		if (!NewFunction->GetMaterialFunction())
		{
			Slot->RemoveLayer(Layer);
		}
	}
}

bool SDMMaterialSlotEditor::IsValidLayerDropForDelete(TSharedPtr<FDragDropOperation> InDragDropOperation)
{
	return InDragDropOperation.IsValid()
		&& (InDragDropOperation->IsOfType<FDMSlotLayerDragDropOperation>()
			|| InDragDropOperation->IsOfType<FDMLayerEffectsDragDropOperation>());
}

bool SDMMaterialSlotEditor::CanDropLayerForDelete(TSharedPtr<FDragDropOperation> InDragDropOperation)
{
	if (InDragDropOperation->IsOfType<FDMSlotLayerDragDropOperation>())
	{
		if (UDMMaterialLayerObject* Layer = StaticCastSharedPtr<FDMSlotLayerDragDropOperation>(InDragDropOperation)->GetLayer())
		{
			if (UDMMaterialSlot* Slot = Layer->GetSlot())
			{
				return Slot->CanRemoveLayer(Layer);
			}
		}
	}
	else if (InDragDropOperation->IsOfType<FDMLayerEffectsDragDropOperation>())
	{
		return IsValid(StaticCastSharedPtr<FDMLayerEffectsDragDropOperation>(InDragDropOperation)->GetMaterialEffect());
	}

	return false;	
}

FReply SDMMaterialSlotEditor::OnLayerDroppedForDelete(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (TSharedPtr<FDMSlotLayerDragDropOperation> LayerDragDropOperation = InDragDropEvent.GetOperationAs<FDMSlotLayerDragDropOperation>())
	{
		if (UDMMaterialLayerObject* Layer = LayerDragDropOperation->GetLayer())
		{
			if (UDMMaterialSlot* Slot = Layer->GetSlot())
			{
				Slot->RemoveLayer(Layer);
			}
		}
	}
	else if (TSharedPtr<FDMLayerEffectsDragDropOperation> EffectDragDropOperation = InDragDropEvent.GetOperationAs<FDMLayerEffectsDragDropOperation>())
	{
		if (UDMMaterialEffect* Effect = EffectDragDropOperation->GetMaterialEffect())
		{
			if (UDMMaterialEffectStack* EffectStack = Effect->GetEffectStack())
			{
				EffectStack->RemoveEffect(Effect);
			}
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE