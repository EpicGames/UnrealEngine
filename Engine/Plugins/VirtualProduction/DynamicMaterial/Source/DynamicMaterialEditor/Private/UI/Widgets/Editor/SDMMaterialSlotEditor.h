// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

#include "DynamicMaterialEditor/Private/UI/Utils/DMWidgetSlot.h"

class FDetailColumnSizeData;
class ICustomDetailsViewItem;
class SDMMaterialEditor;
class SDMMaterialSlotLayerEffectView;
class SDMMaterialSlotLayerItem;
class SDMMaterialSlotLayerView;
class SHorizontalBox;
class SVerticalBox;
class UDMMaterialComponent;
class UDMMaterialEffect;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialValueFloat1;
class UDMTextureSet;
class UMaterialFunctionInterface;
class UTexture;
enum class EDMUpdateType : uint8;
struct FDMMaterialLayerReference;

class SDMMaterialSlotEditor : public SCompoundWidget, public FNotifyHook
{
	SLATE_DECLARE_WIDGET(SDMMaterialSlotEditor, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialSlotEditor) {}
	SLATE_END_ARGS()

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLayerSelectionChanged, const TSharedRef<SDMMaterialSlotLayerView>&,
		const TSharedPtr<FDMMaterialLayerReference>&)

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStageSelectionChanged, const TSharedRef<SDMMaterialSlotLayerItem>&,
		UDMMaterialStage*)

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEffectSelectionChanged, const TSharedRef<SDMMaterialSlotLayerEffectView>&,
		UDMMaterialEffect*)

	virtual ~SDMMaterialSlotEditor() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialSlot* InSlot);

	void ValidateSlots();

	TSharedPtr<SDMMaterialEditor> GetEditorWidget() const;

	DLLEXPORT UDMMaterialSlot* GetSlot() const;

	/** Actions */
	void ClearSelection();

	bool CanAddNewLayer() const;
	void AddNewLayer();

	bool CanInsertNewLayer() const;
	void InsertNewLayer();

	bool CanCopySelectedLayer() const;
	void CopySelectedLayer();

	bool CanCutSelectedLayer() const;
	void CutSelectedLayer();

	bool CanPasteLayer() const;
	void PasteLayer();

	bool CanDuplicateSelectedLayer() const;
	void DuplicateSelectedLayer();

	bool CanDeleteSelectedLayer() const;
	void DeleteSelectedLayer();

	bool SelectLayer_CanExecute(int32 InIndex) const;
	void SelectLayer_Execute(int32 InIndex);

	bool SetOpacity_CanExecute();
	void SetOpacity_Execute(float InOpacity);

	/** Slots */
	TSharedRef<SDMMaterialSlotLayerView> GetLayerView() const;

	void InvalidateSlotSettings();

	void InvalidateLayerView();

	void InvalidateLayerSettings();

	/** Actions */
	void SetSelectedLayer(UDMMaterialLayerObject* InLayer);

	/** Events */
	FOnLayerSelectionChanged::RegistrationType& GetOnLayerSelectionChanged() { return OnLayerSelectionChanged; }
	void TriggerLayerSelectionChange(const TSharedRef<SDMMaterialSlotLayerView>& InLayerView, const TSharedPtr<FDMMaterialLayerReference>& InLayerReference);

	FOnStageSelectionChanged::RegistrationType& GetOnStageSelectionChanged() { return OnStageSelectionChanged; }
	void TriggerStageSelectionChange(const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItem, UDMMaterialStage* InStage);

	FOnEffectSelectionChanged::RegistrationType& GetOnEffectSelectionChanged() { return OnEffectSelectionChanged; }
	void TriggerEffectSelectionChange(const TSharedRef<SDMMaterialSlotLayerEffectView>& InEffectView, UDMMaterialEffect* InEffect);

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<UDMMaterialSlot> MaterialSlotWeak;
	bool bIsDynamic;

	TDMWidgetSlot<SWidget> ContentSlot;
	TDMWidgetSlot<SWidget> SlotSettingsSlot;
	TDMWidgetSlot<SDMMaterialSlotLayerView> LayerViewSlot;
	TDMWidgetSlot<SWidget> LayerSettingsSlot;

	TWeakObjectPtr<UDMMaterialValueFloat1> LayerOpacityValueWeak;
	TSharedPtr<SWidget> LayerOpacityItem;

	FOnLayerSelectionChanged OnLayerSelectionChanged;
	FOnStageSelectionChanged OnStageSelectionChanged;
	FOnEffectSelectionChanged OnEffectSelectionChanged;

	TSharedRef<SWidget> CreateSlot_Container();

	TSharedRef<SWidget> CreateSlot_SlotSettings();

	TSharedRef<SWidget> CreateSlot_LayerOpacity();

	TSharedRef<SDMMaterialSlotLayerView> CreateSlot_LayerView();

	TSharedRef<SWidget> CreateSlot_LayerSettings();

	void OnSlotLayersUpdated(UDMMaterialSlot* InSlot);

	void OnSlotPropertiesUpdated(UDMMaterialSlot* InSlot);

	FText GetLayerButtonsDescription() const;
	TSharedRef<SWidget> GetLayerButtonsMenuContent();

	bool GetLayerCanAddEffect() const;
	TSharedRef<SWidget> GetLayerEffectsMenuContent();

	bool GetLayerRowsButtonsCanDuplicate() const;
	FReply OnLayerRowButtonsDuplicateClicked();

	bool GetLayerRowsButtonsCanRemove() const;
	FReply OnLayerRowButtonsRemoveClicked();

	void OnOpacityUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	/** Drag and drop. */
	bool OnAreAssetsAcceptableForDrop(TArrayView<FAssetData> InAssets);

	void OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets);

	void HandleDrop_Texture(UTexture* InTexture);

	void HandleDrop_CreateTextureSet(const TArray<FAssetData>& InTextureAssets);

	void HandleDrop_TextureSet(UDMTextureSet* InTextureSet);

	void HandleDrop_MaterialFunction(UMaterialFunctionInterface* InMaterialFunction);

	bool IsValidLayerDropForDelete(TSharedPtr<FDragDropOperation> InDragDropOperation);

	bool CanDropLayerForDelete(TSharedPtr<FDragDropOperation> InDragDropOperation);

	FReply OnLayerDroppedForDelete(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
};
