// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMObjectEditorWidgetBase.h"

#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"
#include "CustomDetailsViewSequencer.h"
#include "DetailLayoutBuilder.h"
#include "DMWorldSubsystem.h"
#include "Engine/World.h"
#include "ICustomDetailsView.h"
#include "IDetailKeyframeHandler.h"
#include "Items/ICustomDetailsViewCustomCategoryItem.h"
#include "Items/ICustomDetailsViewCustomItem.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Model/DynamicMaterialModelBase.h"
#include "ScopedTransaction.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMObjectEditorWidgetBase"

void SDMObjectEditorWidgetBase::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMObjectEditorWidgetBase::~SDMObjectEditorWidgetBase()
{
	FDMWidgetLibrary::Get().ClearPropertyHandles(this);
}

void SDMObjectEditorWidgetBase::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UObject* InObject)
{
	EditorWidgetWeak = InEditorWidget;
	ObjectWeak = InObject;
	bConstructing = false;
	KeyframeHandler = nullptr;

	TGuardValue<bool> Constructing = TGuardValue<bool>(bConstructing, true);

	UObject* WorldContext = InObject;

	if (!WorldContext || !WorldContext->GetWorld())
	{
		WorldContext = InEditorWidget->GetOriginalMaterialModelBase();
	}

	if (WorldContext)
	{
		if (const UWorld* const World = WorldContext->GetWorld())
		{
			if (const UDMWorldSubsystem* const WorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>())
			{
				KeyframeHandler = WorldSubsystem->GetKeyframeHandler();
			}
		}
	}

	ContentSlot = TDMWidgetSlot<SWidget>(const_cast<FSlotBase*>(&GetChildren()->GetSlotAt(0)), CreateWidget());
}

void SDMObjectEditorWidgetBase::Validate()
{
	if (!ObjectWeak.IsValid())
	{
		ContentSlot.ClearWidget();
	}
}

void SDMObjectEditorWidgetBase::NotifyPreChange(FProperty* InPropertyAboutToChange)
{	
}

void SDMObjectEditorWidgetBase::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
}

TSharedRef<SWidget> SDMObjectEditorWidgetBase::CreateWidget()
{
	FDMWidgetLibrary::Get().ClearPropertyHandles(this);

	UObject* Object = ObjectWeak.Get();

	FCustomDetailsViewArgs Args;
	Args.KeyframeHandler = KeyframeHandler;
	Args.bAllowGlobalExtensions = true;
	Args.bAllowResetToDefault = true;
	Args.bShowCategories = bShowCategories;
	Args.RightColumnMinWidth = 75.f;
	Args.OnExpansionStateChanged.AddSP(this, &SDMObjectEditorWidgetBase::OnExpansionStateChanged);

	TSharedRef<ICustomDetailsView> DetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(Args);
	FCustomDetailsViewItemId RootId = DetailsView->GetRootItem()->GetItemId();

	TArray<FDMPropertyHandle> PropertyRows = GetPropertyRows();

	for (const FDMPropertyHandle& PropertyRow : PropertyRows)
	{
		const bool bHasValidCustomWidget = PropertyRow.ValueWidget.IsValid() && !PropertyRow.ValueName.IsNone();

		if (!PropertyRow.PreviewHandle.DetailTreeNode && !bHasValidCustomWidget)
		{
			continue;
		}

		ECustomDetailsTreeInsertPosition Position;

		switch (PropertyRow.Priority)
		{
			case EDMPropertyHandlePriority::High:
				Position = ECustomDetailsTreeInsertPosition::FirstChild;
				break;

			case EDMPropertyHandlePriority::Low:
				Position = ECustomDetailsTreeInsertPosition::LastChild;
				break;

			default:
				Position = ECustomDetailsTreeInsertPosition::Child;
				break;
		}

		TSharedPtr<ICustomDetailsViewItem> Parent;

		if (bShowCategories)
		{
			Parent = GetCategoryForRow(DetailsView, RootId, PropertyRow);
		}
		else
		{
			Parent = DetailsView->GetRootItem();
		}

		if (bHasValidCustomWidget)
		{
			AddCustomRow(DetailsView, Parent.ToSharedRef(), Position, PropertyRow);
		}
		else if (PropertyRow.PreviewHandle.DetailTreeNode)
		{
			AddDetailTreeRow(DetailsView, Parent.ToSharedRef(), Position, PropertyRow);
		}		
	}

	DetailsView->RebuildTree(ECustomDetailsViewBuildType::InstantBuild);

	return DetailsView;
}

TSharedRef<ICustomDetailsViewItem> SDMObjectEditorWidgetBase::GetDefaultCategory(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const FCustomDetailsViewItemId& InRootId)
{
	if (!DefaultCategoryItem.IsValid())
	{
		DefaultCategoryItem = InDetailsView->CreateCustomCategoryItem(InDetailsView->GetRootItem(), DefaultCategoryName, LOCTEXT("General", "General"))->AsItem();
		InDetailsView->ExtendTree(InRootId, ECustomDetailsTreeInsertPosition::Child, DefaultCategoryItem.ToSharedRef());

		bool bExpansionState = true;
		FDMWidgetLibrary::Get().GetExpansionState(ObjectWeak.Get(), DefaultCategoryName, bExpansionState);

		InDetailsView->SetItemExpansionState(
			DefaultCategoryItem->GetItemId(), 
			bExpansionState ? ECustomDetailsViewExpansion::SelfExpanded : ECustomDetailsViewExpansion::Collapsed
		);

		Categories.Add(DefaultCategoryName);
	}

	return DefaultCategoryItem.ToSharedRef();
}

TSharedRef<ICustomDetailsViewItem> SDMObjectEditorWidgetBase::GetCategoryForRow(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const FCustomDetailsViewItemId& InRootId, const FDMPropertyHandle& InPropertyRow)
{
	FName CategoryName = InPropertyRow.CategoryOverrideName;

	if (CategoryName.IsNone() && InPropertyRow.PreviewHandle.PropertyHandle.IsValid())
	{
		// Sub category (possibly)
		if (TSharedPtr<IPropertyHandle> SubCategoryProperty = InPropertyRow.PreviewHandle.PropertyHandle->GetParentHandle())
		{
			if (SubCategoryProperty->IsCategoryHandle())
			{
				// "Material Designer" (possibly)
				if (TSharedPtr<IPropertyHandle> MaterialDesignerCategoryProperty = SubCategoryProperty->GetParentHandle())
				{
					if (MaterialDesignerCategoryProperty->IsCategoryHandle())
					{
						CategoryName = *SubCategoryProperty->GetPropertyDisplayName().ToString();
					}
				}
			}
		}
	}

	if (CategoryName.IsNone())
	{
		return GetDefaultCategory(InDetailsView, InRootId);
	}

	TSharedPtr<ICustomDetailsViewItem> CategoryItem = InDetailsView->FindCustomItem(CategoryName);

	if (CategoryItem.IsValid())
	{
		return CategoryItem.ToSharedRef();
	}

	CategoryItem = InDetailsView->CreateCustomCategoryItem(InDetailsView->GetRootItem(), CategoryName, FText::FromName(CategoryName))->AsItem();
	InDetailsView->ExtendTree(InRootId, ECustomDetailsTreeInsertPosition::Child, CategoryItem.ToSharedRef());

	bool bExpansionState = true;
	FDMWidgetLibrary::Get().GetExpansionState(ObjectWeak.Get(), CategoryName, bExpansionState);

	InDetailsView->SetItemExpansionState(
		CategoryItem->GetItemId(),
		bExpansionState ? ECustomDetailsViewExpansion::SelfExpanded : ECustomDetailsViewExpansion::Collapsed
	);

	Categories.Add(CategoryName);

	return CategoryItem.ToSharedRef();
}

void SDMObjectEditorWidgetBase::AddDetailTreeRow(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const TSharedRef<ICustomDetailsViewItem>& InParent, ECustomDetailsTreeInsertPosition InPosition, const FDMPropertyHandle& InPropertyRow)
{
	TSharedRef<ICustomDetailsViewItem> Item = InDetailsView->CreateDetailTreeItem(InParent, InPropertyRow.PreviewHandle.DetailTreeNode.ToSharedRef());

	CustomizeItemContextMenu(Item, InPropertyRow);

	if (InPropertyRow.NameOverride.IsSet())
	{
		Item->SetOverrideWidget(
			ECustomDetailsViewWidgetType::Name,
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(InPropertyRow.NameOverride.GetValue())
			.ToolTipText(InPropertyRow.NameToolTipOverride.Get(FText::GetEmpty()))
		);
	}

	if (!InPropertyRow.bEnabled)
	{
		Item->SetEnabledOverride(false);

		// Disable the expansion widgets (SNullWidget is treated as removing the override).
		Item->SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, SNew(SBox));
	}

	if (!InPropertyRow.bKeyframeable)
	{
		Item->SetKeyframeEnabled(false);
	}

	if (InPropertyRow.ResetToDefaultOverride.IsSet())
	{
		Item->SetResetToDefaultOverride(InPropertyRow.ResetToDefaultOverride.GetValue());
	}

	if (InPropertyRow.MaxWidth.IsSet())
	{
		Item->SetValueWidgetWidthOverride(InPropertyRow.MaxWidth);
	}

	Item->SetCreateChildItemDelegate(FOnCustomDetailsViewGenerateChildItem::CreateSP(this, &SDMObjectEditorWidgetBase::CreateChildItem, InPropertyRow));

	if (InPropertyRow.PreviewHandle.PropertyHandle.IsValid())
	{
		AddDetailTreeRowExtensionWidgets(InDetailsView, InPropertyRow, Item, InPropertyRow.PreviewHandle.PropertyHandle.ToSharedRef());
	}

	InDetailsView->ExtendTree(InParent->GetItemId(), InPosition, Item);
}

void SDMObjectEditorWidgetBase::AddCustomRow(const TSharedRef<ICustomDetailsView>& InDetailsView, 
	const TSharedRef<ICustomDetailsViewItem>& InParent, ECustomDetailsTreeInsertPosition InPosition, 
	const FDMPropertyHandle& InPropertyRow)
{
	const FText RowDisplayName = InPropertyRow.NameOverride.Get(FText::GetEmpty());
	const bool bIsWholeRow = RowDisplayName.IsEmpty();

	TSharedPtr<ICustomDetailsViewCustomItem> Item = InDetailsView->CreateCustomItem(
		InParent,
		InPropertyRow.ValueName,
		RowDisplayName,
		InPropertyRow.NameToolTipOverride.Get(FText::GetEmpty())
	);

	if (!Item.IsValid())
	{
		return;
	}

	CustomizeItemContextMenu(Item->AsItem(), InPropertyRow);

	if (bIsWholeRow)
	{
		Item->SetWholeRowWidget(InPropertyRow.ValueWidget.ToSharedRef());
	}
	else
	{
		Item->SetValueWidget(InPropertyRow.ValueWidget.ToSharedRef());

		if (!InPropertyRow.bEnabled)
		{
			Item->AsItem()->SetEnabledOverride(false);

			// Disable the expansion widgets (SNullWidget is treated as removing the override).
			Item->SetExpansionWidget(SNew(SBox));
		}

		if (InPropertyRow.MaxWidth.IsSet())
		{
			Item->AsItem()->SetValueWidgetWidthOverride(InPropertyRow.MaxWidth);
		}

		if (InPropertyRow.PreviewHandle.PropertyHandle.IsValid())
		{
			AddDetailTreeRowExtensionWidgets(InDetailsView, InPropertyRow, Item->AsItem(), InPropertyRow.PreviewHandle.PropertyHandle.ToSharedRef());
		}
	}

	InDetailsView->ExtendTree(InParent->GetItemId(), InPosition, Item->AsItem());
}

void SDMObjectEditorWidgetBase::OnExpansionStateChanged(const TSharedRef<ICustomDetailsViewItem>& InItem, bool bInExpansionState)
{
	if (bConstructing)
	{
		return;
	}

	const FCustomDetailsViewItemId& ItemId = InItem->GetItemId();

	if (ItemId.GetItemType() != static_cast<uint32>(EDetailNodeType::Category))
	{
		return;
	}

	FDMWidgetLibrary::Get().SetExpansionState(ObjectWeak.Get(), *ItemId.GetItemName(), bInExpansionState);
}

void SDMObjectEditorWidgetBase::AddDetailTreeRowExtensionWidgets(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const FDMPropertyHandle& InPropertyRow, const TSharedRef<ICustomDetailsViewItem>& InItem, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
}

TOptional<FPropertyRowExtensionButton> SDMObjectEditorWidgetBase::CreateKeyframeButton(TSharedPtr<IPropertyHandle> InPreviewPropertyHandle,
	TSharedPtr<IPropertyHandle> InOriginalPropertyHandle)
{
	if (!InPreviewPropertyHandle.IsValid() || !InOriginalPropertyHandle.IsValid()
		|| !InPreviewPropertyHandle->IsValidHandle() || !InOriginalPropertyHandle->IsValidHandle())
	{
		return {};
	}

	TArray<FPropertyRowExtensionButton> ExtensionButtons;
	FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(KeyframeHandler, InPreviewPropertyHandle, ExtensionButtons);

	if (ExtensionButtons.IsEmpty())
	{
		return {};
	}

	ExtensionButtons[0].Icon = TAttribute<FSlateIcon>::CreateSP(
		this,
		&SDMObjectEditorWidgetBase::GetCreateKeyIcon,
		InPreviewPropertyHandle.ToWeakPtr(),
		InOriginalPropertyHandle.ToWeakPtr()
	);

	ExtensionButtons[0].UIAction.CanExecuteAction.BindSP(
		this,
		&SDMObjectEditorWidgetBase::CanCreateKeyFrame,
		InPreviewPropertyHandle.ToWeakPtr(),
		InOriginalPropertyHandle.ToWeakPtr()
	);

	ExtensionButtons[0].UIAction.IsActionVisibleDelegate.BindSP(
		this,
		&SDMObjectEditorWidgetBase::CanCreateKeyFrame,
		InPreviewPropertyHandle.ToWeakPtr(),
		InOriginalPropertyHandle.ToWeakPtr()
	);

	ExtensionButtons[0].UIAction.ExecuteAction.BindSP(
		this,
		&SDMObjectEditorWidgetBase::CreateKeyFrame,
		InPreviewPropertyHandle.ToWeakPtr(),
		InOriginalPropertyHandle.ToWeakPtr()
	);

	return ExtensionButtons[0];
}

FSlateIcon SDMObjectEditorWidgetBase::GetCreateKeyIcon(TWeakPtr<IPropertyHandle> InPreviewPropertyHandleWeak,
	TWeakPtr<IPropertyHandle> InOriginalPropertyHandleWeak) const
{
	if (!CanCreateKeyFrame(InPreviewPropertyHandleWeak, InOriginalPropertyHandleWeak))
	{
		return FSlateIcon();
	}

	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	if (KeyframeHandler.IsValid())
	{
		KeyedStatus = KeyframeHandler->GetPropertyKeyedStatus(*InOriginalPropertyHandleWeak.Pin());
	}

	FName FoundIcon;

	switch (KeyedStatus)
	{
		default:
		case EPropertyKeyedStatus::NotKeyed:
			FoundIcon = TEXT("Sequencer.KeyedStatus.NotKeyed");
			break;

		case EPropertyKeyedStatus::KeyedInFrame:
			FoundIcon = TEXT("Sequencer.KeyedStatus.Keyed");
			break;

		case EPropertyKeyedStatus::KeyedInOtherFrame:
			FoundIcon = TEXT("Sequencer.KeyedStatus.Animated");
			break;

		case EPropertyKeyedStatus::PartiallyKeyed:
			FoundIcon = TEXT("Sequencer.KeyedStatus.PartialKey");
			break;
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), FoundIcon);
}

bool SDMObjectEditorWidgetBase::CanCreateKeyFrame(TWeakPtr<IPropertyHandle> InPreviewPropertyHandleWeak,
	TWeakPtr<IPropertyHandle> InOriginalPropertyHandleWeak) const
{
	if (!KeyframeHandler.IsValid())
	{
		return false;
	}

	TSharedPtr<IPropertyHandle> PreviewPropertyHandle = InPreviewPropertyHandleWeak.Pin();

	if (!PreviewPropertyHandle.IsValid())
	{
		return false;
	}

	TSharedPtr<IPropertyHandle> OriginalPropertyHandle = InOriginalPropertyHandleWeak.Pin();

	if (!OriginalPropertyHandle.IsValid())
	{
		return false;
	}

	if (!PreviewPropertyHandle->IsValidHandle() || !OriginalPropertyHandle->IsValidHandle())
	{
		return false;
	}

	if (UObject* Object = GetObject())
	{
		return KeyframeHandler->IsPropertyKeyable(Object->GetClass(), *OriginalPropertyHandle);
	}

	return false;
}

void SDMObjectEditorWidgetBase::CreateKeyFrame(TWeakPtr<IPropertyHandle> InPreviewPropertyHandleWeak,
	TWeakPtr<IPropertyHandle> InOriginalPropertyHandleWeak)
{
	TSharedPtr<IPropertyHandle> PreviewPropertyHandle = InPreviewPropertyHandleWeak.Pin();
	TSharedPtr<IPropertyHandle> OriginalPropertyHandle = InOriginalPropertyHandleWeak.Pin();

	if (!PreviewPropertyHandle.IsValid() || !OriginalPropertyHandle.IsValid()
		|| !PreviewPropertyHandle->IsValidHandle() || !OriginalPropertyHandle->IsValidHandle())
	{
		return;
	}

	TArray<UObject*> OriginalObjects;
	OriginalPropertyHandle->GetOuterObjects(OriginalObjects);

	if (OriginalObjects.IsEmpty())
	{
		return;
	}

	FString Value;
	PreviewPropertyHandle->GetValueAsFormattedString(Value);

	if (Value.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CreateKeyFrame", "Create Key Frame"));

	for (UObject* Object : OriginalObjects)
	{
		if (IsValid(Object))
		{
			Object->Modify();
		}
	}

	OriginalPropertyHandle->SetValueFromFormattedString(Value);
	KeyframeHandler->OnKeyPropertyClicked(*OriginalPropertyHandle);
}

FPropertyRowExtensionButton SDMObjectEditorWidgetBase::CreateNeedsApplyButton() const
{
	FPropertyRowExtensionButton NeedsApplyButton;
	NeedsApplyButton.Label = LOCTEXT("NeedsApply", "Needs Apply");
	NeedsApplyButton.Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("MaterialEditor.Apply"));
	NeedsApplyButton.ToolTip = LOCTEXT("NeedsApplyTooltip", "This property only exists in the preview material. Apply changes to add this to the source material.");

	return NeedsApplyButton;
}

TSharedPtr<ICustomDetailsViewItem> SDMObjectEditorWidgetBase::CreateChildItem(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const TSharedPtr<ICustomDetailsViewItem>& InParent, const TSharedRef<IDetailTreeNode>& InChildNode, FDMPropertyHandle InOriginalRow)
{
	if (!InParent.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle = InChildNode->CreatePropertyHandle();

	if (!PropertyHandle)
	{
		return nullptr;
	}

	// Only customise struct properties
	if (PropertyHandle->GetPropertyPath().Find(TEXT("->")) == INDEX_NONE)
	{
		return nullptr;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return nullptr;
	}

	UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

	if (!PreviewMaterialModelBase)
	{
		return nullptr;
	}

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	if (Objects.Num() != 1 || !IsValid(Objects[0]))
	{
		return nullptr;
	}

	if (!Objects[0]->IsIn(PreviewMaterialModelBase))
	{
		return nullptr;
	}

	return CreateChildItem_Impl(InDetailsView, InParent.ToSharedRef(), InChildNode, PropertyHandle.ToSharedRef(), InOriginalRow);
}

TSharedPtr<ICustomDetailsViewItem> SDMObjectEditorWidgetBase::CreateChildItem_Impl(const TSharedRef<ICustomDetailsView>& InDetailsView,
	const TSharedRef<ICustomDetailsViewItem>& InParent, const TSharedRef<IDetailTreeNode>& InChildNode, const TSharedRef<IPropertyHandle>& InPropertyHandle,
	const FDMPropertyHandle& InOriginalRow)
{
	TSharedRef<ICustomDetailsViewItem> Item = InDetailsView->CreateDetailTreeItem(InParent, InChildNode);

	CustomizeItemContextMenu(Item, InOriginalRow);

	if (!InOriginalRow.bEnabled)
	{
		Item->SetEnabledOverride(false);

		// Disable the expansion widgets (SNullWidget is treated as removing the override).
		Item->SetOverrideWidget(ECustomDetailsViewWidgetType::Extensions, SNew(SBox));
	}

	if (!InOriginalRow.bKeyframeable)
	{
		Item->SetKeyframeEnabled(false);
	}

	if (InOriginalRow.ResetToDefaultOverride.IsSet())
	{
		Item->SetResetToDefaultOverride(InOriginalRow.ResetToDefaultOverride.GetValue());
	}

	if (InOriginalRow.MaxWidth.IsSet())
	{
		Item->SetValueWidgetWidthOverride(InOriginalRow.MaxWidth);
	}

	Item->SetCreateChildItemDelegate(FOnCustomDetailsViewGenerateChildItem::CreateSP(this, &SDMObjectEditorWidgetBase::CreateChildItem, InOriginalRow));

	AddDetailTreeRowExtensionWidgets(InDetailsView, InOriginalRow, Item, InPropertyHandle);

	return Item;
}

void SDMObjectEditorWidgetBase::CustomizeItemContextMenu(const TSharedRef<ICustomDetailsViewItem>& InItem, const FDMPropertyHandle& InPropertyRow)
{
	if (InPropertyRow.PreviewHandle.DetailTreeNode.IsValid())
	{
		InItem->SetCustomizeItemMenuContext(FOnCustomDetailsViewCustomizeItemMenuContext::CreateSPLambda(this,
			[this, InPropertyRow](const TSharedRef<ICustomDetailsView>&, const TSharedPtr<ICustomDetailsViewItem>&, UObject*, TArray<TSharedPtr<IPropertyHandle>>& InPropertyHandles)
			{
				if (InPropertyRow.OriginalHandle.PropertyHandle.IsValid())
				{
					InPropertyHandles.Add(InPropertyRow.OriginalHandle.PropertyHandle);
				}
			}
		));
	}
}

#undef LOCTEXT_NAMESPACE
