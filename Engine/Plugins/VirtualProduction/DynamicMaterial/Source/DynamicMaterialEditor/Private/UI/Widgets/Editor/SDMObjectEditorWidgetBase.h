// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Containers/Array.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "UI/Utils/DMWidgetSlot.h"
#include "UObject/WeakObjectPtr.h"

class FCustomDetailsViewItemId;
class FName;
class ICustomDetailsView;
class ICustomDetailsViewItem;
class IDetailKeyframeHandler;
class IDetailTreeNode;
class IPropertyHandle;
class SDMMaterialEditor;
class UObject;
enum class ECustomDetailsTreeInsertPosition : uint8;
struct FDMPropertyHandle;
struct FPropertyRowExtensionButton;
struct FSlateIcon;

/** Base class for the object editor. Provides the methods and layout for producing a Custom Details View. */
class SDMObjectEditorWidgetBase : public SCompoundWidget, public FNotifyHook
{
	SLATE_DECLARE_WIDGET(SDMObjectEditorWidgetBase, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMObjectEditorWidgetBase) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMObjectEditorWidgetBase() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UObject* InObject);

	UObject* GetObject() const { return ObjectWeak.Get(); }

	TSharedPtr<SDMMaterialEditor> GetEditorWidget() const { return EditorWidgetWeak.Pin(); }

	void Validate();

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	static constexpr const TCHAR* DefaultCategoryName = TEXT("General");

	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<UObject> ObjectWeak;

	TDMWidgetSlot<SWidget> ContentSlot;

	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler;
	bool bConstructing;
	TArray<FName> Categories;
	TSharedPtr<ICustomDetailsViewItem> DefaultCategoryItem;
	bool bShowCategories = true;

	TSharedRef<SWidget> CreateWidget();

	virtual TSharedRef<ICustomDetailsViewItem> GetDefaultCategory(const TSharedRef<ICustomDetailsView>& InDetailsView,
		const FCustomDetailsViewItemId& InRootId);

	TSharedRef<ICustomDetailsViewItem> GetCategoryForRow(const TSharedRef<ICustomDetailsView>& InDetailsView, 
		const FCustomDetailsViewItemId& InRootId, const FDMPropertyHandle& InPropertyRow);

	void AddDetailTreeRow(const TSharedRef<ICustomDetailsView>& InDetailsView, const TSharedRef<ICustomDetailsViewItem>& InParent,
		ECustomDetailsTreeInsertPosition InPosition, const FDMPropertyHandle& InPropertyRow);

	void AddCustomRow(const TSharedRef<ICustomDetailsView>& InDetailsView, const TSharedRef<ICustomDetailsViewItem>& InParent,
		ECustomDetailsTreeInsertPosition InPosition, const FDMPropertyHandle& InPropertyRow);

	void OnExpansionStateChanged(const TSharedRef<ICustomDetailsViewItem>& InItem, bool bInExpansionState);

	virtual TArray<FDMPropertyHandle> GetPropertyRows() = 0;

	virtual void AddDetailTreeRowExtensionWidgets(const TSharedRef<ICustomDetailsView>& InDetailsView, const FDMPropertyHandle& InPropertyRow,
		const TSharedRef<ICustomDetailsViewItem>& InItem, const TSharedRef<IPropertyHandle>& InPropertyHandle);

	TOptional<FPropertyRowExtensionButton> CreateKeyframeButton(TSharedPtr<IPropertyHandle> InPreviewPropertyHandle, TSharedPtr<IPropertyHandle> InOriginalPropertyHandle);

	FSlateIcon GetCreateKeyIcon(TWeakPtr<IPropertyHandle> InPreviewPropertyHandleWeak, TWeakPtr<IPropertyHandle> InOriginalPropertyHandleWeak) const;

	bool CanCreateKeyFrame(TWeakPtr<IPropertyHandle> InPreviewPropertyHandleWeak, TWeakPtr<IPropertyHandle> InOriginalPropertyHandleWeak) const;

	void CreateKeyFrame(TWeakPtr<IPropertyHandle> InPreviewPropertyHandleWeak, TWeakPtr<IPropertyHandle> InOriginalPropertyHandleWeak);

	FPropertyRowExtensionButton CreateNeedsApplyButton() const;

	TSharedPtr<ICustomDetailsViewItem> CreateChildItem(const TSharedRef<ICustomDetailsView>& InDetailsView, const TSharedPtr<ICustomDetailsViewItem>& InParent,
		const TSharedRef<IDetailTreeNode>& InChildNode, FDMPropertyHandle InOriginalRow);

	virtual TSharedPtr<ICustomDetailsViewItem> CreateChildItem_Impl(const TSharedRef<ICustomDetailsView>& InDetailsView, const TSharedRef<ICustomDetailsViewItem>& InParent,
		const TSharedRef<IDetailTreeNode>& InChildNode, const TSharedRef<IPropertyHandle>& InPropertyHandle, const FDMPropertyHandle& InOriginalRow);

	void CustomizeItemContextMenu(const TSharedRef<ICustomDetailsViewItem>& InItem, const FDMPropertyHandle& InPropertyRow);
};
