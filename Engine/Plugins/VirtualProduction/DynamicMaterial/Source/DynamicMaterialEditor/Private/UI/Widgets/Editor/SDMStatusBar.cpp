// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMStatusBar.h"

#include "DynamicMaterialEditorStyle.h"
#include "DynamicMaterialModule.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMStatusBar"

SDMStatusBar::~SDMStatusBar()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = EditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	EditorOnlyData->GetOnMaterialBuiltDelegate().RemoveAll(this);
}

void SDMStatusBar::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDynamicMaterialModelBase* InMaterialModelBase)
{
	ensure(IsValid(InMaterialModelBase));
	MaterialModelBaseWeak = InMaterialModelBase;
	EditorWidgetWeak = InEditorWidget;

	SetCanTick(false);

	UDynamicMaterialModel* MaterialModel = InEditorWidget->GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	CachedMaterialStats = EditorOnlyData->GetMaterialStats();

	EditorOnlyData->GetOnMaterialBuiltDelegate().AddSP(this, &SDMStatusBar::OnMaterialBuilt);

	ContentSlot = TDMWidgetSlot<SWidget>(SharedThis(this), 0, SNullWidget::NullWidget);

	// No stats!
	if (CachedMaterialStats.NumPixelShaderInstructions > 0)
	{
		ContentSlot << CreateContent();
	}
}

TSharedRef<SWidget> SDMStatusBar::CreateContent()
{
	return SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.BorderImage(FDynamicMaterialEditorStyle::Get().GetBrush("Border.Top"))
		.BorderBackgroundColor(FLinearColor(1, 1, 1, 0.05f))
		[
			SNew(SWrapBox)
			.HAlign(HAlign_Right)
			.InnerSlotPadding(FVector2D(5.0f))
			.UseAllottedSize(true)
			+ CreateStatsWrapBoxEntry(
				GetNumPixelShaderInstructionsText(), 
				LOCTEXT("NumPixelShaderInstructions_ToolTip", "Pixel shader instruction count")
			)
			+ CreateStatsWrapBoxEntry(
				GetNumVertexShaderInstructionsText(), 
				LOCTEXT("NumVertexShaderInstructions_ToolTip", "Vertex shader instruction count")
			)
			+ CreateStatsWrapBoxEntry(
				GetNumSamplersText(), 
				LOCTEXT("NumSamplers_ToolTip", "Sampler count")
			)
		];
}

SWrapBox::FSlot::FSlotArguments SDMStatusBar::CreateStatsWrapBoxEntry(const FText& InText, const FText& InTooltipText)
{
	constexpr FLinearColor SeparatorColor = FLinearColor(1, 1, 1, 0.1f);

	SWrapBox::FSlot::FSlotArguments SlotArgs(SWrapBox::Slot());

	SlotArgs
		.Padding(0.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			.ToolTipText(InTooltipText)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			[
				SNew(SBorder)
				.BorderBackgroundColor(SeparatorColor)
				.BorderImage(FDynamicMaterialEditorStyle::Get().GetBrush("Border.Left"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(5.0f, 0.0f, 5.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(InText)
			]
		];

	return SlotArgs;
}

FText SDMStatusBar::GetNumPixelShaderInstructionsText() const
{
	return FText::Format(LOCTEXT("NumPixelShaderInstructions_Text", "PS Instructions {0}"), CachedMaterialStats.NumPixelShaderInstructions);
}

FText SDMStatusBar::GetNumVertexShaderInstructionsText() const
{
	return FText::Format(LOCTEXT("NumVertexShaderInstructions_Text", "VS Instructions {0}"), CachedMaterialStats.NumVertexShaderInstructions);
}

FText SDMStatusBar::GetNumSamplersText() const
{
	return FText::Format(LOCTEXT("NumSamplers_Text", "Samplers {0}"), CachedMaterialStats.NumSamplers);
}

void SDMStatusBar::OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModelBase)
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(InMaterialModelBase);

	if (!EditorOnlyData)
	{
		return;
	}

	CachedMaterialStats = EditorOnlyData->GetMaterialStats();

	ContentSlot << CreateContent();
}

#undef LOCTEXT_NAMESPACE
