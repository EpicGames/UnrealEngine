// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "MaterialEditingLibrary.h"
#include "UI/Utils/DMWidgetSlot.h"
#include "Widgets/Layout/SWrapBox.h"

class SDMMaterialEditor;
class UDynamicMaterialModelBase;

class SDMStatusBar : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDMStatusBar) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMStatusBar() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDynamicMaterialModelBase* InMaterialModelBase);

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<UDynamicMaterialModelBase> MaterialModelBaseWeak;

	TDMWidgetSlot<SWidget> ContentSlot;

	int32 CachedSlotCount = 0;
	int32 CachedCurrentLayerCount = 0;
	int32 CachedTotalLayerCount = 0;

	FMaterialStatistics CachedMaterialStats;

	TSharedRef<SWidget> CreateContent();

	SWrapBox::FSlot::FSlotArguments CreateStatsWrapBoxEntry(const FText& InText, const FText& InTooltipText);

	FText GetNumPixelShaderInstructionsText() const;
	FText GetNumVertexShaderInstructionsText() const;
	FText GetNumSamplersText() const;

	void OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModelBase);
};
