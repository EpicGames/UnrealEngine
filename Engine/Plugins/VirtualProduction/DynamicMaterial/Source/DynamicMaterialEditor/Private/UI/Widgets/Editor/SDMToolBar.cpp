// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SDMToolBar.h"

#include "AssetRegistry/AssetData.h"
#include "Components/PrimitiveComponent.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorModule.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "Editor.h"
#include "GameFramework/Actor.h"
#include "Layout/Margin.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Math/Vector2D.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "UI/Menus/DMToolBarMenus.h"
#include "UI/Widgets/SDMMaterialDesigner.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UObject/Object.h"
#include "Utils/DMMaterialInstanceFunctionLibrary.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMToolBar"

namespace UE::DynamicMaterialEditor::Private
{
	static const FMargin DefaultToolBarButtonContentPadding = FMargin(2.0f);
	static const FVector2D DefaultToolBarButtonSize = FVector2D(20.f);

	static const FMargin LargeIconToolBarButtonContentPadding = FMargin(4.f);
	static const FVector2D LargeIconToolBarButtonSize = FVector2D(16.f);
}

void SDMToolBar::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, const FDMObjectMaterialProperty& InObjectProperty)
{
	EditorWidgetWeak = InEditorWidget;

	if (UObject* Outer = InObjectProperty.GetOuter())
	{
		if (AActor* OuterActor = Cast<AActor>(Outer))
		{
			MaterialActorWeak = OuterActor;
		}
		else if (AActor* OuterActorOuter = Outer->GetTypedOuter<AActor>())
		{
			MaterialActorWeak = OuterActorOuter;
		}
	}

	SelectedMaterialElementIndex = InObjectProperty.GetIndex();
	
	SetCanTick(false);
	
	ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.BorderImage(FDynamicMaterialEditorStyle::Get().GetBrush("Border.Bottom"))
			.BorderBackgroundColor(FLinearColor(1, 1, 1, 0.05f))
			.Padding(0.f, 3.f, 0.f, 3.f)
			[
				CreateToolBarEntries()
			]
		];

	SetActorPropertySelected(InObjectProperty);
	UpdateButtonVisibilities();
}

TSharedRef<SWidget> SDMToolBar::CreateToolBarEntries()
{
	using namespace UE::DynamicMaterialEditor::Private;

	const FSlateBrush* ActorBrush = nullptr;

	if (AActor* Actor = MaterialActorWeak.Get())
	{
		ActorBrush = FSlateIconFinder::FindIconBrushForClass(Actor->GetClass());
	}

	if (!ActorBrush)
	{
		ActorBrush = FSlateIconFinder::FindIconBrushForClass(AActor::StaticClass());
	}

	return SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SWrapBox)
			.Orientation(Orient_Horizontal)
			.UseAllottedSize(true)
			.HAlign(HAlign_Left)
			.InnerSlotPadding(FVector2D(5.0f))

			+ SWrapBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(AssetRowWidget, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ContentPadding(LargeIconToolBarButtonContentPadding)
					.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("MaterialDesignerBrowseTooltip", "Browse to the selected asset in the content browser."))
					.OnClicked(this, &SDMToolBar::OnBrowseClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.BrowseContent")))
						.DesiredSizeOverride(LargeIconToolBarButtonSize)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 0.f, 0.f)
				[
					SAssignNew(AssetNameWidget, STextBlock)
					.TextStyle(FDynamicMaterialEditorStyle::Get(), "ActorName")
				]
			]

			+ SWrapBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SAssignNew(SaveButtonWidget, SButton)
					.Visibility(EVisibility::Collapsed)
					.ContentPadding(LargeIconToolBarButtonContentPadding)
					.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("MaterialDesignerSaveTooltip", "Save the Material Designer Asset\n\nCaution: If this asset lives inside an actor, the actor/level will be saved."))
					.OnClicked(this, &SDMToolBar::OnSaveClicked)
					[
						SNew(SImage)
						.Image(this, &SDMToolBar::GetSaveIcon)
						.DesiredSizeOverride(LargeIconToolBarButtonSize)
					] 
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f)
				[
					SAssignNew(OpenParentButton, SButton)
					.Visibility(EVisibility::Collapsed)
					.ContentPadding(LargeIconToolBarButtonContentPadding)
					.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("MaterialDesignerOpenParentTooltip", "Open the parent of this Material Designer Instance."))
					.OnClicked(this, &SDMToolBar::OnOpenParentClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Blueprints")))
						.DesiredSizeOverride(LargeIconToolBarButtonSize)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f)
				[
					SAssignNew(ConvertToEditableButton, SButton)
					.Visibility(EVisibility::Collapsed)
					.ContentPadding(LargeIconToolBarButtonContentPadding)
					.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("MaterialDesignerConvertToEditableTooltip", "Convert this Material Designer Instance to a fully editable Material (and create a new shader)."))
					.OnClicked(this, &SDMToolBar::OnConvertToEditableClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Edit")))
						.DesiredSizeOverride(LargeIconToolBarButtonSize)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f)
				[
					SAssignNew(ConvertToInstanceButton, SButton)
					.Visibility(EVisibility::Collapsed)
					.ContentPadding(LargeIconToolBarButtonContentPadding)
					.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("MaterialDesignerConvertToInstanceTooltip", "Convert this Material Designer Material to an Instance, creating a local Instance inside the Actor."))
					.OnClicked(this, &SDMToolBar::OnConvertToInstanceClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Toolbar.Export")))
						.DesiredSizeOverride(LargeIconToolBarButtonSize)
					]
				]
			]

			+ SWrapBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(InstanceWidget, STextBlock)
				.TextStyle(FDynamicMaterialEditorStyle::Get(), "ActorName")
				.Text(LOCTEXT("Instance", "(Inst)"))
				.Visibility(EVisibility::Collapsed)
			]

			+ SWrapBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(ActorRowWidget, SHorizontalBox)
				.Visibility(EVisibility::Collapsed)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 5.f, 0.f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(SImage)
					.Image(ActorBrush)
					.DesiredSizeOverride(LargeIconToolBarButtonSize)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 0.f, 0.f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SAssignNew(ActorNameWidget, STextBlock)
					.TextStyle(FDynamicMaterialEditorStyle::Get(), "ActorName")
				]	
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(PropertySelectorContainer, SBox)
					[
						CreateSlotsComboBoxWidget()
					]
				]	
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ContentPadding(LargeIconToolBarButtonContentPadding)
					.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("MaterialDesignerUseTooltip", "Replace the material in this slot with the one selected in the content browser."))
					.OnClicked(this, &SDMToolBar::OnUseClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.Use")))
						.DesiredSizeOverride(LargeIconToolBarButtonSize)
					]
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.ContentPadding(LargeIconToolBarButtonContentPadding)
			.Visibility(this, &SDMToolBar::GetAutoApplyVisibility)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("MaterialDesignerAutoApplyTooltip", "Auto Apply to Source\n\nWhen enabled, and when the preview material is recompiled, it will automatically apply the changes to the source asset.\n\nToggling on Auto Apply to Source will cause any compiled changes to apply to the source immediately."))
			.OnClicked(this, &SDMToolBar::OnAutoApplyClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("MaterialEditor.Apply"))
				.ColorAndOpacity(this, &SDMToolBar::GetAutoApplyColor)
				.DesiredSizeOverride(LargeIconToolBarButtonSize)
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.ContentPadding(LargeIconToolBarButtonContentPadding)
			.Visibility(this, &SDMToolBar::GetAutoCompileVisibility)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("MaterialDesignerAutoCompileTooltip", "Live Preview\n\nWhen enabled the preview material will be compiled for every structural change.\n\nToggling on Live Preview will cause any uncompiled changes to compile immediately."))
			.OnClicked(this, &SDMToolBar::OnAutoCompileClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("MaterialEditor.LiveUpdate"))
				.ColorAndOpacity(this, &SDMToolBar::GetAutoCompileColor)
				.DesiredSizeOverride(LargeIconToolBarButtonSize)
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.ContentPadding(LargeIconToolBarButtonContentPadding)
			.Visibility(this, &SDMToolBar::GetLiveEditVisibility)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("MaterialDesignerCombinedAutoApplyTooltip", "Live Edit\n\nWhen enabled the preview and source material will recompile for every structural change made to the material (such as adding new layers or changing layer types).\n\nWhen disabled the material is freely editable and will not trigger any material compiles until this option is enabled."))
			.OnClicked(this, &SDMToolBar::OnLiveEditClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("MaterialEditor.Apply"))
				.ColorAndOpacity(this, &SDMToolBar::GetLiveEditColor)
				.DesiredSizeOverride(LargeIconToolBarButtonSize)
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.ContentPadding(DefaultToolBarButtonContentPadding)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("MaterialDesignerFollowSelectionTooltip", "Follow Selection\n\nWhen enabled the Material Designer will open newly selected objects, assets and actors."))
			.OnClicked(this, &SDMToolBar::OnFollowSelectionButtonClicked)
			[
				SNew(SImage)
				.Image(this, &SDMToolBar::GetFollowSelectionBrush)
				.DesiredSizeOverride(DefaultToolBarButtonSize)
				.ColorAndOpacity(this, &SDMToolBar::GetFollowSelectionColor)
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.IsFocusable(true)
			.ContentPadding(DefaultToolBarButtonContentPadding)
			.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("MaterialDesignerSettingsTooltip", "Material Designer Settings"))
			.OnGetMenuContent(this, &SDMToolBar::GenerateSettingsMenu)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FDynamicMaterialEditorStyle::Get().GetBrush("Icons.Menu.Dropdown"))
				.DesiredSizeOverride(DefaultToolBarButtonSize)
			]
		];
}

void SDMToolBar::SetActorPropertySelected(const FDMObjectMaterialProperty& InObjectProperty)
{
	AActor* Actor = nullptr;

	if (UObject* Outer = InObjectProperty.GetOuter())
	{
		if (AActor* OuterActor = Cast<AActor>(Outer))
		{
			Actor = OuterActor;
		}
		else if (AActor* OuterActorOuter = Outer->GetTypedOuter<AActor>())
		{
			Actor = OuterActorOuter;
		}
	}
	
	// We can only deal with non-property materials.
	if (Actor && InObjectProperty.IsValid() && InObjectProperty.IsElement())
	{
		MaterialActorWeak = Actor;
		SelectedMaterialElementIndex = InObjectProperty.GetIndex();

		ActorNameWidget->SetText(GetActorName());
		ActorRowWidget->SetVisibility(EVisibility::Visible);

		TArray<FDMObjectMaterialProperty> ActorProperties = UDMMaterialInstanceFunctionLibrary::GetActorMaterialProperties(Actor);
		UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase();

		const int32 ActorPropertyCount = ActorProperties.Num();
		ActorMaterialProperties.Empty(ActorPropertyCount);

		for (int32 MaterialPropertyIdx = 0; MaterialPropertyIdx < ActorPropertyCount; ++MaterialPropertyIdx)
		{
			const FDMObjectMaterialProperty& MaterialProperty = ActorProperties[MaterialPropertyIdx];
			ActorMaterialProperties.Add(MakeShared<FDMObjectMaterialProperty>(MaterialProperty));
		}
	}
	else
	{
		MaterialActorWeak.Reset();
		ActorMaterialProperties.Empty(0);
		ActorNameWidget->SetText(FText::GetEmpty());
		ActorRowWidget->SetVisibility(EVisibility::Collapsed);
	}

	PropertySelectorContainer->SetContent(CreateSlotsComboBoxWidget());
}

void SDMToolBar::UpdateButtonVisibilities()
{
	bool bIsAsset = false;
	bool bIsDynamic = false;

	UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase();

	if (IsValid(MaterialModelBase))
	{
		if (MaterialModelBase->IsAsset())
		{
			bIsAsset = true;
		}
		else if (UDynamicMaterialInstance* MaterialInstance = MaterialModelBase->GetDynamicMaterialInstance())
		{
			if (MaterialInstance->IsAsset())
			{
				bIsAsset = true;
			}
		}

		bIsDynamic = !MaterialModelBase->IsA<UDynamicMaterialModel>();
	}

	if (bIsAsset)
	{
		SaveButtonWidget->SetVisibility(EVisibility::Visible);

		AssetNameWidget->SetText(GetAssetName());
		AssetNameWidget->SetToolTipText(GetAssetToolTip());
		AssetNameWidget->SetVisibility(EVisibility::Visible);

		AssetRowWidget->SetVisibility(EVisibility::Visible);
	}
	else
	{
		SaveButtonWidget->SetVisibility(EVisibility::Collapsed);

		AssetNameWidget->SetText(FText::GetEmpty());
		AssetNameWidget->SetToolTipText(FText::GetEmpty());
		AssetNameWidget->SetVisibility(EVisibility::Collapsed);

		AssetRowWidget->SetVisibility(EVisibility::Collapsed);
	}

	if (bIsDynamic)
	{
		OpenParentButton->SetVisibility(EVisibility::Visible);
		ConvertToEditableButton->SetVisibility(EVisibility::Visible);
		InstanceWidget->SetVisibility(EVisibility::Visible);
	}
	else
	{
		OpenParentButton->SetVisibility(EVisibility::Collapsed);
		ConvertToEditableButton->SetVisibility(EVisibility::Collapsed);
		InstanceWidget->SetVisibility(EVisibility::Collapsed);
	}

	if (bIsAsset && !bIsDynamic && 
		!ActorMaterialProperties.IsEmpty() 
		&& ActorMaterialProperties.IsValidIndex(SelectedMaterialElementIndex)
		&& ActorMaterialProperties[SelectedMaterialElementIndex].IsValid() && GetMaterialActor())
	{
		ConvertToInstanceButton->SetVisibility(EVisibility::Visible);
	}
	else
	{
		ConvertToInstanceButton->SetVisibility(EVisibility::Collapsed);
	}
}

TSharedRef<SWidget> SDMToolBar::CreateToolBarButton(TAttribute<const FSlateBrush*> InImageBrush, const TAttribute<FText>& InTooltipText, FOnClicked InOnClicked)
{
	using namespace UE::DynamicMaterialEditor::Private;

	return SNew(SButton)
		.ContentPadding(DefaultToolBarButtonContentPadding)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(InTooltipText)
		.OnClicked(InOnClicked)
		[
			SNew(SImage)
			.Image(InImageBrush)
			.DesiredSizeOverride(DefaultToolBarButtonSize)
		];
}

TSharedRef<SWidget> SDMToolBar::CreateSlotsComboBoxWidget()
{
	UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase();

	if (!MaterialActorWeak.IsValid() || !IsValid(MaterialModelBase))
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<FDMObjectMaterialProperty> InitiallySelectedItem =
		ActorMaterialProperties.IsValidIndex(SelectedMaterialElementIndex) ? ActorMaterialProperties[SelectedMaterialElementIndex] : nullptr;

	return SNew(SComboBox<TSharedPtr<FDMObjectMaterialProperty>>)
		.IsEnabled(ActorMaterialProperties.Num() > 1)
		.InitiallySelectedItem(InitiallySelectedItem)
		.OptionsSource(&ActorMaterialProperties)
		.OnGenerateWidget(this, &SDMToolBar::GenerateSelectedMaterialSlotRow)
		.OnSelectionChanged(this, &SDMToolBar::OnMaterialSlotChanged)
		[
			SNew(STextBlock)
			.MinDesiredWidth(100.0f)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
			.Text(this, &SDMToolBar::GetSelectedMaterialSlotName)
		];
}

TSharedRef<SWidget> SDMToolBar::GenerateSelectedMaterialSlotRow(TSharedPtr<FDMObjectMaterialProperty> InSelectedSlot) const
{
	if (!InSelectedSlot.IsValid() || InSelectedSlot->IsProperty())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(STextBlock)
		.MinDesiredWidth(100.f)
		.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
		.Text(this, &SDMToolBar::GetSlotDisplayName, InSelectedSlot.ToWeakPtr())
		.IsEnabled(this, &SDMToolBar::GetSlotEnabled, InSelectedSlot.ToWeakPtr());
}

int32 SDMToolBar::GetDuplicateIndex(const FDMObjectMaterialProperty& InObjectProperty) const
{
	UPrimitiveComponent* Outer = Cast<UPrimitiveComponent>(InObjectProperty.GetOuter());

	if (!Outer)
	{
		return INDEX_NONE;
	}

	UDynamicMaterialModelBase* MyModel = InObjectProperty.GetMaterialModelBase();

	if (!MyModel)
	{
		return INDEX_NONE;
	}

	const int32 CurrentIndex = InObjectProperty.GetIndex();

	int32 SameMaterialAsSlot = -1;

	for (int32 Index = 0; Index < CurrentIndex; ++Index)
	{
		UDynamicMaterialModelBase* Model = FDMObjectMaterialProperty(Outer, Index).GetMaterialModelBase();

		if (Model && Model == MyModel)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

FText SDMToolBar::GetSlotDisplayName(TWeakPtr<FDMObjectMaterialProperty> InSlotWeak) const
{
	TSharedPtr<FDMObjectMaterialProperty> Slot = InSlotWeak.Pin();

	if (!Slot.IsValid())
	{
		return FText::GetEmpty();
	}

	UPrimitiveComponent* Outer = Cast<UPrimitiveComponent>(Slot->GetOuter());

	if (!Outer)
	{
		return FText::GetEmpty();
	}

	const int32 DuplicateIndex = GetDuplicateIndex(*Slot);

	if (DuplicateIndex != INDEX_NONE)
	{
		return FText::Format(
			LOCTEXT("ElementXCopy", "Element {0} [Duplicate of {1}]"),
			FText::AsNumber(Slot->GetIndex()),
			FText::AsNumber(DuplicateIndex)
		);
	}

	return Slot->GetPropertyName(/* Ignore "create new" status */ false);
}

bool SDMToolBar::GetSlotEnabled(TWeakPtr<FDMObjectMaterialProperty> InSlotWeak) const
{
	TSharedPtr<FDMObjectMaterialProperty> Slot = InSlotWeak.Pin();

	if (!Slot.IsValid())
	{
		return false;
	}

	return GetDuplicateIndex(*Slot) == INDEX_NONE;
}

FText SDMToolBar::GetSelectedMaterialSlotName() const
{
	if (ActorMaterialProperties.IsValidIndex(SelectedMaterialElementIndex) && ActorMaterialProperties[SelectedMaterialElementIndex].IsValid())
	{
		return GetSlotDisplayName(ActorMaterialProperties[SelectedMaterialElementIndex]);
	}
	return FText::GetEmpty();
}

void SDMToolBar::OnMaterialSlotChanged(TSharedPtr<FDMObjectMaterialProperty> InSelectedSlot, ESelectInfo::Type InSelectInfoType)
{
	if (!InSelectedSlot.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialDesigner> DesignerWidget = EditorWidget->GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	if (GetDuplicateIndex(*InSelectedSlot) != INDEX_NONE)
	{
		return;
	}

	UDynamicMaterialModelBase* SelectedMaterialModelBase = InSelectedSlot->GetMaterialModelBase();

	if (EditorWidget->GetOriginalMaterialModelBase() == SelectedMaterialModelBase)
	{
		return;
	}

	if (IsValid(SelectedMaterialModelBase))
	{
		DesignerWidget->OpenObjectMaterialProperty(*InSelectedSlot);
	}
	else if (InSelectedSlot->GetOuter())
	{
		if (UDynamicMaterialModel* NewModel = UDMMaterialInstanceFunctionLibrary::CreateMaterialInObject(*InSelectedSlot.Get()))
		{
			DesignerWidget->OpenObjectMaterialProperty(*InSelectedSlot);
		}
	}
}

bool SDMToolBar::IsDynamicMaterialModel() const
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		if (UDynamicMaterialModelBase* MaterialModelBase = EditorWidget->GetPreviewMaterialModel())
		{
			return MaterialModelBase->IsA<UDynamicMaterialModelDynamic>();
		}
	}

	return false;
}

bool SDMToolBar::IsAutoCompileAndApplyOverridden() const
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		return Settings->LiveEditMode != EDMLiveEditMode::Disabled;
	}

	return true;
}

UDynamicMaterialModelBase* SDMToolBar::GetOriginalMaterialModelBase() const
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		return EditorWidget->GetOriginalMaterialModelBase();
	}

	return nullptr;
}

const FSlateBrush* SDMToolBar::GetFollowSelectionBrush() const
{
	static const FSlateBrush* Unlocked = FAppStyle::Get().GetBrush("Icons.Unlock");
	static const FSlateBrush* Locked = FAppStyle::Get().GetBrush("Icons.Lock");

	if (!SDMMaterialDesigner::IsFollowingSelection())
	{
		return Locked;
	}

	return Unlocked;
}

FSlateColor SDMToolBar::GetFollowSelectionColor() const
{
	// We want the icon to stand out when it's locked.
	static FSlateColor EnabledColor = FSlateColor(EStyleColor::AccentGray);
	static FSlateColor DisabledColor = FSlateColor(EStyleColor::AccentBlue);

	if (SDMMaterialDesigner::IsFollowingSelection())
	{
		return EnabledColor;
	}

	return DisabledColor;
}

FReply SDMToolBar::OnFollowSelectionButtonClicked()
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		Settings->bFollowSelection = !Settings->bFollowSelection;
		Settings->SaveConfig();
	}

	return FReply::Handled();
}

FReply SDMToolBar::OnExportMaterialInstanceButtonClicked()
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<SDMMaterialDesigner> DesignerWidget = EditorWidget->GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return FReply::Handled();
	}

	UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase();

	if (!MaterialModelBase)
	{
		return FReply::Handled();
	}

	UDynamicMaterialInstance* NewInstance = UDMMaterialModelFunctionLibrary::ExportMaterial(MaterialModelBase);

	if (!NewInstance)
	{
		return FReply::Handled();
	}

	DesignerWidget->OpenMaterialInstance(NewInstance);

	return FReply::Handled();
}

FReply SDMToolBar::OnBrowseClicked()
{
	UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase();

	if (!IsValid(MaterialModelBase))
	{
		return FReply::Handled();
	}

	UObject* Asset = nullptr;

	if (MaterialModelBase->IsAsset())
	{
		Asset = MaterialModelBase;
	}
	else if (UDynamicMaterialInstance* MaterialInstance = MaterialModelBase->GetDynamicMaterialInstance())
	{
		if (MaterialInstance->IsAsset())
		{
			Asset = MaterialInstance;
		}
	}

	if (!Asset)
	{
		return FReply::Handled();
	}

	TArray<FAssetData> AssetDataList;
	AssetDataList.Add(Asset);
	GEditor->SyncBrowserToObjects(AssetDataList);

	return FReply::Handled();
}

FReply SDMToolBar::OnUseClicked()
{
	if (!ActorMaterialProperties.IsValidIndex(SelectedMaterialElementIndex))
	{
		return FReply::Handled();
	}

	UDynamicMaterialModelBase* CurrentModelBase = GetOriginalMaterialModelBase();
	UDMWorldSubsystem* DMSubsystem = nullptr;

	if (CurrentModelBase)
	{
		AActor* Actor = MaterialActorWeak.Get();

		if (!IsValid(Actor))
		{
			return FReply::Handled();
		}

		UWorld* World = Actor->GetWorld();

		if (!IsValid(World))
		{
			return FReply::Handled();
		}

		DMSubsystem = World->GetSubsystem<UDMWorldSubsystem>();

		if (!DMSubsystem)
		{
			return FReply::Handled();
		}
	}

	USelection* Selection = GEditor->GetSelectedObjects();

	if (!Selection)
	{
		return FReply::Handled();
	}

	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	UDynamicMaterialInstance* SelectedInstance = nullptr;

	TArray<UDynamicMaterialInstance*> SelectedInstances;
	Selection->GetSelectedObjects(SelectedInstances);

	for (UDynamicMaterialInstance* SelectedInstanceIter : SelectedInstances)
	{
		if (!IsValid(SelectedInstanceIter) || !SelectedInstanceIter->IsAsset())
		{
			continue;
		}

		SelectedInstance = SelectedInstanceIter;
		break;
	}

	if (!SelectedInstance)
	{
		return FReply::Handled();
	}

	FDMObjectMaterialProperty& CurrentActorProperty = *ActorMaterialProperties[SelectedMaterialElementIndex];

	if (!UDMMaterialInstanceFunctionLibrary::SetMaterialInObject(CurrentActorProperty, SelectedInstance))
	{
		return FReply::Handled();
	}

	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		if (TSharedPtr<SDMMaterialDesigner> DesignerWidget = EditorWidget->GetDesignerWidget())
		{
			DesignerWidget->OpenObjectMaterialProperty(CurrentActorProperty);
		}
	}

	return FReply::Handled();
}

FText SDMToolBar::GetActorName() const
{
	if (const AActor* const SlotActor = GetMaterialActor())
	{
		return FText::FromString(SlotActor->GetActorLabel());
	}

	return FText::GetEmpty();
}

TSharedPtr<SDMMaterialEditor> SDMToolBar::GetEditorWidget() const
{
	return EditorWidgetWeak.Pin();
}

FText SDMToolBar::GetAssetName() const
{
	if (UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase())
	{
		if (UDynamicMaterialInstance* MaterialInstance = MaterialModelBase->GetDynamicMaterialInstance())
		{
			if (MaterialInstance->IsAsset())
			{
				return FText::FromString(MaterialInstance->GetName());
			}
		}
		else if (MaterialModelBase->IsAsset())
		{
			return FText::FromString(MaterialModelBase->GetName());
		}
	}

	return FText::GetEmpty();
}

FText SDMToolBar::GetAssetToolTip() const
{
	if (UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase())
	{
		if (UDynamicMaterialInstance* MaterialInstance = MaterialModelBase->GetDynamicMaterialInstance())
		{
			if (MaterialInstance->IsAsset())
			{
				return FText::FromString(MaterialInstance->GetPathName());
			}
		}
		else if (MaterialModelBase->IsAsset())
		{
			return FText::FromString(MaterialModelBase->GetPathName());
		}
	}

	return FText::GetEmpty();
}

const FSlateBrush* SDMToolBar::GetSaveIcon() const
{
	if (UPackage* Package = SDMMaterialEditor::GetSaveablePackage(GetOriginalMaterialModelBase()))
	{
		if (Package->IsDirty())
		{
			return FAppStyle::Get().GetBrush("Icons.SaveModified");
		}
	}

	return FAppStyle::Get().GetBrush("Icons.Save");
}

FReply SDMToolBar::OnSaveClicked()
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		FScopedTransaction Transaction(LOCTEXT("SaveOriginalMaterial", "Save Original Material"));
		EditorWidget->SaveOriginal();
	}

	return FReply::Handled();
}

FReply SDMToolBar::OnOpenParentClicked()
{
	if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
	{
		if (TSharedPtr<SDMMaterialDesigner> DesignerWidget = EditorWidget->GetDesignerWidget())
		{
			if (UDynamicMaterialModelDynamic* DynamicMaterialModel = Cast<UDynamicMaterialModelDynamic>(GetOriginalMaterialModelBase()))
			{
				if (UDynamicMaterialModel* ParentModel = DynamicMaterialModel->ResolveMaterialModel())
				{
					DesignerWidget->OpenMaterialModelBase(ParentModel);
				}
			}
		}
	}

	return FReply::Handled();
}

FReply SDMToolBar::OnConvertToEditableClicked()
{
	UDynamicMaterialModelDynamic* CurrentModelDynamic = Cast<UDynamicMaterialModelDynamic>(GetOriginalMaterialModelBase());

	if (!CurrentModelDynamic)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Tried to convert a null or non-dynamic model to editable.");
		return FReply::Handled();
	}

	UDynamicMaterialModel* ParentModel = CurrentModelDynamic->GetParentModel();

	if (!ParentModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find parent model.");
		return FReply::Handled();
	}

	UDynamicMaterialInstance* OldInstance = CurrentModelDynamic->GetDynamicMaterialInstance();

	bool bIsAsset = false;

	if (CurrentModelDynamic->IsAsset())
	{
		bIsAsset = true;
	}
	else if (OldInstance)
	{
		if (OldInstance->IsAsset())
		{
			bIsAsset = true;
		}
	}

	UDMWorldSubsystem* DMSubsystem = nullptr;
	AActor* Actor = MaterialActorWeak.Get();
	TSharedPtr<FDMObjectMaterialProperty> CurrentActorProperty = nullptr;

	if (Actor && ActorMaterialProperties.IsValidIndex(SelectedMaterialElementIndex) && ActorMaterialProperties[SelectedMaterialElementIndex].IsValid())
	{
		UWorld* World = Actor->GetWorld();

		if (IsValid(World))
		{
			DMSubsystem = World->GetSubsystem<UDMWorldSubsystem>();
		}

		CurrentActorProperty = ActorMaterialProperties[SelectedMaterialElementIndex];
	}

	// In-actor models/instance must have a world subsystem to query.
	if (!bIsAsset && !DMSubsystem)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Cannot create a new asset for embedded instances without an active world subsystem.");
		return FReply::Handled();
	}

	UDynamicMaterialInstance* NewInstance = nullptr;
	UDynamicMaterialModel* NewModel = nullptr;

	if (OldInstance)
	{
		NewInstance = UDMMaterialModelFunctionLibrary::ExportToTemplateMaterial(CurrentModelDynamic);

		if (NewInstance)
		{
			NewModel = NewInstance->GetMaterialModel();
		}
	}
	else
	{
		NewModel = UDMMaterialModelFunctionLibrary::ExportToTemplateMaterialModel(CurrentModelDynamic);
	}

	if (!NewModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to create new model.");
		return FReply::Handled();
	}

	// If it was in an actor, set it on the actor
	if (NewInstance && CurrentActorProperty.IsValid())
	{
		// Setting it on the actor will automatically open it if the actor property is currently active.
		UDMMaterialInstanceFunctionLibrary::SetMaterialInObject(*CurrentActorProperty, NewInstance);
	}
	else
	{
		if (TSharedPtr<SDMMaterialEditor> EditorWidget = GetEditorWidget())
		{
			if (TSharedPtr<SDMMaterialDesigner> DesignerWidget = EditorWidget->GetDesignerWidget())
			{
				DesignerWidget->OpenMaterialModelBase(NewModel);
			}
		}
	}

	return FReply::Handled();
}

FReply SDMToolBar::OnConvertToInstanceClicked()
{
	UDynamicMaterialModel* CurrentModel = Cast<UDynamicMaterialModel>(GetOriginalMaterialModelBase());

	if (!CurrentModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Tried to convert a null or dynamic model to an instance.");
		return FReply::Handled();
	}

	UDynamicMaterialInstance* OldInstance = CurrentModel->GetDynamicMaterialInstance();

	if (!OldInstance)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Unable to find material to replace.");
		return FReply::Handled();
	}

	bool bIsAsset = false;

	if (CurrentModel->IsAsset())
	{
		bIsAsset = true;
	}
	else if (OldInstance)
	{
		if (OldInstance->IsAsset())
		{
			bIsAsset = true;
		}
	}

	if (!bIsAsset)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Tried to convert a non-asset material to an instance.");
		return FReply::Handled();
	}

	if (ActorMaterialProperties.IsEmpty() 
		|| !ActorMaterialProperties.IsValidIndex(SelectedMaterialElementIndex)
		|| !ActorMaterialProperties[SelectedMaterialElementIndex].IsValid()
		|| !GetMaterialActor())
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Unable to find actor or actor property to insert the new instance.");
		return FReply::Handled();
	}

	FDMObjectMaterialProperty& Property = *ActorMaterialProperties[SelectedMaterialElementIndex];

	UDynamicMaterialInstanceFactory* InstanceFactory = NewObject<UDynamicMaterialInstanceFactory>();
	check(InstanceFactory);

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(InstanceFactory->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(),
		Property.GetOuter(),
		NAME_None,
		RF_Transactional,
		nullptr,
		GWarn	
	));

	if (!NewInstance)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to create new material.");
		return FReply::Handled();
	}

	if (!UDMMaterialModelFunctionLibrary::CreateModelInstanceInMaterial(CurrentModel, NewInstance))
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to create new material model instance.");
		return FReply::Handled();
	}

	// Setting it on the actor will automatically open it if the actor property is currently active.
	if (!UDMMaterialInstanceFunctionLibrary::SetMaterialInObject(Property, NewInstance))
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to assign new material to actor.");
		return FReply::Handled();
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SDMToolBar::GenerateSettingsMenu()
{
	return FDMToolBarMenus::MakeEditorLayoutMenu(GetEditorWidget());
}

EVisibility SDMToolBar::GetAutoCompileVisibility() const
{
	return IsDynamicMaterialModel() || IsAutoCompileAndApplyOverridden()
		? EVisibility::Collapsed
		: EVisibility::Visible;
}

FSlateColor SDMToolBar::GetAutoCompileColor() const
{
	static FSlateColor EnabledColor = FSlateColor(EStyleColor::AccentBlue);
	static FSlateColor DisabledColor = FSlateColor(EStyleColor::AccentGray);

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->bAutomaticallyCompilePreviewMaterial)
		{
			return EnabledColor;
		}
	}

	return DisabledColor;
}

FReply SDMToolBar::OnAutoCompileClicked()
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		FScopedTransaction Transaction(LOCTEXT("ToggleAutoPreviewCompile", "Toggle Live Preview"));
		Settings->Modify();
		Settings->bAutomaticallyCompilePreviewMaterial = !Settings->bAutomaticallyCompilePreviewMaterial;
		Settings->SaveConfig();

		if (Settings->bAutomaticallyCompilePreviewMaterial)
		{
			if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
			{
				if (UDynamicMaterialModel* PreviewMaterialModel = Cast<UDynamicMaterialModel>(EditorWidget->GetPreviewMaterialModelBase()))
				{
					if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(PreviewMaterialModel))
					{
						if (EditorOnlyData->HasBuildBeenRequested())
						{
							EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Async);
						}
					}
				}
			}
		}
	}

	return FReply::Handled();
}

EVisibility SDMToolBar::GetAutoApplyVisibility() const
{
	return IsDynamicMaterialModel() || IsAutoCompileAndApplyOverridden()
		? EVisibility::Collapsed
		: EVisibility::Visible;
}

FSlateColor SDMToolBar::GetAutoApplyColor() const
{
	static FSlateColor EnabledColor = FSlateColor(EStyleColor::AccentBlue);
	static FSlateColor DisabledColor = FSlateColor(EStyleColor::AccentGray);

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->bAutomaticallyApplyToSourceOnPreviewCompile)
		{
			return EnabledColor;
		}
	}

	return DisabledColor;
}

FReply SDMToolBar::OnAutoApplyClicked()
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		FScopedTransaction Transaction(LOCTEXT("ToggleAutoApply", "Toggle Auto Apply to Source"));
		Settings->Modify();
		Settings->bAutomaticallyApplyToSourceOnPreviewCompile = !Settings->bAutomaticallyApplyToSourceOnPreviewCompile;
		Settings->SaveConfig();

		if (Settings->bAutomaticallyApplyToSourceOnPreviewCompile)
		{
			if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
			{
				EditorWidget->ApplyToOriginal();
			}
		}
	}

	return FReply::Handled();
}

EVisibility SDMToolBar::GetLiveEditVisibility() const
{
	return IsDynamicMaterialModel() || !IsAutoCompileAndApplyOverridden()
		? EVisibility::Collapsed
		: EVisibility::Visible;
}

FSlateColor SDMToolBar::GetLiveEditColor() const
{
	static FSlateColor EnabledColor = FSlateColor(EStyleColor::AccentBlue);
	static FSlateColor DisabledColor = FSlateColor(EStyleColor::AccentGray);

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->LiveEditMode == EDMLiveEditMode::LiveEditOn)
		{
			return EnabledColor;
		}
	}

	return DisabledColor;
}

FReply SDMToolBar::OnLiveEditClicked()
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		FScopedTransaction Transaction(LOCTEXT("ToggleLiveEdit", "Toggle Live Edit"));
		Settings->Modify();

		switch (Settings->LiveEditMode)
		{
			case EDMLiveEditMode::LiveEditOn:
				Settings->LiveEditMode = EDMLiveEditMode::LiveEditOff;
				break;

			case EDMLiveEditMode::LiveEditOff:
				Settings->LiveEditMode = EDMLiveEditMode::LiveEditOn;
				break;
		}

		Settings->SaveConfig();

		if (Settings->ShouldAutomaticallyApplyToSourceOnPreviewCompile())
		{
			if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
			{
				EditorWidget->ApplyToOriginal();
			}
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
