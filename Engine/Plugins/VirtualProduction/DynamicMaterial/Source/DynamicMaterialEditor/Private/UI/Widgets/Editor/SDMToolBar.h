// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class FText;
class SBox;
class SDMMaterialEditor;
class STextBlock;
class SWidget;
class UDynamicMaterialModelBase;
class UPackage;
struct FDMObjectMaterialProperty;
struct FSlateBrush;
struct FSlateColor;

namespace ESelectInfo
{
	enum Type : int;
}

/**
 * Material Designer ToolBar
 *
 * Displays the selected actor that the Material Designer is editing and allows for switching between slots for that actor.
 */
class SDMToolBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMToolBar) {}
	SLATE_END_ARGS()

	virtual ~SDMToolBar() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, const FDMObjectMaterialProperty& InObjectProperty);

	AActor* GetMaterialActor() const { return MaterialActorWeak.Get(); }

	FText GetActorName() const;

	TSharedPtr<SDMMaterialEditor> GetEditorWidget() const;

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<AActor> MaterialActorWeak;

	TArray<TSharedPtr<FDMObjectMaterialProperty>> ActorMaterialProperties;
	int32 SelectedMaterialElementIndex;

	TSharedPtr<SBox> PropertySelectorContainer;
	TSharedPtr<SWidget> SaveButtonWidget;
	TSharedPtr<SWidget> ActorRowWidget;
	TSharedPtr<SWidget> AssetRowWidget;
	TSharedPtr<STextBlock> ActorNameWidget;
	TSharedPtr<STextBlock> AssetNameWidget;
	TSharedPtr<STextBlock> InstanceWidget;
	TSharedPtr<SWidget> OpenParentButton;
	TSharedPtr<SWidget> ConvertToEditableButton;
	TSharedPtr<SWidget> ConvertToInstanceButton;

	bool IsDynamicMaterialModel() const;

	bool IsAutoCompileAndApplyOverridden() const;

	UDynamicMaterialModelBase* GetOriginalMaterialModelBase() const;

	TSharedRef<SWidget> CreateToolBarEntries();

	void SetActorPropertySelected(const FDMObjectMaterialProperty& InObjectProperty);

	int32 GetDuplicateIndex(const FDMObjectMaterialProperty& InObjectProperty) const;

	void UpdateButtonVisibilities();

	TSharedRef<SWidget> CreateToolBarButton(TAttribute<const FSlateBrush*> InImageBrush, const TAttribute<FText>& InTooltipText, FOnClicked InOnClicked);

	TSharedRef<SWidget> CreateSlotsComboBoxWidget();

	TSharedRef<SWidget> GenerateSelectedMaterialSlotRow(TSharedPtr<FDMObjectMaterialProperty> InSelectedSlot) const;
	FText GetSlotDisplayName(TWeakPtr<FDMObjectMaterialProperty> InSlotWeak) const;
	bool GetSlotEnabled(TWeakPtr<FDMObjectMaterialProperty> InSlotWeak) const;
	FText GetSelectedMaterialSlotName() const;
	void OnMaterialSlotChanged(TSharedPtr<FDMObjectMaterialProperty> InSelectedSlot, ESelectInfo::Type InSelectInfoType);

	const FSlateBrush* GetFollowSelectionBrush() const;
	FSlateColor GetFollowSelectionColor() const;
	FReply OnFollowSelectionButtonClicked();

	FReply OnExportMaterialInstanceButtonClicked();

	FReply OnBrowseClicked();

	FReply OnUseClicked();

	FText GetAssetName() const;

	FText GetAssetToolTip() const;

	const FSlateBrush* GetSaveIcon() const;

	FReply OnSaveClicked();

	FReply OnOpenParentClicked();

	FReply OnConvertToEditableClicked();

	FReply OnConvertToInstanceClicked();

	TSharedRef<SWidget> GenerateSettingsMenu();

	EVisibility GetAutoCompileVisibility() const;
	FSlateColor GetAutoCompileColor() const;
	FReply OnAutoCompileClicked();

	EVisibility GetAutoApplyVisibility() const;
	FSlateColor GetAutoApplyColor() const;
	FReply OnAutoApplyClicked();

	EVisibility GetLiveEditVisibility() const;
	FSlateColor GetLiveEditColor() const;
	FReply OnLiveEditClicked();
};
