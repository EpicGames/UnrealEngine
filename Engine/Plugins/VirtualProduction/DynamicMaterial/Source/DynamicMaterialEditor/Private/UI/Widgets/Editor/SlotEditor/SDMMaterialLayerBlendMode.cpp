// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SlotEditor/SDMMaterialLayerBlendMode.h"

#include "Components/DMMaterialLayer.h"
#include "Components/MaterialStageBlends/DMMSBAdd.h"
#include "Components/MaterialStageBlends/DMMSBColor.h"
#include "Components/MaterialStageBlends/DMMSBColorBurn.h"
#include "Components/MaterialStageBlends/DMMSBColorDodge.h"
#include "Components/MaterialStageBlends/DMMSBDarken.h"
#include "Components/MaterialStageBlends/DMMSBDarkenColor.h"
#include "Components/MaterialStageBlends/DMMSBDifference.h"
#include "Components/MaterialStageBlends/DMMSBDivide.h"
#include "Components/MaterialStageBlends/DMMSBExclusion.h"
#include "Components/MaterialStageBlends/DMMSBHardLight.h"
#include "Components/MaterialStageBlends/DMMSBHardMix.h"
#include "Components/MaterialStageBlends/DMMSBHue.h"
#include "Components/MaterialStageBlends/DMMSBLighten.h"
#include "Components/MaterialStageBlends/DMMSBLightenColor.h"
#include "Components/MaterialStageBlends/DMMSBLinearBurn.h"
#include "Components/MaterialStageBlends/DMMSBLinearDodge.h"
#include "Components/MaterialStageBlends/DMMSBLinearLight.h"
#include "Components/MaterialStageBlends/DMMSBLuminosity.h"
#include "Components/MaterialStageBlends/DMMSBMultiply.h"
#include "Components/MaterialStageBlends/DMMSBNormal.h"
#include "Components/MaterialStageBlends/DMMSBOverlay.h"
#include "Components/MaterialStageBlends/DMMSBPinLight.h"
#include "Components/MaterialStageBlends/DMMSBSaturation.h"
#include "Components/MaterialStageBlends/DMMSBScreen.h"
#include "Components/MaterialStageBlends/DMMSBSoftLight.h"
#include "Components/MaterialStageBlends/DMMSBSubtract.h"
#include "Components/MaterialStageBlends/DMMSBVividLight.h"
#include "DetailLayoutBuilder.h"
#include "DynamicMaterialEditorStyle.h"
#include "Model/DynamicMaterialModel.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SDMMaterialLayerBlendMode)

#define LOCTEXT_NAMESPACE "SDMMaterialLayerBlendMode"

namespace UE::DynamicMaterialEditor::Private
{
	static const FName SourceBlendMenuName = "SourceBlendMenu";

	struct FDMBlendCategory
	{
		FText Name;
		TArray<UClass*> Classes;
	};

	const TArray<FDMBlendCategory>& SupportedBlendCategories()
	{
		static TArray<FDMBlendCategory> OutBlendCategories;

		if (OutBlendCategories.IsEmpty() == false)
		{
			return OutBlendCategories;
		}

		OutBlendCategories.Append({
			{
				LOCTEXT("BlendNormal", "Normal Blends"),
				{
					UDMMaterialStageBlendNormal::StaticClass()
				}
			},
			{
				LOCTEXT("BlendDarken", "Darken Blends"),
				{
					UDMMaterialStageBlendDarken::StaticClass(),
					UDMMaterialStageBlendDarkenColor::StaticClass(),
					UDMMaterialStageBlendMultiply::StaticClass(),
					UDMMaterialStageBlendColorBurn::StaticClass(),
					UDMMaterialStageBlendLinearBurn::StaticClass()
				}
			},
			{
				LOCTEXT("BlendLighten", "Lighten Blends"),
				{
					UDMMaterialStageBlendLighten::StaticClass(),
					UDMMaterialStageBlendLightenColor::StaticClass(),
					UDMMaterialStageBlendScreen::StaticClass(),
					UDMMaterialStageBlendColorDodge::StaticClass(),
					UDMMaterialStageBlendLinearDodge::StaticClass()
				}
			},
			{
				LOCTEXT("BlendContrast", "Contrast Blends"),
				{
					UDMMaterialStageBlendOverlay::StaticClass(),
					UDMMaterialStageBlendSoftLight::StaticClass(),
					UDMMaterialStageBlendHardLight::StaticClass(),
					UDMMaterialStageBlendVividLight::StaticClass(),
					UDMMaterialStageBlendLinearLight::StaticClass(),
					UDMMaterialStageBlendPinLight::StaticClass(),
					UDMMaterialStageBlendHardMix::StaticClass()
				}
			},
			{
				LOCTEXT("BlendInversion", "Inversion Blends"),
				{
					UDMMaterialStageBlendDifference::StaticClass(),
					UDMMaterialStageBlendExclusion::StaticClass(),
					UDMMaterialStageBlendSubtract::StaticClass(),
					UDMMaterialStageBlendDivide::StaticClass()
				}
			},
			{
				LOCTEXT("BlendHSL", "HSL Blends"),
				{
					UDMMaterialStageBlendColor::StaticClass(),
					UDMMaterialStageBlendHue::StaticClass(),
					UDMMaterialStageBlendSaturation::StaticClass(),
					UDMMaterialStageBlendLuminosity::StaticClass()
				}
			}
		});

		return OutBlendCategories;
	};
}

TArray<TStrongObjectPtr<UClass>> SDMMaterialLayerBlendMode::SupportedBlendClasses = {};
TMap<FName, FDMBlendNameClass> SDMMaterialLayerBlendMode::BlendMap = {};

TSharedPtr<SDMMaterialLayerBlendMode> UDMSourceBlendModeContextObject::GetBlendModeWidget() const
{
	return BlendModeWidgetWeak.Pin();
}

void UDMSourceBlendModeContextObject::SetBlendModeWidget(const TSharedPtr<SDMMaterialLayerBlendMode>& InBlendModeWidget)
{
	BlendModeWidgetWeak = InBlendModeWidget;
}

void SDMMaterialLayerBlendMode::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerItem> InLayerItem)
{
	LayerItemWidgetWeak = InLayerItem;
	SelectedItem = InArgs._SelectedItem;

	SetCanTick(false);

	EnsureBlendMap();
	EnsureMenuRegistered();

	FName InitialSourceBlendModeName = NAME_None;

	if (TSubclassOf<UDMMaterialStageBlend> Blend = SelectedItem.Get())
	{
		InitialSourceBlendModeName = Blend->GetFName();
	}

	const FComboBoxStyle& ComboBoxStyle = FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox");
	const FComboButtonStyle& ComboButtonStyle = ComboBoxStyle.ComboButtonStyle;
	const FButtonStyle& ButtonStyle = ComboButtonStyle.ButtonStyle;
	
	ChildSlot
	[
		SNew(SComboButton)
		.ComboButtonStyle(&ComboButtonStyle)
		.ButtonStyle(&ButtonStyle)
		.ContentPadding(ComboBoxStyle.ContentPadding)
		.IsFocusable(true)
		.IsEnabled(this, &SDMMaterialLayerBlendMode::IsSelectorEnabled)
		.ForegroundColor(FSlateColor::UseStyle())
		.ToolTipText(LOCTEXT("SourceBlendMode", "Source Blend Mode"))
		.OnGetMenuContent(this, &SDMMaterialLayerBlendMode::MakeSourceBlendMenuWidget)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SDMMaterialLayerBlendMode::GetSelectedItemText)
		]
	];
}

void SDMMaterialLayerBlendMode::EnsureBlendMap()
{
	if (SupportedBlendClasses.IsEmpty() == false)
	{
		return;
	}

	SupportedBlendClasses = UDMMaterialStageBlend::GetAvailableBlends();

	for (const TStrongObjectPtr<UClass>& BlendClass : SupportedBlendClasses)
	{
		UDMMaterialStageBlend* StageBlendCDO = Cast<UDMMaterialStageBlend>(BlendClass->GetDefaultObject());

		if (ensure(IsValid(StageBlendCDO)))
		{
			const FName BlendClassName = BlendClass->GetFName();
			const FText BlendClassText = StageBlendCDO->GetDescription();
			const TSubclassOf<UDMMaterialStageBlend> BlendClassObject = TSubclassOf<UDMMaterialStageBlend>(BlendClass.Get());

			if (!BlendClassName.IsNone() && !BlendClassText.IsEmpty())
			{
				BlendMap.Emplace(BlendClassName, {BlendClassText, BlendClassObject});
			}
		}
	}
}

void SDMMaterialLayerBlendMode::EnsureMenuRegistered()
{
	using namespace UE::DynamicMaterialEditor::Private;

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (ToolMenus->IsMenuRegistered(SourceBlendMenuName))
	{
		return;
	}

	UToolMenu* NewMenu = ToolMenus->RegisterMenu(SourceBlendMenuName, NAME_None, EMultiBoxType::Menu, false);

	if (!IsValid(NewMenu))
	{
		return;
	}

	NewMenu->bToolBarForceSmallIcons = true;
	NewMenu->bShouldCloseWindowAfterMenuSelection = true;
	NewMenu->bCloseSelfOnly = true;

	NewMenu->AddDynamicSection("PopulateToolBar", FNewToolMenuDelegate::CreateStatic(&SDMMaterialLayerBlendMode::MakeSourceBlendMenu));
}

TSharedRef<SWidget> SDMMaterialLayerBlendMode::OnGenerateWidget(const FName InItem)
{
	EnsureBlendMap();

	if (!InItem.IsValid() || !BlendMap.Contains(InItem))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(STextBlock)
		.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
		.Text(BlendMap[InItem].BlendName);
}

FText SDMMaterialLayerBlendMode::GetSelectedItemText() const
{
	if (TSubclassOf<UDMMaterialStageBlend> Blend = SelectedItem.Get())
	{
		const FName SelectedName = Blend->GetFName();

		if (BlendMap.Contains(SelectedName))
		{
			return BlendMap[SelectedName].BlendName;
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SDMMaterialLayerBlendMode::MakeSourceBlendMenuWidget()
{
	using namespace UE::DynamicMaterialEditor::Private;

	UDMSourceBlendModeContextObject* Context = NewObject<UDMSourceBlendModeContextObject>();
	Context->SetBlendModeWidget(SharedThis(this));

	return UToolMenus::Get()->GenerateWidget(SourceBlendMenuName, FToolMenuContext(Context));
}

void SDMMaterialLayerBlendMode::OnBlendModeSelected(UClass* InBlendClass)
{
	SelectedItem = InBlendClass;

	TSharedPtr<SDMMaterialSlotLayerItem> LayerItemWiget = LayerItemWidgetWeak.Pin();

	if (!LayerItemWiget.IsValid())
	{
		return;
	}

	UDMMaterialLayerObject* Layer = LayerItemWiget->GetLayer();

	if (!Layer)
	{
		return;
	}

	// We only want to change if the stage is enabled.
	UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base, /* Check Enabled */ true);

	if (!BaseStage)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("SetStageBlendMode", "Set Blend Mode"));
	BaseStage->Modify();
	BaseStage->ChangeSource<UDMMaterialStageBlend>(InBlendClass);

	if (TSharedPtr<SDMMaterialSlotLayerItem> LayerItem = LayerItemWidgetWeak.Pin())
	{
		if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = LayerItem->GetSlotLayerView())
		{
			if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = LayerView->GetSlotEditorWidget())
			{
				SlotEditorWidget->InvalidateSlotSettings();
			}
		}
	}
}

bool SDMMaterialLayerBlendMode::CanSelectBlendMode(UClass* InBlendClass)
{
	return SelectedItem.Get() != InBlendClass;
}

bool SDMMaterialLayerBlendMode::InBlendModeSelected(UClass* InBlendClass)
{
	return SelectedItem.Get() == InBlendClass;
}

bool SDMMaterialLayerBlendMode::IsSelectorEnabled() const
{
	TSharedPtr<SDMMaterialSlotLayerItem> LayerItemWidget = LayerItemWidgetWeak.Pin();

	if (!LayerItemWidget.IsValid())
	{
		return false;
	}

	UDMMaterialLayerObject* Layer = LayerItemWidget->GetLayer();

	if (!Layer)
	{
		return false;
	}

	// We only want to change if the stage is enabled.
	if (!Layer->GetStage(EDMMaterialLayerStage::Base, /* Check Enabled */ true))
	{
		return false;
	}

	if (TSharedPtr<SDMMaterialSlotLayerView> SlotLayerView = LayerItemWidget->GetSlotLayerView())
	{
		if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = SlotLayerView->GetSlotEditorWidget())
		{
			if (TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget())
			{
				if (UDynamicMaterialModelBase* ModelBase = EditorWidget->GetOriginalMaterialModelBase())
				{
					if (ModelBase->IsA<UDynamicMaterialModel>())
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void SDMMaterialLayerBlendMode::MakeSourceBlendMenu(UToolMenu* InToolMenu)
{
	using namespace UE::DynamicMaterialEditor::Private;

	static const TArray<FDMBlendCategory> BlendCategories = SupportedBlendCategories();

	if (SupportedBlendClasses.IsEmpty())
	{
		return;
	}

	UDMSourceBlendModeContextObject* ContextObject = InToolMenu->FindContext<UDMSourceBlendModeContextObject>();

	if (!ContextObject)
	{
		return;
	}

	TSharedPtr<SDMMaterialLayerBlendMode> SourceBlendWidget = ContextObject->GetBlendModeWidget();

	if (!SourceBlendWidget.IsValid())
	{
		return;
	}

	auto AddClassToSection = [&SourceBlendWidget](FToolMenuSection& InSection, const FText& InName, UClass* InBlendClass)
	{
		if (!InBlendClass)
		{
			return;
		}

		if (InBlendClass->GetClassFlags() & EClassFlags::CLASS_Deprecated)
		{
			return;
		}

		UDMMaterialStageBlend* BlendCDO = InBlendClass->GetDefaultObject<UDMMaterialStageBlend>();

		if (!BlendCDO)
		{
			return;
		}

		InSection.AddMenuEntry(
			NAME_None,
			InName,
			BlendCDO->GetBlendDescription(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(SourceBlendWidget.ToSharedRef(), &SDMMaterialLayerBlendMode::OnBlendModeSelected, InBlendClass),
				FCanExecuteAction::CreateSP(SourceBlendWidget.ToSharedRef(), &SDMMaterialLayerBlendMode::CanSelectBlendMode, InBlendClass),
				FIsActionChecked::CreateSP(SourceBlendWidget.ToSharedRef(), &SDMMaterialLayerBlendMode::InBlendModeSelected, InBlendClass),
				EUIActionRepeatMode::RepeatDisabled
			)
		);
	};

	for (const FDMBlendCategory& BlendCategory : BlendCategories)
	{
		FToolMenuSection& BlendCategorySection = InToolMenu->AddSection(FName(BlendCategory.Name.ToString()), BlendCategory.Name);

		for (UClass* BlendCategoryClass : BlendCategory.Classes)
		{
			if (UDMMaterialStageBlend* BlendCDO = Cast<UDMMaterialStageBlend>(BlendCategoryClass->GetDefaultObject()))
			{
				AddClassToSection(BlendCategorySection, BlendCDO->GetDescription(), BlendCategoryClass);
			}
		}
	}

	FToolMenuSection& UncategorizedSection = InToolMenu->AddSection(TEXT("OtherBlends"), LOCTEXT("OtherBlends", "Other Blends"));

	for (TStrongObjectPtr<UClass> BlendClass : SupportedBlendClasses)
	{
		UDMMaterialStageBlend* BlendCDO = Cast<UDMMaterialStageBlend>(BlendClass->GetDefaultObject());

		if (!BlendCDO)
		{
			continue;
		}

		bool bInBlendCategories = false;

		for (const FDMBlendCategory& BlendCategory : BlendCategories)
		{
			for (UClass* BlendCategoryClass : BlendCategory.Classes)
			{
				if (BlendClass.Get() == BlendCategoryClass)
				{
					bInBlendCategories = true;
					break;
				}
			}
		}

		if (bInBlendCategories)
		{
			continue;
		}

		AddClassToSection(UncategorizedSection, BlendCDO->GetDescription(), BlendClass.Get());
	}
}

#undef LOCTEXT_NAMESPACE
