// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Widgets/SCompoundWidget.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"

#include "SDMMaterialLayerBlendMode.generated.h"

class SDMMaterialLayerBlendMode;
class SDMMaterialSlotLayerItem;
class SWidget;
class UClass;
class UDMMaterialStageBlend;
class UToolMenu;

UCLASS(MinimalAPI)
class UDMSourceBlendModeContextObject : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<SDMMaterialLayerBlendMode> GetBlendModeWidget() const;

	void SetBlendModeWidget(const TSharedPtr<SDMMaterialLayerBlendMode>& InBlendModeWidget);

private:
	TWeakPtr<SDMMaterialLayerBlendMode> BlendModeWidgetWeak;
};

struct FDMBlendNameClass
{
	FText BlendName;
	TSubclassOf<UDMMaterialStageBlend> BlendClass;
};

class SDMMaterialLayerBlendMode : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMMaterialLayerBlendMode)
		{}
		SLATE_ATTRIBUTE(TSubclassOf<UDMMaterialStageBlend>, SelectedItem)
	SLATE_END_ARGS()

	virtual ~SDMMaterialLayerBlendMode() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerItem> InLayerItem);

protected:
	static TArray<TStrongObjectPtr<UClass>> SupportedBlendClasses;
	static TMap<FName, FDMBlendNameClass> BlendMap;

	static void EnsureBlendMap();
	static void EnsureMenuRegistered();
	static void MakeSourceBlendMenu(UToolMenu* InToolMenu);

	TWeakPtr<SDMMaterialSlotLayerItem> LayerItemWidgetWeak;
	TAttribute<TSubclassOf<UDMMaterialStageBlend>> SelectedItem;

	TSharedRef<SWidget> OnGenerateWidget(const FName InItem);

	FText GetSelectedItemText() const;

	TSharedRef<SWidget> MakeSourceBlendMenuWidget();

	void OnBlendModeSelected(UClass* InBlendClass);
	bool CanSelectBlendMode(UClass* InBlendClass);
	bool InBlendModeSelected(UClass* InBlendClass);

	bool IsSelectorEnabled() const;
};
