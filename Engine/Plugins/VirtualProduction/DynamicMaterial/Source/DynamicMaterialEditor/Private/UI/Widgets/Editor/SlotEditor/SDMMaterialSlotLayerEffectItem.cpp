// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectItem.h"

#include "Components/DMMaterialEffectStack.h"
#include "DynamicMaterialEditorStyle.h"
#include "Editor.h"
#include "Model/DynamicMaterialModel.h"
#include "UI/DragDrop/DMLayerEffectsDragDropOperation.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectView.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SDMMaterialSlotLayerEffectItem"

void SDMMaterialSlotLayerEffectItem::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SDMMaterialSlotLayerEffectItem::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerEffectView>& InEffectView, 
	UDMMaterialEffect* InMaterialEffect)
{
	EffectViewWeak = InEffectView;
	EffectWeak = InMaterialEffect;

	STableRow<UDMMaterialEffect*>::Construct(
		STableRow<UDMMaterialEffect*>::FArguments()
		.Padding(2.0f)
		.ShowSelection(true)
		.ToolTipText(GetToolTipText())
		.Style(FDynamicMaterialEditorStyle::Get(), "EffectsView.Row")
		.OnPaintDropIndicator(this, &SDMMaterialSlotLayerEffectItem::OnEffectItemPaintDropIndicator)
		.OnCanAcceptDrop(this, &SDMMaterialSlotLayerEffectItem::OnEffectItemCanAcceptDrop)
		.OnDragDetected(this, &SDMMaterialSlotLayerEffectItem::OnEffectItemDragDetected)
		.OnAcceptDrop(this, &SDMMaterialSlotLayerEffectItem::OnEffectItemAcceptDrop)
		, InEffectView
	);

	SetContent(CreateMainContent());

	SetCursor(EMouseCursor::GrabHand);
}

TSharedPtr<SDMMaterialSlotLayerEffectView> SDMMaterialSlotLayerEffectItem::GetEffectView() const
{
	return EffectViewWeak.Pin();
}

UDMMaterialEffect* SDMMaterialSlotLayerEffectItem::GetMaterialEffect() const
{
	return EffectWeak.Get();
}

FCursorReply SDMMaterialSlotLayerEffectItem::OnCursorQuery(const FGeometry& InMyGeometry, const FPointerEvent& InCursorEvent) const
{
	if (InCursorEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	}

	return STableRow<UDMMaterialEffect*>::OnCursorQuery(InMyGeometry, InCursorEvent);
}

FReply SDMMaterialSlotLayerEffectItem::OnLayerBypassButtonClick()
{
	if (UDMMaterialEffect* MaterialEffect = GetMaterialEffect())
	{
		FDMScopedUITransaction Transaction(LOCTEXT("ToggleEffectEnabled", "Toggle Effect"));
		MaterialEffect->Modify();
		MaterialEffect->SetEnabled(!MaterialEffect->IsEnabled());
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SDMMaterialSlotLayerEffectItem::CreateMainContent()
{
	constexpr float HorizontalSpacing = 2.f;
	constexpr float HorizontalSpacingEnd = 3.f;
	constexpr float VerticalSpacing = 2.f;

	return SNew(SHorizontalBox)
		.ToolTipText(GetToolTipText())
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(HorizontalSpacingEnd, VerticalSpacing, HorizontalSpacingEnd, VerticalSpacing)
		[
			CreateLayerBypassButton()
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(HorizontalSpacing, VerticalSpacing, HorizontalSpacing, VerticalSpacing)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "SmallFont")
			.Text(GetLayerHeaderText())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(HorizontalSpacing, VerticalSpacing, HorizontalSpacing, VerticalSpacing)
		[
			CreateBrowseToEffectButton()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(HorizontalSpacingEnd, VerticalSpacing, HorizontalSpacingEnd, VerticalSpacing)
		[
			CreateLayerRemoveButton()
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerEffectItem::CreateLayerBypassButton()
{
	return SNew(SButton)
		.ContentPadding(FMargin(2.0f, 2.f))
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("LayerBypassTooltip", "Toggle the bypassing of this layer."))
		.Cursor(EMouseCursor::Default)
		.IsEnabled(CanModifyMaterialModel())
		.OnClicked(this, &SDMMaterialSlotLayerEffectItem::OnLayerBypassButtonClick)
		[
			SNew(SImage)
			.Image(this, &SDMMaterialSlotLayerEffectItem::GetLayerBypassButtonImage)
			.DesiredSizeOverride(FVector2D(12.0f))
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerEffectItem::CreateLayerRemoveButton()
{
	return SNew(SButton)
		.ContentPadding(FMargin(2.0f, 2.f))
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("RemoveEffectTooltip", "Remove Effect"))
		.Cursor(EMouseCursor::Default)
		.IsEnabled(CanModifyMaterialModel())
		.OnClicked(this, &SDMMaterialSlotLayerEffectItem::OnLayerRemoveButtonClick)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
			.DesiredSizeOverride(FVector2D(12.0f))
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerEffectItem::CreateBrowseToEffectButton()
{
	return SNew(SButton)
		.Visibility(!!GetEffectAsset() ? EVisibility::Visible : EVisibility::Collapsed)
		.ContentPadding(FMargin(2.0f, 2.f))
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("BrowseToEffectTooltip", "Browse to Effect in Content Browser"))
		.Cursor(EMouseCursor::Default)
		.OnClicked(this, &SDMMaterialSlotLayerEffectItem::OnBrowseToEffectButtonClick)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("SystemWideCommands.FindInContentBrowser"))
			.DesiredSizeOverride(FVector2D(12.0f))
		];
}

FText SDMMaterialSlotLayerEffectItem::GetToolTipText() const
{
	if (UDMMaterialEffect* MaterialEffect = GetMaterialEffect())
	{
		return MaterialEffect->GetEffectDescription();
	}

	return FText::GetEmpty();
}

FText SDMMaterialSlotLayerEffectItem::GetLayerHeaderText() const
{
	if (UDMMaterialEffect* MaterialEffect = GetMaterialEffect())
	{
		return MaterialEffect->GetEffectName();
	}

	return FText::GetEmpty();
}

bool SDMMaterialSlotLayerEffectItem::CanModifyMaterialModel() const
{
	if (TSharedPtr<SDMMaterialSlotLayerEffectView> EffectView = GetEffectView())
	{
		if (TSharedPtr<SDMMaterialSlotLayerItem> LayerItem = EffectView->GetLayerItem())
		{
			if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = LayerItem->GetSlotLayerView())
			{
				if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = LayerView->GetSlotEditorWidget())
				{
					if (TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget())
					{
						if (UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase())
						{
							return PreviewMaterialModelBase->IsA<UDynamicMaterialModel>();
						}
					}
				}
			}
		}
	}

	return false;
}

const FSlateBrush* SDMMaterialSlotLayerEffectItem::GetLayerBypassButtonImage() const
{
	static const FSlateIcon VisibleIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Visible");
	static const FSlateIcon InvisibleIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Hidden");

	if (UDMMaterialEffect* MaterialEffect = GetMaterialEffect())
	{
		if (MaterialEffect->IsEnabled())
		{
			return VisibleIcon.GetIcon();
		}
	}

	return InvisibleIcon.GetIcon();
}

FReply SDMMaterialSlotLayerEffectItem::OnLayerRemoveButtonClick()
{
	if (UDMMaterialEffect* MaterialEffect = GetMaterialEffect())
	{
		if (UDMMaterialEffectStack* EffectStack = MaterialEffect->GetEffectStack())
		{
			FDMScopedUITransaction Transaction(LOCTEXT("RemoveEffect", "Remove Effect"));
			EffectStack->Modify();
			EffectStack->RemoveEffect(MaterialEffect);
		}
	}

	return FReply::Handled();
}

FReply SDMMaterialSlotLayerEffectItem::OnBrowseToEffectButtonClick()
{
	if (UObject* Asset = GetEffectAsset())
	{
		TArray<UObject*> Assets = {Asset};
		GEditor->SyncBrowserToObjects(Assets); // Already validated in GetEffectAsset
	}

	return FReply::Handled();
}

int32 SDMMaterialSlotLayerEffectItem::OnEffectItemPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& InArgs,
	const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, 
	const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(InItemDropZone);
	static float OffsetX = 10.0f;
	FVector2D Offset(OffsetX * GetIndentLevel(), 0.0f);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId++,
		InAllottedGeometry.ToPaintGeometry(
			FVector2D(InAllottedGeometry.GetLocalSize() - Offset),
			FSlateLayoutTransform(Offset)
		),
		DropIndicatorBrush,
		ESlateDrawEffect::None,
		DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return InLayerId;
}

UObject* SDMMaterialSlotLayerEffectItem::GetEffectAsset() const
{
	if (GEditor)
	{
		if (UDMMaterialEffect* Effect = GetMaterialEffect())
		{
			if (UObject* Asset = Effect->GetAsset())
			{
				if (Asset->IsAsset())
				{
					return Asset;
				}
			}
		}
	}

	return nullptr;
}

TOptional<EItemDropZone> SDMMaterialSlotLayerEffectItem::OnEffectItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent, 
	EItemDropZone InDropZone, UDMMaterialEffect* InMaterialEffect) const
{
	if (IsValid(InMaterialEffect))
	{
		TSharedPtr<FDMLayerEffectsDragDropOperation> DragDropOperation = InDragDropEvent.GetOperationAs<FDMLayerEffectsDragDropOperation>();

		if (DragDropOperation.IsValid())
		{
			return InDropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SDMMaterialSlotLayerEffectItem::OnEffectItemDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const bool bShouldDuplicate = InMouseEvent.IsAltDown();

	TSharedRef<FDMLayerEffectsDragDropOperation> DragDropOperation = MakeShared<FDMLayerEffectsDragDropOperation>(SharedThis(this), bShouldDuplicate);

	return FReply::Handled().BeginDragDrop(DragDropOperation);
}

FReply SDMMaterialSlotLayerEffectItem::OnEffectItemAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, 
	UDMMaterialEffect* InMaterialEffect)
{
	if (!IsValid(InMaterialEffect))
	{
		return FReply::Handled();
	}

	UDMMaterialEffectStack* MaterialEffectStack = InMaterialEffect->GetEffectStack();

	if (!MaterialEffectStack)
	{
		return FReply::Handled();
	}

	TSharedPtr<FDMLayerEffectsDragDropOperation> DragDropOperation = InDragDropEvent.GetOperationAs<FDMLayerEffectsDragDropOperation>();

	if (!DragDropOperation.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<SDMMaterialSlotLayerEffectItem> DraggedWidget = DragDropOperation->GetLayerItemWidget();

	if (!DraggedWidget.IsValid())
	{
		return FReply::Handled();
	}

	UDMMaterialEffect* DraggedMaterialEffect = DraggedWidget->GetMaterialEffect();

	if (!IsValid(DraggedMaterialEffect) || DraggedMaterialEffect == InMaterialEffect)
	{
		return FReply::Handled();
	}

	UDMMaterialEffectStack* DraggedMaterialEffectStack = DraggedMaterialEffect->GetEffectStack();

	if (!IsValid(DraggedMaterialEffectStack) || DraggedMaterialEffectStack != MaterialEffectStack)
	{
		return FReply::Handled();
	}

	const int32 MaterialEffectIndex = InMaterialEffect->FindIndex();

	if (MaterialEffectIndex == INDEX_NONE)
	{
		return FReply::Handled();
	}

	const int32 DraggedMaterialEffectIndex = DraggedMaterialEffect->FindIndex();

	if (DraggedMaterialEffectIndex == INDEX_NONE)
	{
		return FReply::Handled();
	}

	FDMScopedUITransaction Transaction(LOCTEXT("MoveEffect", "Move Effect"));

	if (InDropZone == EItemDropZone::AboveItem)
	{
		MaterialEffectStack->MoveEffect(DraggedMaterialEffect, MaterialEffectIndex);
	}
	else
	{
		MaterialEffectStack->MoveEffect(DraggedMaterialEffect, MaterialEffectIndex + 1);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
