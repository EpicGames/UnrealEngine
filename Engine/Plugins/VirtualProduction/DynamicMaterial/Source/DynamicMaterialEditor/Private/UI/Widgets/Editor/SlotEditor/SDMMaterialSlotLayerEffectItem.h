// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"

#include "Components/DMMaterialEffect.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Views/SListView.h"

class SDMMaterialSlotLayerEffectView;
class UDMMaterialEffect;

class SDMMaterialSlotLayerEffectItem : public STableRow<UDMMaterialEffect*>
{
	SLATE_DECLARE_WIDGET(SDMMaterialSlotLayerEffectItem, STableRow<UDMMaterialEffect*>)

	SLATE_BEGIN_ARGS(SDMMaterialSlotLayerEffectItem) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialSlotLayerEffectItem() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerEffectView>& InEffectView, UDMMaterialEffect* InMaterialEffect);

	TSharedPtr<SDMMaterialSlotLayerEffectView> GetEffectView() const;

	UDMMaterialEffect* GetMaterialEffect() const;

protected:
	TWeakPtr<SDMMaterialSlotLayerEffectView> EffectViewWeak;
	TWeakObjectPtr<UDMMaterialEffect> EffectWeak;

	int32 OnEffectItemPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry,
		const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, 
		bool bInParentEnabled) const;

	UObject* GetEffectAsset() const;

	/** Slots */
	TSharedRef<SWidget> CreateMainContent();

	TSharedRef<SWidget> CreateLayerBypassButton();

	TSharedRef<SWidget> CreateLayerRemoveButton();

	TSharedRef<SWidget> CreateBrowseToEffectButton();

	FText GetToolTipText() const;

	FText GetLayerHeaderText() const;

	bool CanModifyMaterialModel() const;

	const FSlateBrush* GetLayerBypassButtonImage() const;

	FReply OnLayerRemoveButtonClick();

	FReply OnBrowseToEffectButtonClick();

	/** Drag and Drop */
	TOptional<EItemDropZone> OnEffectItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone,
		UDMMaterialEffect* InMaterialEffect) const;

	FReply OnEffectItemDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	FReply OnEffectItemAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone,
		UDMMaterialEffect* InMaterialEffect);

	/** Events */
	FCursorReply OnCursorQuery(const FGeometry& InMyGeometry, const FPointerEvent& InCursorEvent) const override;

	FReply OnLayerBypassButtonClick();
};
