// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectView.h"

#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "DynamicMaterialModule.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectItem.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialSlotLayerEffectView"

void SDMMaterialSlotLayerEffectView::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialSlotLayerEffectView::~SDMMaterialSlotLayerEffectView()
{
	if (TSharedPtr<SDMMaterialSlotLayerItem> LayerItem = LayerItemWeak.Pin())
	{
		if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = LayerItem->GetSlotLayerView())
		{
			if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = LayerView->GetSlotEditorWidget())
			{
				if (TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget())
				{
					EditorWidget->GetOnEditedComponentChanged().AddSP(this, &SDMMaterialSlotLayerEffectView::OnEditedComponentChanged);
				}
			}
		}
	}

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (TSharedPtr<SDMMaterialSlotLayerItem> LayerItem = LayerItemWeak.Pin())
	{
		if (UDMMaterialLayerObject* Layer = LayerItem->GetLayer())
		{
			if (UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack())
			{
				EffectStack->GetOnUpdate().RemoveAll(this);
			}
		}
	}
}

void SDMMaterialSlotLayerEffectView::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItem)
{
	LayerItemWeak = InLayerItem;

	SListView::Construct(SListView::FArguments()
		.ListItemsSource(&Effects)
		.SelectionMode(ESelectionMode::Single)
		.ClearSelectionOnClick(false)
		.EnableAnimatedScrolling(false)
		.ScrollbarVisibility(EVisibility::Collapsed)
		.ConsumeMouseWheel(EConsumeMouseWheel::Never)
		.OnGenerateRow(this, &SDMMaterialSlotLayerEffectView::OnGenerateEffectItemWidget)
		.OnSelectionChanged(this, &SDMMaterialSlotLayerEffectView::OnEffectItemSelectionChanged)
		.OnContextMenuOpening(this, &SDMMaterialSlotLayerEffectView::CreateEffectItemContextMenu)
	);

	RegenerateItems();
	RequestListRefresh();

	if (UDMMaterialLayerObject* Layer = InLayerItem->GetLayer())
	{
		if (UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack())
		{
			EffectStack->GetOnUpdate().AddSP(this, &SDMMaterialSlotLayerEffectView::OnEffectStackUpdate);
		}
	}

	if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = InLayerItem->GetSlotLayerView())
	{
		if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = LayerView->GetSlotEditorWidget())
		{
			if (TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget())
			{
				EditorWidget->GetOnEditedComponentChanged().AddSP(this, &SDMMaterialSlotLayerEffectView::OnEditedComponentChanged);
			}
		}
	}
}

TSharedPtr<SDMMaterialSlotLayerItem> SDMMaterialSlotLayerEffectView::GetLayerItem() const
{
	return LayerItemWeak.Pin();
}

UDMMaterialEffect* SDMMaterialSlotLayerEffectView::GetSelectedEffect() const
{
	TArray<UDMMaterialEffect*> SelectedEffects = GetSelectedItems();

	if (!SelectedEffects.IsEmpty())
	{
		return SelectedEffects[0];
	}

	return nullptr;
}

void SDMMaterialSlotLayerEffectView::SetSelectedEffect(UDMMaterialEffect* InEffect)
{
	ClearSelection();
	SetItemSelection(InEffect, /* Selected */ true);
}

TSharedPtr<SDMMaterialSlotLayerEffectItem> SDMMaterialSlotLayerEffectView::GetWidgetForEffect(UDMMaterialEffect* InEffect) const
{
	return StaticCastSharedPtr<SDMMaterialSlotLayerEffectItem>(WidgetFromItem(InEffect));
}

void SDMMaterialSlotLayerEffectView::PostUndo(bool bSuccess)
{
	OnUndo();
}

void SDMMaterialSlotLayerEffectView::PostRedo(bool bSuccess)
{
	OnUndo();
}

void SDMMaterialSlotLayerEffectView::RegenerateItems()
{
	TSharedPtr<SDMMaterialSlotLayerItem> LayerItem = LayerItemWeak.Pin();

	if (!LayerItem.IsValid())
	{
		return;
	}

	UDMMaterialLayerObject* Layer = LayerItem->GetLayer();

	if (!Layer)
	{
		return;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack)
	{
		return;
	}

	Effects = EffectStack->GetEffects();
}

TSharedRef<ITableRow> SDMMaterialSlotLayerEffectView::OnGenerateEffectItemWidget(UDMMaterialEffect* InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	TSharedRef<SDMMaterialSlotLayerEffectItem> LayerItem = SNew(
		SDMMaterialSlotLayerEffectItem,
		StaticCastSharedRef<SDMMaterialSlotLayerEffectView>(InOwnerTable),
		InItem
	);

	return LayerItem;
}

void SDMMaterialSlotLayerEffectView::OnEffectItemSelectionChanged(UDMMaterialEffect* InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	if (!IsValid(InSelectedItem))
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerItem> LayerItem = LayerItemWeak.Pin();

	if (!LayerItem.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerView> LayerView = LayerItem->GetSlotLayerView();

	if (!LayerView.IsValid())
	{
		return;
	}

	UDMMaterialLayerObject* Layer = LayerItem->GetLayer();

	if (!Layer)
	{
		return;
	}

	UDMMaterialEffectStack* MaterialEffectStack = InSelectedItem->GetEffectStack();

	if (!MaterialEffectStack)
	{
		return;
	}

	if (Layer != MaterialEffectStack->GetLayer())
	{
		return;
	}

	LayerView->SetSelectedLayer(Layer);

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = LayerView->GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return;
	}

	SlotEditorWidget->TriggerEffectSelectionChange(SharedThis(this), InSelectedItem);
}

void SDMMaterialSlotLayerEffectView::OnUndo()
{
	RegenerateItems();
	RequestListRefresh();
}

void SDMMaterialSlotLayerEffectView::OnEffectStackUpdate(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		RegenerateItems();
		RequestListRefresh();
	}
}

TSharedPtr<SWidget> SDMMaterialSlotLayerEffectView::CreateEffectItemContextMenu()
{
	return SNullWidget::NullWidget;
}

void SDMMaterialSlotLayerEffectView::OnEditedComponentChanged(const TSharedRef<SDMMaterialComponentEditor>& InComponentEditor, 
	UDMMaterialComponent* InComponent)
{
	UDMMaterialEffect* SelectedEffect = GetSelectedEffect();
	UDMMaterialEffect* MaterialEffect = Cast<UDMMaterialEffect>(InComponent);

	if (SelectedEffect == MaterialEffect)
	{
		return;
	}

	SetSelectedEffect(MaterialEffect);
}

#undef LOCTEXT_NAMESPACE
