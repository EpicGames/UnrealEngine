// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/Views/SListView.h"

#include "Components/DMMaterialEffect.h"
#include "Templates/SharedPointer.h"

class SDMMaterialComponentEditor;
class SDMMaterialSlotLayerEffectItem;
class SDMMaterialSlotLayerItem;
class UDMMaterialComponent;
enum class EDMUpdateType : uint8;

class SDMMaterialSlotLayerEffectView : public SListView<UDMMaterialEffect*>, public FSelfRegisteringEditorUndoClient
{
	SLATE_DECLARE_WIDGET(SDMMaterialSlotLayerEffectView, SListView<UDMMaterialEffect*>)

	SLATE_BEGIN_ARGS(SDMMaterialSlotLayerEffectView) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialSlotLayerEffectView() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItem);

	TSharedPtr<SDMMaterialSlotLayerItem> GetLayerItem() const;

	UDMMaterialEffect* GetSelectedEffect() const;

	void SetSelectedEffect(UDMMaterialEffect* InEffect);

	TSharedPtr<SDMMaterialSlotLayerEffectItem> GetWidgetForEffect(UDMMaterialEffect* InEffect) const;

	//~ Begin FUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FUndoClient

protected:
	TWeakPtr<SDMMaterialSlotLayerItem> LayerItemWeak;
	TArray<UDMMaterialEffect*> Effects;

	/** List */
	void RegenerateItems();

	TSharedRef<ITableRow> OnGenerateEffectItemWidget(UDMMaterialEffect* InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	void OnEffectItemSelectionChanged(UDMMaterialEffect* InSelectedItem, ESelectInfo::Type InSelectInfo);

	/** Events */
	void OnUndo();

	void OnEffectStackUpdate(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	TSharedPtr<SWidget> CreateEffectItemContextMenu();

	void OnEditedComponentChanged(const TSharedRef<SDMMaterialComponentEditor>& InComponentEditor, UDMMaterialComponent* InComponent);
};
