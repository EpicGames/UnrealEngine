// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"

#include "Components/DMMaterialEffectStack.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStageThroughput.h"
#include "Components/MaterialStageInputs/DMMSITextureUV.h"
#include "ContentBrowserDataDragDropOp.h"
#include "DetailLayoutBuilder.h"
#include "DynamicMaterialEditorStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Styling/StyleColors.h"
#include "UI/DragDrop/DMSlotLayerDragDropOperation.h"
#include "UI/Menus/DMMaterialStageSourceMenus.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialLayerBlendMode.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerEffectView.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialStage.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SDMMaterialSlotLayerItem"

namespace UE::DynamicMaterialEditor::Private
{
	static const FVector2D StagePreviewImageSize = FVector2D(30.f);
}

const FLazyName SDMMaterialSlotLayerItem::EffectViewName = TEXT("EffectView");

void SDMMaterialSlotLayerItem::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SDMMaterialSlotLayerItem::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerView>& InLayerView,
	const TSharedPtr<FDMMaterialLayerReference>& InLayerReferenceItem)
{
	LayerViewWeak = InLayerView;
	LayerItem = InLayerReferenceItem;

	bIsDynamic = false;

	if (TSharedPtr<SDMMaterialSlotEditor> SlotEditor = InLayerView->GetSlotEditorWidget())
	{
		if (TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditor->GetEditorWidget())
		{
			bIsDynamic = EditorWidget->IsDynamicModel();
		}
	}

	STableRow<TSharedPtr<FDMMaterialLayerReference>>::Construct(
		STableRow<TSharedPtr<FDMMaterialLayerReference>>::FArguments()
		.Padding(2.0f)
		.ShowSelection(true)
		.ToolTipText(GetToolTipText())
		.Style(FDynamicMaterialEditorStyle::Get(), "LayerView.Row")
		.OnPaintDropIndicator(this, &SDMMaterialSlotLayerItem::OnLayerItemPaintDropIndicator)
		.OnCanAcceptDrop(this, &SDMMaterialSlotLayerItem::OnLayerItemCanAcceptDrop)
		.OnDragDetected(this, &SDMMaterialSlotLayerItem::OnLayerItemDragDetected)
		.OnAcceptDrop(this, &SDMMaterialSlotLayerItem::OnLayerItemAcceptDrop)
		, InLayerView
	);

	SetContent(CreateMainContent());

	SetCursor(EMouseCursor::GrabHand);
}

TSharedPtr<SDMMaterialSlotLayerView> SDMMaterialSlotLayerItem::GetSlotLayerView() const
{
	return LayerViewWeak.Pin();
}

UDMMaterialLayerObject* SDMMaterialSlotLayerItem::GetLayer() const
{
	if (LayerItem.IsValid())
	{
		return LayerItem->GetLayer();
	}

	return nullptr;
}

int32 SDMMaterialSlotLayerItem::GetLayerIndex() const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		return Layer->FindIndex();
	}

	return INDEX_NONE;
}

TSharedPtr<SDMMaterialSlotLayerEffectView> SDMMaterialSlotLayerItem::GetEffectView() const
{
	return EffectView;
}

bool SDMMaterialSlotLayerItem::AreEffectsExpanded() const
{
	// Default to expanded
	bool bExpanded = true;

	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		FDMWidgetLibrary::Get().GetExpansionState(Layer, EffectViewName, bExpanded);
	}

	return bExpanded;
}

void SDMMaterialSlotLayerItem::SetEffectsExpanded(bool bInExpanded)
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		FDMWidgetLibrary::Get().SetExpansionState(Layer, EffectViewName, bInExpanded);
	}
}

TSharedPtr<SDMMaterialStage> SDMMaterialSlotLayerItem::GetWidgetForStageType(EDMMaterialLayerStage InLayerStage) const
{
	if (EnumHasAnyFlags(InLayerStage, EDMMaterialLayerStage::Base))
	{
		// Technically it could be base or mask stage, so let's check.
		if (BaseStageWidget.IsValid())
		{
			return BaseStageWidget;
		}
	}
	
	if (EnumHasAnyFlags(InLayerStage, EDMMaterialLayerStage::Mask))
	{
		return MaskStageWidget;
	}

	return nullptr;
}

TSharedPtr<SDMMaterialStage> SDMMaterialSlotLayerItem::GetWidgetForStage(UDMMaterialStage* InStage) const
{
	if (BaseStageWidget.IsValid() && BaseStageWidget->GetStage() == InStage)
	{
		return BaseStageWidget;
	}

	if (MaskStageWidget.IsValid() && MaskStageWidget->GetStage() == InStage)
	{
		return MaskStageWidget;
	}

	return nullptr;
}

bool SDMMaterialSlotLayerItem::AreStagesLinked() const
{
	if (const UDMMaterialLayerObject* Layer = GetLayer())
	{
		return Layer->IsTextureUVLinkEnabled();
	}

	return false;
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateMainContent()
{
	UDMMaterialEffectStack* EffectStack = nullptr;

	if (LayerItem.IsValid())
	{
		if (UDMMaterialLayerObject* Layer = LayerItem->GetLayer())
		{
			EffectStack = Layer->GetEffectStack();
		}
	}

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			CreateHeaderRowContent()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(56.0f, 0.0f, 2.0f, 2.0f)
		[
			CreateEffectsRowContent()
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateHeaderRowContent()
{
	using namespace UE::DynamicMaterialEditor::Private;

	constexpr float HorizontalSpacing = 1.f;
	constexpr float VerticalSpacing = 3.f;

	return SNew(SBox)
		.MinDesiredWidth(310.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5.0f, VerticalSpacing, HorizontalSpacing, VerticalSpacing)
			[
				CreateLayerBypassButton()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					CreateLayerBaseToggleButton()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					CreateStageSourceButton(EDMMaterialLayerStage::Base)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, VerticalSpacing, HorizontalSpacing, VerticalSpacing)
			[
				CreateStageWidget(EDMMaterialLayerStage::Base)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, VerticalSpacing, HorizontalSpacing, VerticalSpacing)
			[
				CreateLayerLinkToggleButton()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					CreateLayerMaskToggleButton()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					CreateStageSourceButton(EDMMaterialLayerStage::Mask)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, VerticalSpacing, HorizontalSpacing, VerticalSpacing)
			[
				CreateStageWidget(EDMMaterialLayerStage::Mask)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.Padding(HorizontalSpacing + 5.f, VerticalSpacing, HorizontalSpacing, VerticalSpacing + 5.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.0f, VerticalSpacing, 0.0f, VerticalSpacing)
				[
					SAssignNew(LayerHeaderTextContainer, SBox)
					[
						CreateLayerHeaderText()
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.0f, VerticalSpacing, 0.0f, VerticalSpacing)
				[
					CreateBlendModeSelector()
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, VerticalSpacing, 5.0f, VerticalSpacing)
			[
				CreateEffectsToggleButton()
			]
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateEffectsRowContent()
{
	return SNew(SHorizontalBox)
		.Visibility(this, &SDMMaterialSlotLayerItem::GetEffectsListVisibility)
		.Cursor(EMouseCursor::Default)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(3.0f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(FLinearColor(1, 1, 1, 0.2f))
				.BorderImage(FDynamicMaterialEditorStyle::Get().GetBrush("Border.Right"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(1.0f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(FLinearColor(1, 1, 1, 0.2f))
				.BorderImage(FDynamicMaterialEditorStyle::Get().GetBrush("Border.Left"))
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(EffectView, SDMMaterialSlotLayerEffectView, SharedThis(this))
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateStageWidget(EDMMaterialLayerStage InLayerStage)
{
	if (!LayerItem.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	UDMMaterialLayerObject* Layer = LayerItem->GetLayer();

	if (!Layer)
	{
		return SNullWidget::NullWidget;
	}

	UDMMaterialStage* Stage = Layer->GetStage(InLayerStage);

	switch (InLayerStage)
	{
		case EDMMaterialLayerStage::Base:
			return SAssignNew(BaseStageWidget, SDMMaterialStage, SharedThis(this), Stage);

		case EDMMaterialLayerStage::Mask:
			return SAssignNew(MaskStageWidget, SDMMaterialStage, SharedThis(this), Stage);
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateHandleWidget()
{
	TSharedRef<SBox> LayerIndexTextHandleWidget = 
		SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Cursor(bIsDynamic ? EMouseCursor::Default : EMouseCursor::GrabHand)
		.ToolTipText(this, &SDMMaterialSlotLayerItem::GetToolTipText)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.BorderImage(this, &SDMMaterialSlotLayerItem::GetRowHandleBrush)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.TextStyle(&FDynamicMaterialEditorStyle::Get(), "LayerView.Row.HeaderText.Small")
					.Text(this, &SDMMaterialSlotLayerItem::GetLayerIndexText)
				]
			]
		];

	// Make sure all the index numbers align between single and double digit values.
	constexpr float HandleThickness = 20.0f;

	LayerIndexTextHandleWidget->SetHeightOverride(HandleThickness);

	return LayerIndexTextHandleWidget;
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateLayerBypassButton()
{
	return SNew(SButton)
		.IsEnabled(!bIsDynamic)
		.ContentPadding(4.0f)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.Cursor(EMouseCursor::Default)
		.ToolTipText(LOCTEXT("LayerBypassTooltip", "Toggle the bypassing of this layer."))
		.OnClicked(this, &SDMMaterialSlotLayerItem::OnCreateLayerBypassButtonClicked)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(16.0f))
			.Image(this, &SDMMaterialSlotLayerItem::GetCreateLayerBypassButtonImage)
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateTogglesWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateLayerBaseToggleButton()
			]
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateLayerMaskToggleButton()
			]
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateLayerBaseToggleButton()
{
	return SNew(SButton)
		.IsEnabled(!bIsDynamic)
		.ContentPadding(0.0f)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.Cursor(EMouseCursor::Default)
		.ToolTipText(LOCTEXT("MaterialLayerBaseToggleTooltip", "Toggle the Layer Base."))
		.OnClicked(this, &SDMMaterialSlotLayerItem::OnStageToggleButtonClicked, EDMMaterialLayerStage::Base)
		[
			SNew(SImage)
			.Image(this, &SDMMaterialSlotLayerItem::GetStageToggleButtonImage, EDMMaterialLayerStage::Base)
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateLayerMaskToggleButton()
{
	return SNew(SButton)
		.IsEnabled(!bIsDynamic)
		.ContentPadding(0.0f)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("MaterialLayerMaskToggleTooltip", "Toggle the Layer Mask."))
		.Cursor(EMouseCursor::Default)
		.OnClicked(this, &SDMMaterialSlotLayerItem::OnStageToggleButtonClicked, EDMMaterialLayerStage::Mask)
		[
			SNew(SImage)
			.Image(this, &SDMMaterialSlotLayerItem::GetStageToggleButtonImage, EDMMaterialLayerStage::Mask)
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateLayerLinkToggleButton()
{
	return SNew(SButton)
		.IsEnabled(!bIsDynamic)
		.ContentPadding(0.0f)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("MaterialStageLinkTooltip", "Toggle Layer UV Link."))
		.Cursor(EMouseCursor::Default)
		.OnClicked(this, &SDMMaterialSlotLayerItem::OnLayerLinkToggleButton)
		.Visibility(this, &SDMMaterialSlotLayerItem::GetLayerLinkToggleButtonVisibility)
		[
			SNew(SImage)
			.Image(this, &SDMMaterialSlotLayerItem::GetLayerLinkToggleButtonImage)
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateLayerHeaderText()
{
	TSharedRef<SWidget> TextBlock = SNew(STextBlock)
		.ColorAndOpacity(FSlateColor(EStyleColor::PrimaryHover))
		.TextStyle(FDynamicMaterialEditorStyle::Get(), "SmallFont")
		.Text(GetLayerHeaderText());

	if (!bIsDynamic)
	{
		TextBlock->SetCursor(EMouseCursor::TextEditBeam);

		TextBlock->SetOnMouseButtonDown(FPointerEventHandler::CreateSPLambda(
			this,
			[this](const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
			{
				if (LayerHeaderTextContainer.IsValid())
				{
					if (TSharedPtr<SWidget> EditableContent = CreateLayerHeaderEditableText())
					{
						LayerHeaderTextContainer->SetContent(EditableContent.ToSharedRef());
						FSlateApplication::Get().SetKeyboardFocus(EditableContent);
					}
				}

				return FReply::Handled();
			}
		));
	}

	return TextBlock;
}

TSharedPtr<SWidget> SDMMaterialSlotLayerItem::CreateLayerHeaderEditableText()
{
	if (!LayerItem.IsValid())
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = LayerItem->GetLayer();

	if (!Layer)
	{
		return nullptr;
	}

	FText LayerName = Layer->GetLayerName();

	return SNew(SEditableTextBox)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.HintText(LOCTEXT("LayerName", "Layer Name"))
		.IsEnabled(true)
		.Text(LayerName)
		.Style(FDynamicMaterialEditorStyle::Get(), "InlineEditableTextBoxStyle")
		.OnTextCommitted(FOnTextCommitted::CreateSP(this, &SDMMaterialSlotLayerItem::OnLayerNameChangeCommited));
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateEffectsToggleButton()
{
	return SNew(SButton)
		.ContentPadding(0.0f)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("MaterialLayerFxTooltip", "Show or hide the effect list."))
		.Visibility(this, &SDMMaterialSlotLayerItem::GetEffectsToggleButtonVisibility)
		.Cursor(EMouseCursor::Default)
		.OnClicked(this, &SDMMaterialSlotLayerItem::OnEffectsToggleButtonClicked)
		[
			SNew(SImage)
			.Image(this, &SDMMaterialSlotLayerItem::GetEffectsToggleButtonImage)
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateStageSourceButton(EDMMaterialLayerStage InStage)
{
	return SNew(SButton)
		.IsEnabled(!bIsDynamic)
		.ContentPadding(5.f)
		.ButtonStyle(FDynamicMaterialEditorStyle::Get(), "HoverHintOnly")
		.ToolTipText(this, &SDMMaterialSlotLayerItem::GetStageSourceButtonToolTip, InStage)
		.Cursor(EMouseCursor::Default)
		.OnClicked(this, &SDMMaterialSlotLayerItem::OnStageSourceButtonClicked, InStage)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(16.f))
			.Image(this, &SDMMaterialSlotLayerItem::GetStageSourceButtonImage, InStage)
		];
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::CreateBlendModeSelector()
{
	TSubclassOf<UDMMaterialStageBlend> SelectedBlendMode = nullptr;

	if (const UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialStage* BaseStage = Layer->GetFirstEnabledStage(EDMMaterialLayerStage::Base))
		{
			if (UDMMaterialStageSource* BaseStageSource = BaseStage->GetSource())
			{
				SelectedBlendMode = BaseStageSource->GetClass();
			}
		}
	}

	return SNew(SDMMaterialLayerBlendMode, SharedThis(this))
		.SelectedItem(SelectedBlendMode);
}

EVisibility SDMMaterialSlotLayerItem::GetEffectsListVisibility() const
{
	if (!LayerItem.IsValid())
	{
		return EVisibility::Collapsed;
	}

	UDMMaterialLayerObject* Layer = LayerItem->GetLayer();

	if (!Layer)
	{
		return EVisibility::Collapsed;
	}

	UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack();

	if (!EffectStack || EffectStack->GetEffects().IsEmpty())
	{
		return EVisibility::Collapsed;
	}

	return AreEffectsExpanded()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SDMMaterialSlotLayerItem::GetEffectsToggleButtonVisibility() const
{
	if (LayerItem.IsValid())
	{
		if (UDMMaterialLayerObject* Layer = LayerItem->GetLayer())
		{
			if (UDMMaterialEffectStack* EffectStack = Layer->GetEffectStack())
			{
				if (!EffectStack->GetEffects().IsEmpty())
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Hidden;
}

void SDMMaterialSlotLayerItem::OnLayerNameChangeCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	ON_SCOPE_EXIT
	{
		LayerHeaderTextContainer->SetContent(CreateLayerHeaderText());
	};

	switch (InCommitType)
	{
		case ETextCommit::OnUserMovedFocus:
		case ETextCommit::OnCleared:
			return;
	}

	if (!LayerItem.IsValid())
	{
		return;
	}

	UDMMaterialLayerObject* Layer = LayerItem->GetLayer();

	if (!Layer)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("ChangeLayerName", "Change Layer Name"));
	Layer->Modify();
	Layer->SetLayerName(InText);	
}

FReply SDMMaterialSlotLayerItem::OnEffectsToggleButtonClicked()
{
	SetEffectsExpanded(!AreEffectsExpanded());

	return FReply::Handled();
}

const FSlateBrush* SDMMaterialSlotLayerItem::GetEffectsToggleButtonImage() const
{
	static const FSlateBrush* Displayed = FDynamicMaterialEditorStyle::Get().GetBrush("EffectsView.Row.Fx.Opened");
	static const FSlateBrush* Hidden = FDynamicMaterialEditorStyle::Get().GetBrush("EffectsView.Row.Fx.Closed");

	if (AreEffectsExpanded())
	{
		return Displayed;
	}

	return Hidden;
}

const FSlateBrush* SDMMaterialSlotLayerItem::GetRowHandleBrush() const
{
	static const FSlateBrush* const Default = FDynamicMaterialEditorStyle::Get().GetBrush("LayerView.Row.Handle.Left");
	static const FSlateBrush* const Selected = FDynamicMaterialEditorStyle::Get().GetBrush("LayerView.Row.Handle.Left.Select");
	static const FSlateBrush* const Hovered = FDynamicMaterialEditorStyle::Get().GetBrush("LayerView.Row.Handle.Left.Hover");
	static const FSlateBrush* const SelectedHovered = FDynamicMaterialEditorStyle::Get().GetBrush("LayerView.Row.Handle.Left.Select.Hover");

	const bool bSelected = IsSelected();
	const bool bHovered = IsHovered();

	if (bSelected && bHovered)
	{
		return SelectedHovered;
	}
	else if (bSelected)
	{
		return Selected;
	}
	else if (bHovered)
	{
		return Hovered;
	}
	else
	{
		return Default;
	}
}

const FSlateBrush* SDMMaterialSlotLayerItem::GetCreateLayerBypassButtonImage() const
{
	static const FSlateBrush* const ExposeBrush = FCoreStyle::Get().GetBrush("Kismet.VariableList.ExposeForInstance");
	static const FSlateBrush* const HideBrush = FCoreStyle::Get().GetBrush("Kismet.VariableList.HideForInstance");

	if (const UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (Layer->IsEnabled())
		{
			return ExposeBrush;
		}
	}

	return HideBrush;
}

FReply SDMMaterialSlotLayerItem::OnCreateLayerBypassButtonClicked()
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		FDMScopedUITransaction Transaction(LOCTEXT("ToggledLayerVisibility", "Toggle Layer Visibility"));
		Layer->Modify();
		Layer->SetEnabled(!Layer->IsEnabled());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

EVisibility SDMMaterialSlotLayerItem::GetLayerLinkToggleButtonVisibility() const
{
	if (const UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base))
		{
			if (UDMMaterialStageSource* const StageSource = BaseStage->GetSource())
			{
				if (StageSource->IsA<UDMMaterialStageInputTextureUV>())
				{
					return EVisibility::Visible;
				}

				if (const UDMMaterialStageThroughput* const Throughput = Cast<UDMMaterialStageThroughput>(StageSource))
				{
					if (Throughput->SupportsLayerMaskTextureUVLink())
					{
						return EVisibility::Visible;
					}
				}
			}
		}
	}

	return EVisibility::Hidden;
}

const FSlateBrush* SDMMaterialSlotLayerItem::GetLayerLinkToggleButtonImage() const
{
	static const FSlateBrush* const Unlinked = FDynamicMaterialEditorStyle::Get().GetBrush("Icons.Stage.ChainUnlinked.Vertical");
	static const FSlateBrush* const Linked = FDynamicMaterialEditorStyle::Get().GetBrush("Icons.Stage.ChainLinked.Vertical");

	return AreStagesLinked() ? Linked : Unlinked;
}

FReply SDMMaterialSlotLayerItem::OnLayerLinkToggleButton()
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		FDMScopedUITransaction Transaction(LOCTEXT("UVLayerLinkToggle", "Toggle Layer UV Link"));
		Layer->Modify();
		Layer->ToggleTextureUVLinkEnabled();

		if (MaskStageWidget.IsValid())
		{
			if (UDMMaterialStage* MaskStage = MaskStageWidget->GetStage())
			{
				if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = GetSlotLayerView())
				{
					if (TSharedPtr<SDMMaterialSlotEditor> SlotEditor = LayerView->GetSlotEditorWidget())
					{
						if (TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditor->GetEditorWidget())
						{
							if (TSharedPtr<SDMMaterialComponentEditor> ComponentEditor = EditorWidget->GetComponentEditorWidget())
							{
								if (ComponentEditor->GetObject() == MaskStage)
								{
									EditorWidget->EditComponent(MaskStage, /* Force Refresh */ true);
								}
							}
						}
					}
				}
			}
		}
	}

	return FReply::Handled();
}

const FSlateBrush* SDMMaterialSlotLayerItem::GetStageToggleButtonImage(EDMMaterialLayerStage InLayerStage) const
{
	static const FSlateBrush* Disabled = FDynamicMaterialEditorStyle::Get().GetBrush("Icons.Stage.Disabled");
	static const FSlateBrush* Enabled = FDynamicMaterialEditorStyle::Get().GetBrush("Icons.Stage.Enabled");

	if (const UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialStage* Stage = Layer->GetStage(InLayerStage))
		{
			if (Stage->IsEnabled())
			{
				return Enabled;
			}
		}
	}

	return Disabled;
}

FReply SDMMaterialSlotLayerItem::OnStageToggleButtonClicked(EDMMaterialLayerStage InLayerStage)
{
	if (const UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialStage* Stage = Layer->GetStage(InLayerStage))
		{
			FDMScopedUITransaction Transaction(LOCTEXT("ToggleBaseStageEnabled", "Toggle Stage Enabled"));
			Stage->Modify();

			if (!Stage->SetEnabled(!Stage->IsEnabled()))
			{
				Transaction.Transaction.Cancel();
			}
		}
	}

	return FReply::Handled();
}

FText SDMMaterialSlotLayerItem::GetStageSourceButtonToolTip(EDMMaterialLayerStage InLayerStage) const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialStage* Stage = Layer->GetFirstEnabledStage(InLayerStage))
		{
			return FText::Format(
				LOCTEXT("StageSourceToolTipFormat", "Click to change the Material Stage Source.\n\nSource: {0}."),
				Stage->GetComponentDescription()
			);
		}
	}

	return GetDefault<UDMMaterialStage>()->GetComponentDescription();
}

FText SDMMaterialSlotLayerItem::GetToolTipText() const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		return Layer->GetComponentDescription();
	}

	return FText::GetEmpty();
}

FText SDMMaterialSlotLayerItem::GetLayerHeaderText() const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		return Layer->GetComponentDescription();
	}

	return FText::GetEmpty();
}

FText SDMMaterialSlotLayerItem::GetLayerIndexText() const
{
	return FText::Format(LOCTEXT("LayerIndexText", "{0}"), GetLayerIndex());
}

FText SDMMaterialSlotLayerItem::GetBlendModeText() const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialSlot* Slot = Layer->GetSlot())
		{
			if (UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = Slot->GetMaterialModelEditorOnlyData())
			{
				if (UDMMaterialProperty* Property = ModelEditorOnlyData->GetMaterialProperty(Layer->GetMaterialProperty()))
				{
					return Property->GetDescription();
				}
			}
		}
	}

	return LOCTEXT("Error", "Error");
}

const FSlateBrush* SDMMaterialSlotLayerItem::GetStageSourceButtonImage(EDMMaterialLayerStage InLayerStage) const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		if (UDMMaterialStage* Stage = Layer->GetFirstValidStage(InLayerStage))
		{
			return Stage->GetComponentIcon().GetIcon();
		}
	}

	return FAppStyle::Get().GetBrush("Icons.ErrorWithColor");
}

FReply SDMMaterialSlotLayerItem::OnStageSourceButtonClicked(EDMMaterialLayerStage InLayerStage)
{
	FSlateApplication& SlateApplication = FSlateApplication::Get();

	SlateApplication.PushMenu(
		SharedThis(this),
		FWidgetPath(),
		GetStageSourceMenuContent(InLayerStage),
		SlateApplication.GetCursorPos(),
		FPopupTransitionEffect::ContextMenu
	);

	return FReply::Handled();
}

TSharedRef<SWidget> SDMMaterialSlotLayerItem::GetStageSourceMenuContent(EDMMaterialLayerStage InLayerStage)
{
	TSharedPtr<SDMMaterialSlotLayerView> LayerView = GetSlotLayerView();

	if (!LayerView.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = LayerView->GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	switch (InLayerStage)
	{
		case EDMMaterialLayerStage::Base:
			return FDMMaterialStageSourceMenus::MakeChangeSourceMenu(SlotEditorWidget, BaseStageWidget);

		case EDMMaterialLayerStage::Mask:
			return FDMMaterialStageSourceMenus::MakeChangeSourceMenu(SlotEditorWidget, MaskStageWidget);

		default:
			return SNullWidget::NullWidget;
	}
}

int32 SDMMaterialSlotLayerItem::OnLayerItemPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& InArgs, 
	const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, 
	const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(InItemDropZone);
	static float OffsetX = 10.0f;
	FVector2D Offset(OffsetX * GetIndentLevel(), 0.0f);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId++,
		InAllottedGeometry.ToPaintGeometry(
			FVector2D(InAllottedGeometry.GetLocalSize() - Offset),
			FSlateLayoutTransform(Offset)
		),
		DropIndicatorBrush,
		ESlateDrawEffect::None,
		DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return InLayerId;
}

TOptional<EItemDropZone> SDMMaterialSlotLayerItem::OnLayerItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, 
	TSharedPtr<FDMMaterialLayerReference> InSlotLayer) const
{
	if (!InSlotLayer.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	UDMMaterialLayerObject* Layer = GetLayer();

	if (!Layer)
	{
		return TOptional<EItemDropZone>();
	}

	if (TSharedPtr<FDMSlotLayerDragDropOperation> SlotLayerDragDropOp = InDragDropEvent.GetOperationAs<FDMSlotLayerDragDropOperation>())
	{
		UDMMaterialLayerObject* DraggedSlotLayer = SlotLayerDragDropOp->GetLayer();

		if (!DraggedSlotLayer)
		{
			return TOptional<EItemDropZone>();
		}

		SlotLayerDragDropOp->SetToInvalidDropLocation();

		switch (InDropZone)
		{
			case EItemDropZone::AboveItem:
				if (Layer->CanMoveLayerAbove(DraggedSlotLayer))
				{
					SlotLayerDragDropOp->SetToValidDropLocation();
					return InDropZone;
				}
				break;

			case EItemDropZone::OntoItem:
			case EItemDropZone::BelowItem:
				if (Layer->CanMoveLayerBelow(DraggedSlotLayer))
				{
					SlotLayerDragDropOp->SetToValidDropLocation();
					return EItemDropZone::BelowItem;
				}
				break;
		}
	}
	else if (TSharedPtr<FContentBrowserDataDragDropOp> ContentBrowserDragDropOp = InDragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
	{
		for (const FAssetData& DraggedAsset : ContentBrowserDragDropOp->GetAssets())
		{
			UClass* AssetClass = DraggedAsset.GetClass(EResolveClass::Yes);

			if (AssetClass && AssetClass->IsChildOf(UMaterialFunctionInterface::StaticClass()))
			{
				return EItemDropZone::OntoItem;
			}
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SDMMaterialSlotLayerItem::OnLayerItemDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (bIsDynamic)
	{
		return FReply::Handled();
	}

	const bool bShouldDuplicate = InMouseEvent.IsAltDown();

	TSharedRef<FDMSlotLayerDragDropOperation> SlotLayerDragDropOp = MakeShared<FDMSlotLayerDragDropOperation>(
		SharedThis(this), 
		bShouldDuplicate
	);

	return FReply::Handled().BeginDragDrop(SlotLayerDragDropOp);
}

FReply SDMMaterialSlotLayerItem::OnLayerItemAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, 
	TSharedPtr<FDMMaterialLayerReference> InSlotLayer)
{
	if (!InSlotLayer.IsValid())
	{
		return FReply::Handled();
	}

	UDMMaterialLayerObject* DraggedOverLayer = GetLayer();

	if (!DraggedOverLayer)
	{
		return FReply::Handled();
	}

	if (TSharedPtr<FDMSlotLayerDragDropOperation> SlotLayerDragDropOp = InDragDropEvent.GetOperationAs<FDMSlotLayerDragDropOperation>())
	{
		if (DraggedOverLayer->GetStage(EDMMaterialLayerStage::Base))
		{
			HandleLayerDrop(SlotLayerDragDropOp);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDMMaterialSlotLayerItem::HandleLayerDrop(const TSharedPtr<FDMSlotLayerDragDropOperation>& InOperation)
{
	UDMMaterialLayerObject* DraggedLayer = InOperation->GetLayer();

	if (!DraggedLayer || !DraggedLayer->GetStage(EDMMaterialLayerStage::Base))
	{
		return;
	}

	const int32 ThisLayerIndex = GetLayerIndex();

	if (ThisLayerIndex == INDEX_NONE)
	{
		return;
	}

	UDMMaterialSlot* const Slot = DraggedLayer->GetSlot();

	if (!Slot)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("MoveLayer", "Move Material Designer Layer"));
	Slot->Modify();
	Slot->MoveLayer(DraggedLayer, ThisLayerIndex);

	if (TSharedPtr<SDMMaterialSlotLayerView> LayerView = LayerViewWeak.Pin())
	{
		LayerView->RequestListRefresh();

		if (TSharedPtr<SDMMaterialSlotEditor> SlotEditor = LayerView->GetSlotEditorWidget())
		{
			SlotEditor->InvalidateSlotSettings();
		}
	}
}

#undef LOCTEXT_NAMESPACE
