// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/Views/STableRow.h"

#include "Utils/DMPrivate.h"
#include "Widgets/Views/SListView.h"

class FContentBrowserDataDragDropOp;
class FDMSlotLayerDragDropOperation;
class SDMMaterialSlotLayerEffectView;
class SDMMaterialSlotLayerView;
class SDMMaterialStage;
class UDMMaterialEffectStack;
enum class EDMMaterialLayerStage : uint8;
struct FSlateBrush;

/**
 * Material Slot Layer
 *
 * Represents a single item in the layer view that can be dragged and re-arranged.
 * Displays the base and mask stages, along with enable and link buttons.
 */
class SDMMaterialSlotLayerItem : public STableRow<TSharedPtr<FDMMaterialLayerReference>>
{
public:
	SLATE_DECLARE_WIDGET(SDMMaterialSlotLayerItem, STableRow<TSharedPtr<FDMMaterialLayerReference>>)

	SLATE_BEGIN_ARGS(SDMMaterialSlotLayerItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerView>& InLayerView,
		const TSharedPtr<FDMMaterialLayerReference>& InLayerReferenceItem);

	TSharedPtr<SDMMaterialSlotLayerView> GetSlotLayerView() const;

	UDMMaterialLayerObject* GetLayer() const;

	int32 GetLayerIndex() const;

	TSharedPtr<SDMMaterialSlotLayerEffectView> GetEffectView() const;

	bool AreEffectsExpanded() const;

	void SetEffectsExpanded(bool bInExpanded);

	TSharedPtr<SDMMaterialStage> GetWidgetForStageType(EDMMaterialLayerStage InLayerStage) const;

	TSharedPtr<SDMMaterialStage> GetWidgetForStage(UDMMaterialStage* InStage) const;

protected:
	static const FLazyName EffectViewName;

	TWeakPtr<SDMMaterialSlotLayerView> LayerViewWeak;
	TSharedPtr<FDMMaterialLayerReference> LayerItem;
	bool bIsDynamic;

	TSharedPtr<SDMMaterialStage> BaseStageWidget;
	TSharedPtr<SDMMaterialStage> MaskStageWidget;
	TSharedPtr<SBox> LayerHeaderTextContainer;
	TSharedPtr<SDMMaterialSlotLayerEffectView> EffectView;

	bool AreStagesLinked() const;

	/** Slots */
	TSharedRef<SWidget> CreateMainContent();
	TSharedRef<SWidget> CreateHeaderRowContent();
	TSharedRef<SWidget> CreateEffectsRowContent();

	/** Create components of the slot layer. */
	TSharedRef<SWidget> CreateStageWidget(EDMMaterialLayerStage InLayerStage);
	TSharedRef<SWidget> CreateHandleWidget();

	TSharedRef<SWidget> CreateLayerBypassButton();
	TSharedRef<SWidget> CreateTogglesWidget();
	TSharedRef<SWidget> CreateLayerBaseToggleButton();
	TSharedRef<SWidget> CreateLayerMaskToggleButton();
	TSharedRef<SWidget> CreateLayerLinkToggleButton();
	TSharedRef<SWidget> CreateLayerHeaderText();
	TSharedPtr<SWidget> CreateLayerHeaderEditableText();
	TSharedRef<SWidget> CreateEffectsToggleButton();
	TSharedRef<SWidget> CreateStageSourceButton(EDMMaterialLayerStage InStage);
	TSharedRef<SWidget> CreateBlendModeSelector();

	EVisibility GetEffectsListVisibility() const;
	EVisibility GetEffectsToggleButtonVisibility() const;

	void OnLayerNameChangeCommited(const FText& InText, ETextCommit::Type InCommitType);

	const FSlateBrush* GetEffectsToggleButtonImage() const;
	FReply OnEffectsToggleButtonClicked();

	const FSlateBrush* GetRowHandleBrush() const;

	const FSlateBrush* GetCreateLayerBypassButtonImage() const;
	FReply OnCreateLayerBypassButtonClicked();

	EVisibility GetLayerLinkToggleButtonVisibility() const;
	const FSlateBrush* GetLayerLinkToggleButtonImage() const;
	FReply OnLayerLinkToggleButton();

	const FSlateBrush* GetStageToggleButtonImage(EDMMaterialLayerStage InLayerStage) const;
	FReply OnStageToggleButtonClicked(EDMMaterialLayerStage InLayerStage);

	FText GetStageSourceButtonToolTip(EDMMaterialLayerStage InLayerStage) const;
	const FSlateBrush* GetStageSourceButtonImage(EDMMaterialLayerStage InLayerStage) const;
	FReply OnStageSourceButtonClicked(EDMMaterialLayerStage InLayerStage);
	TSharedRef<SWidget> GetStageSourceMenuContent(EDMMaterialLayerStage InLayerStage);

	/** Text */
	FText GetToolTipText() const;
	FText GetLayerHeaderText() const;
	FText GetLayerIndexText() const;
	FText GetBlendModeText() const;

	/** Drag and Drop */
	int32 OnLayerItemPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry,
		const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, 
		bool bInParentEnabled) const;

	TOptional<EItemDropZone> OnLayerItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone,
		TSharedPtr<FDMMaterialLayerReference> InSlotLayer) const;

	FReply OnLayerItemDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	FReply OnLayerItemAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone,
		TSharedPtr<FDMMaterialLayerReference> InSlotLayer);

	void HandleLayerDrop(const TSharedPtr<FDMSlotLayerDragDropOperation>& InOperation);
};
