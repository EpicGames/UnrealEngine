// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "DynamicMaterialEditorCommands.h"
#include "DynamicMaterialModule.h"
#include "Framework/Application/SlateApplication.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UI/Menus/DMMaterialSlotLayerMenus.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialSlotLayerView"

void SDMMaterialSlotLayerView::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialSlotLayerView::~SDMMaterialSlotLayerView()
{
	UnbindCommands();

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = SlotEditorWidgetWeak.Pin())
	{
		if (UDMMaterialSlot* Slot = SlotEditorWidget->GetSlot())
		{
			Slot->GetOnLayersUpdateDelegate().RemoveAll(this);
		}
	}
}

void SDMMaterialSlotLayerView::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotEditor> InSlotEditorWidget)
{
	SlotEditorWidgetWeak = InSlotEditorWidget;

	UDMMaterialSlot* Slot = InSlotEditorWidget->GetSlot();
	ensure(Slot);

	SListView::Construct(SListView::FArguments()
		.ListItemsSource(&LayerItems)
		.SelectionMode(ESelectionMode::Single)
		.ClearSelectionOnClick(false)
		.EnableAnimatedScrolling(true)
		.ScrollbarVisibility(EVisibility::Visible)
		.ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
		.OnGenerateRow(this, &SDMMaterialSlotLayerView::OnGenerateLayerItemWidget)
		.OnSelectionChanged(this, &SDMMaterialSlotLayerView::OnLayerItemSelectionChanged)
		.OnContextMenuOpening(this, &SDMMaterialSlotLayerView::CreateLayerItemContextMenu)
	);

	RegenerateItems();
	RequestListRefresh();
	BindCommands();

	Slot->GetOnLayersUpdateDelegate().AddSP(this, &SDMMaterialSlotLayerView::OnLayersUpdated);
}

TSharedPtr<SDMMaterialSlotEditor> SDMMaterialSlotLayerView::GetSlotEditorWidget() const
{
	return SlotEditorWidgetWeak.Pin();
}

UDMMaterialLayerObject* SDMMaterialSlotLayerView::GetSelectedLayer() const
{
	if (SelectedItems.Num() != 1)
	{
		return nullptr;
	}

	TArray<TSharedPtr<FDMMaterialLayerReference>> SelectedLayers = GetSelectedItems();

	if (!SelectedLayers[0]->IsValid())
	{
		return nullptr;
	}

	return SelectedLayers[0]->GetLayer();
}

void SDMMaterialSlotLayerView::SetSelectedLayer(UDMMaterialLayerObject* InLayer)
{
	UDMMaterialLayerObject* SelectedLayer = GetSelectedLayer();

	if (InLayer == SelectedLayer)
	{
		return;
	}

	ClearSelection();

	for (const TSharedPtr<FDMMaterialLayerReference>& LayerItem : LayerItems)
	{
		if (LayerItem->LayerWeak == InLayer)
		{
			SetItemSelection(LayerItem, /* Selected */ true);
			break;
		}
	}
}

TSharedPtr<SDMMaterialSlotLayerItem> SDMMaterialSlotLayerView::GetWidgetForLayer(UDMMaterialLayerObject* InLayer) const
{
	for (const TSharedPtr<FDMMaterialLayerReference>& LayerItem : LayerItems)
	{
		if (LayerItem->LayerWeak == InLayer)
		{
			return WidgetFromLayerItem(LayerItem);
		}
	}

	return nullptr;
}

void SDMMaterialSlotLayerView::EnsureSelectedStage()
{
	if (GetSelectedLayer())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = SlotEditorWidget->GetSlot();

	if (!Slot)
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	if (UDMMaterialComponent* ComponentToEdit = EditorWidget->GetSelectedComponent())
	{
		if (ComponentToEdit->GetTypedParent<UDMMaterialSlot>(/* Allow Subclasses */ true) == Slot)
		{
			return;
		}
	}

	const TArray<UDMMaterialLayerObject*>& Layers = Slot->GetLayers();

	if (Layers.IsEmpty())
	{
		return;
	}

	UDMMaterialLayerObject* LastLayer = Layers.Last();
	SetSelectedLayer(LastLayer);

	if (UDMMaterialStage* Stage = LastLayer->GetFirstEnabledStage(EDMMaterialLayerStage::All))
	{
		EditorWidget->EditComponent(Stage);
	}
}

void SDMMaterialSlotLayerView::PostUndo(bool bSuccess)
{
	OnUndo();
}

void SDMMaterialSlotLayerView::PostRedo(bool bSuccess)
{
	OnUndo();
}

void SDMMaterialSlotLayerView::BindCommands()
{
	TSharedPtr<SDMMaterialSlotEditor> SlotEditor = GetSlotEditorWidget();

	if (!SlotEditor.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditor->GetEditorWidget();

	if (!EditorWidget)
	{
		return;
	}

	const TSharedRef<FUICommandList> CommandList = EditorWidget->GetCommandList();

	const FDynamicMaterialEditorCommands& Commands = FDynamicMaterialEditorCommands::Get();

	CommandList->MapAction(
		Commands.SelectLayerBaseStage,
		FExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::ExecuteSelectLayerStage, EDMMaterialLayerStage::Base),
		FCanExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::CanSelectLayerStage, EDMMaterialLayerStage::Base)
	);

	CommandList->MapAction(
		Commands.SelectLayerMaskStage,
		FExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::ExecuteSelectLayerStage, EDMMaterialLayerStage::Mask),
		FCanExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::CanSelectLayerStage, EDMMaterialLayerStage::Mask)
	);

	CommandList->MapAction(
		Commands.MoveLayerUp,
		FExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::ExecuteMoveLayer, -1),
		FCanExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::CanMoveLayer, -1)
	);

	CommandList->MapAction(
		Commands.MoveLayerDown,
		FExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::ExecuteMoveLayer, 1),
		FCanExecuteAction::CreateSP(this, &SDMMaterialSlotLayerView::CanMoveLayer, 1)
	);
}

void SDMMaterialSlotLayerView::UnbindCommands()
{
	TSharedPtr<SDMMaterialSlotEditor> SlotEditor = GetSlotEditorWidget();

	if (!SlotEditor.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditor->GetEditorWidget();

	if (!EditorWidget)
	{
		return;
	}

	const TSharedRef<FUICommandList> CommandList = EditorWidget->GetCommandList();
	const FDynamicMaterialEditorCommands& Commands = FDynamicMaterialEditorCommands::Get();

	CommandList->UnmapAction(Commands.SelectLayerBaseStage);
	CommandList->UnmapAction(Commands.SelectLayerMaskStage);
	CommandList->UnmapAction(Commands.MoveLayerUp);
	CommandList->UnmapAction(Commands.MoveLayerDown);
}

void SDMMaterialSlotLayerView::RegenerateItems()
{
	LayerItems.Empty();

	TSharedPtr<SDMMaterialSlotEditor> SlotEditor = GetSlotEditorWidget();

	if (!SlotEditor.IsValid())
	{
		return;
	}

	UDMMaterialSlot* Slot = SlotEditor->GetSlot();

	if (!Slot)
	{
		return;
	}

	const TArray<TObjectPtr<UDMMaterialLayerObject>>& SlotLayers = Slot->GetLayers();
	const int32 LayerCount = SlotLayers.Num();

	LayerItems.Reserve(LayerCount);

	for (int32 LayerIdx = LayerCount - 1; LayerIdx >= 0; --LayerIdx)
	{
		LayerItems.Add(MakeShared<FDMMaterialLayerReference>(SlotLayers[LayerIdx]));
	}
}

TSharedRef<ITableRow> SDMMaterialSlotLayerView::OnGenerateLayerItemWidget(TSharedPtr<FDMMaterialLayerReference> InItem, 
	const TSharedRef<STableViewBase>& InOwnerTable)
{
	TSharedRef<SDMMaterialSlotLayerItem> LayerItem = SNew(
		SDMMaterialSlotLayerItem,
		StaticCastSharedRef<SDMMaterialSlotLayerView>(InOwnerTable),
		InItem
	);

	return LayerItem;
}

void SDMMaterialSlotLayerView::OnLayerItemSelectionChanged(TSharedPtr<FDMMaterialLayerReference> InSelectedItem, 
	ESelectInfo::Type InSelectInfo)
{
	if (TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = GetSlotEditorWidget())
	{
		SlotEditorWidget->TriggerLayerSelectionChange(SharedThis(this), InSelectedItem);
	}
}

TSharedPtr<SWidget> SDMMaterialSlotLayerView::CreateLayerItemContextMenu()
{
	TSharedPtr<SDMMaterialSlotEditor> SlotEditor = GetSlotEditorWidget();

	if (!SlotEditor.IsValid())
	{
		return nullptr;
	}

	return FDMMaterialSlotLayerMenus::GenerateSlotLayerMenu(SlotEditor, GetSelectedLayer());
}

TSharedPtr<SDMMaterialSlotLayerItem> SDMMaterialSlotLayerView::WidgetFromLayerItem(const TSharedPtr<FDMMaterialLayerReference>& InItem) const
{
	return StaticCastSharedPtr<SDMMaterialSlotLayerItem>(WidgetFromItem(InItem));
}

bool SDMMaterialSlotLayerView::CanSelectLayerStage(EDMMaterialLayerStage InStageType) const
{
	if (SelectedItems.Num() != 1)
	{
		return false;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditor = GetSlotEditorWidget();

	if (!SlotEditor.IsValid())
	{
		return false;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditor->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return false;
	}

	UDMMaterialLayerObject* SelectedLayer = GetSelectedLayer();

	if (!SelectedLayer)
	{
		return false;
	}

	return !!SelectedLayer->GetFirstValidStage(InStageType);
}

void SDMMaterialSlotLayerView::ExecuteSelectLayerStage(EDMMaterialLayerStage InStageType)
{
	TSharedPtr<SDMMaterialSlotEditor> SlotEditor = GetSlotEditorWidget();
	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditor->GetEditorWidget();
	UDMMaterialLayerObject* SelectedLayer = GetSelectedLayer();
	UDMMaterialStage* Stage = SelectedLayer->GetStage(InStageType);

	EditorWidget->EditComponent(Stage);
}

bool SDMMaterialSlotLayerView::CanMoveLayer(int32 InOffset) const
{
	if (SelectedItems.Num() != 1)
	{
		return false;
	}

	UDMMaterialLayerObject* SelectedLayer = GetSelectedLayer();

	if (!SelectedLayer)
	{
		return false;
	}

	UDMMaterialSlot* Slot = SelectedLayer->GetSlot();

	if (!Slot)
	{
		return false;
	}

	const int32 LayerIndex = SelectedLayer->FindIndex();

	if (LayerIndex == INDEX_NONE)
	{
		return false;
	}

	switch (InOffset)
	{
		case -1:
			if (LayerIndex == 0)
			{
				return false;
			}
			break;

		case 1:
			if (LayerIndex == (Slot->GetLayers().Num() - 1))
			{
				return false;
			}
			break;

		default:
			return false;
	}

	return true;
}

void SDMMaterialSlotLayerView::ExecuteMoveLayer(int32 InOffset)
{
	UDMMaterialLayerObject* SelectedLayer = GetSelectedLayer();
	UDMMaterialSlot* Slot = SelectedLayer->GetSlot();
	const int32 LayerIndex = SelectedLayer->FindIndex();

	FDMScopedUITransaction Transaction(LOCTEXT("MoveLayer", "Move Layer"));
	Slot->Modify();
	SelectedLayer->Modify();

	if (!Slot->MoveLayer(SelectedLayer, LayerIndex + InOffset))
	{
		Transaction.Transaction.Cancel();
	}
}

void SDMMaterialSlotLayerView::OnUndo()
{
	RegenerateItems();
	RequestListRefresh();
}

void SDMMaterialSlotLayerView::OnLayersUpdated(UDMMaterialSlot* InSlot)
{
	RegenerateItems();
	RequestListRefresh();
	EnsureSelectedStage();
}

#undef LOCTEXT_NAMESPACE
