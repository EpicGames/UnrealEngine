// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Templates/SharedPointer.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Views/SListView.h"

#include "Containers/Array.h"

class SDMMaterialSlotEditor;
class SDMMaterialSlotLayerItem;
enum class EDMMaterialLayerStage : uint8;

class SDMMaterialSlotLayerView : public SListView<TSharedPtr<FDMMaterialLayerReference>>, public FSelfRegisteringEditorUndoClient
{
	SLATE_DECLARE_WIDGET(SDMMaterialSlotLayerView, SListView<TSharedPtr<FDMMaterialLayerReference>>)

	SLATE_BEGIN_ARGS(SDMMaterialSlotLayerView) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialSlotLayerView() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotEditor> InSlotEditorWidget);

	TSharedPtr<SDMMaterialSlotEditor> GetSlotEditorWidget() const;

	UDMMaterialLayerObject* GetSelectedLayer() const;

	void SetSelectedLayer(UDMMaterialLayerObject* InLayer);

	TSharedPtr<SDMMaterialSlotLayerItem> GetWidgetForLayer(UDMMaterialLayerObject* InLayer) const;

	void EnsureSelectedStage();

	//~ Begin FUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FUndoClient

protected:
	TWeakPtr<SDMMaterialSlotEditor> SlotEditorWidgetWeak;

	TArray<TSharedPtr<FDMMaterialLayerReference>> LayerItems;

	void BindCommands();

	void UnbindCommands();

	/** List */
	void RegenerateItems();

	TSharedRef<ITableRow> OnGenerateLayerItemWidget(TSharedPtr<FDMMaterialLayerReference> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	void OnLayerItemSelectionChanged(TSharedPtr<FDMMaterialLayerReference> InSelectedItem, ESelectInfo::Type InSelectInfo);

	TSharedPtr<SWidget> CreateLayerItemContextMenu();

	TSharedPtr<SDMMaterialSlotLayerItem> WidgetFromLayerItem(const TSharedPtr<FDMMaterialLayerReference>& InItem) const;

	/** Commands */
	bool CanSelectLayerStage(EDMMaterialLayerStage InStageType) const;

	void ExecuteSelectLayerStage(EDMMaterialLayerStage InStageType);

	bool CanMoveLayer(int32 InOffset) const;

	void ExecuteMoveLayer(int32 InOffset);

	/** Events */
	void OnUndo();

	void OnLayersUpdated(UDMMaterialSlot* InSlot);
};
