// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Editor/SlotEditor/SDMMaterialStage.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "DynamicMaterialModule.h"
#include "Engine/Texture.h"
#include "Framework/Application/SlateApplication.h"
#include "Materials/Material.h"
#include "SAssetDropTarget.h"
#include "Styling/StyleColors.h"
#include "UI/Menus/DMMaterialStageMenus.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerItem.h"
#include "UI/Widgets/Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/Visualizers/SDMMaterialComponentPreview.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#if UE_BUILD_DEBUG
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#endif

#define LOCTEXT_NAMESPACE "SDMMaterialStage"

void SDMMaterialStage::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialStage::~SDMMaterialStage()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		Settings->GetOnSettingsChanged().RemoveAll(this);
	}
}

void SDMMaterialStage::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerItem>& InSlotLayerItem, 
	UDMMaterialStage* InStage)
{
	SlotLayerItemWeak = InSlotLayerItem;
	StageWeak = InStage;

	SetCanTick(false);
	SetCursor(EMouseCursor::Default);

	if (!IsValid(InStage))
	{
		return;
	}

	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerView> SlotLayerView = InSlotLayerItem->GetSlotLayerView();

	if (!SlotLayerView.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = SlotLayerView->GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	ChildSlot
	[
		SNew(SAssetDropTarget)
		.OnAreAssetsAcceptableForDrop(this, &SDMMaterialStage::OnAssetDraggedOver)
		.OnAssetsDropped(this, &SDMMaterialStage::OnAssetsDropped)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(1, 1, 1, 0.5f))
			.Padding(2.0f)
			.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.AssetTileItem.DropShadow"))
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.f)
				[
					SNew(SBorder)
					.Clipping(EWidgetClipping::ClipToBounds)
					.BorderBackgroundColor(FLinearColor::Transparent)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SAssignNew(PreviewImage, SDMMaterialComponentPreview, EditorWidget.ToSharedRef(), InStage)
							.PreviewSize(FVector2D(Settings->StagePreviewSize))
						]
						+ SOverlay::Slot()
						.Padding(5.f)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.X"))
							.DesiredSizeOverride(FVector2D(Settings->StagePreviewSize - 10.f))
							.ColorAndOpacity(FStyleColors::AccentRed)
							.Visibility(this, &SDMMaterialStage::GetDisabledOverlayVisibility)
						]
					]
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(1.0f)
				[
					SNew(SBorder)
					.BorderBackgroundColor(FLinearColor::White)
					.BorderImage(this, &SDMMaterialStage::GetBorderBrush)
				]
			]
		]
	];

	SetToolTip(
		SNew(SToolTip)
		.IsInteractive(false)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.Background"))
		[
			SAssignNew(ToolTipImage, SDMMaterialComponentPreview, EditorWidget.ToSharedRef(), InStage)
			.PreviewSize(FVector2D(Settings->ThumbnailSize))
		]
	);

	Settings->GetOnSettingsChanged().AddSP(this, &SDMMaterialStage::OnSettingsUpdated);
}

TSharedPtr<SDMMaterialSlotLayerItem> SDMMaterialStage::GetSlotLayerView() const
{
	return SlotLayerItemWeak.Pin();
}

UDMMaterialStage* SDMMaterialStage::GetStage() const
{
	return StageWeak.Get();
}

FReply SDMMaterialStage::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		OnMouseButtonDown_Left();
		return FReply::Handled();
	}

#if UE_BUILD_DEBUG
	else if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton
		&& InMouseEvent.IsShiftDown() && InMouseEvent.IsControlDown())
	{
		if (PreviewImage.IsValid())
		{
			if (UMaterial* PreviewMaterial = PreviewImage->GetPreviewMaterial())
			{
				IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
				AssetTools.OpenEditorForAssets({PreviewMaterial});

				return FReply::Handled();
			}
		}
	}
#endif

	return FReply::Unhandled();
}

FReply SDMMaterialStage::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		OnMouseButtonUp_Right();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SDMMaterialStage::IsStageSelected() const
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return false;
	}

	TSharedPtr<SDMMaterialSlotLayerItem> SlotLayerItem = SlotLayerItemWeak.Pin();

	if (!SlotLayerItem.IsValid())
	{
		return false;
	}

	TSharedPtr<SDMMaterialSlotLayerView> SlotLayerView = SlotLayerItem->GetSlotLayerView();

	if (!SlotLayerView.IsValid())
	{
		return false;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = SlotLayerView->GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return false;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = SlotEditorWidget->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return false;
	}

	TSharedPtr<SDMMaterialComponentEditor> ComponentEditorWidget = EditorWidget->GetComponentEditorWidget();

	if (!ComponentEditorWidget.IsValid())
	{
		return false;
	}

	return ComponentEditorWidget->GetObject() == Stage;
}

const FSlateBrush* SDMMaterialStage::GetBorderBrush() const
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return FDynamicMaterialEditorStyle::Get().GetBrush(TEXT("Stage.Inactive"));
	}

	FString BrushName = Stage->IsEnabled()
		? TEXT("Stage.Enabled")
		: TEXT("Stage.Disabled");

	if (IsStageSelected())
	{
		BrushName.Append(".Select");
	}

	if (IsHovered())
	{
		BrushName.Append(".Hover");
	}

	return FDynamicMaterialEditorStyle::Get().GetBrush(*BrushName);
}

EVisibility SDMMaterialStage::GetDisabledOverlayVisibility() const
{
	if (UDMMaterialStage* Stage = GetStage())
	{
		return Stage->IsEnabled()
			? EVisibility::Collapsed
			: EVisibility::HitTestInvisible;
	}

	return EVisibility::Collapsed;
}


void SDMMaterialStage::OnSettingsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return;
	}

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, StagePreviewSize))
	{		
		if (PreviewImage.IsValid())
		{
			PreviewImage->SetPreviewSize(FVector2D(Settings->StagePreviewSize));
		}
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, ThumbnailSize))
	{
		if (ToolTipImage.IsValid())
		{
			ToolTipImage->SetPreviewSize(FVector2D(Settings->ThumbnailSize));
		}
	}
}

bool SDMMaterialStage::OnAssetDraggedOver(TArrayView<FAssetData> InAssets)
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage || !Stage->IsEnabled())
	{
		return false;
	}

	const TArray<UClass*> AllowedClasses = {
		UTexture::StaticClass()
	};

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		for (UClass* AllowedClass : AllowedClasses)
		{
			if (AssetClass->IsChildOf(AllowedClass))
			{
				return true;
			}
		}
	}

	return false;
}

void SDMMaterialStage::OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets)
{
	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			HandleDrop_Texture(Cast<UTexture>(Asset.GetAsset()));
			break;
		}
	}
}

void SDMMaterialStage::HandleDrop_Texture(UTexture* InTexture)
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage || !Stage->IsEnabled())
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("DragTextureOntoStage", "Drag Texture onto Stage"), !FDMInitializationGuard::IsInitializing());
	UDMMaterialValueTexture* TextureValue = nullptr;

	Stage->Modify();

	UDMMaterialStageSource* StageSource = Stage->GetSource();
	StageSource->Modify();

	if (StageSource->IsA<UDMMaterialStageBlend>())
	{
		UDMMaterialStageInputExpression* NewInput = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionTextureSample::StaticClass(),
			UDMMaterialStageBlend::InputB,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		UDMMaterialSubStage* SubStage = NewInput->GetSubStage();

		UDMMaterialStageInputValue* NewInputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			SubStage,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			EDMValueType::VT_Texture,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		TextureValue = Cast<UDMMaterialValueTexture>(NewInputValue->GetValue());
	}
	else if (StageSource->IsA<UDMMaterialStageThroughputLayerBlend>())
	{
		const bool bHasAlpha = UE::DynamicMaterial::Private::HasAlpha(InTexture);

		UDMMaterialStageInputExpression* NewInput = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			Stage,
			UDMMaterialStageExpressionTextureSample::StaticClass(),
			2,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			bHasAlpha ? 1 : 0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		UDMMaterialSubStage* SubStage = NewInput->GetSubStage();

		UDMMaterialStageInputValue* NewInputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			SubStage,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			EDMValueType::VT_Texture,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		TextureValue = Cast<UDMMaterialValueTexture>(NewInputValue->GetValue());
	}
	else
	{
		UDMMaterialStageExpression* NewExpression = Stage->ChangeSource<UDMMaterialStageExpressionTextureSample>();

		UDMMaterialStageInputValue* NewInputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
			Stage,
			0,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			EDMValueType::VT_Texture,
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
		);

		TextureValue = Cast<UDMMaterialValueTexture>(NewInputValue->GetValue());
	}

	if (TextureValue)
	{
		TextureValue->Modify();
		TextureValue->SetValue(InTexture);
	}
}

void SDMMaterialStage::OnMouseButtonDown_Left()
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerItem> SlotLayerItem = SlotLayerItemWeak.Pin();

	if (!SlotLayerItem.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerView> SlotLayerView = SlotLayerItem->GetSlotLayerView();

	if (!SlotLayerView.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = SlotLayerView->GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return;
	}

	SlotEditorWidget->TriggerStageSelectionChange(SlotLayerItem.ToSharedRef(), Stage);
}

void SDMMaterialStage::OnMouseButtonUp_Right()
{
	UDMMaterialStage* Stage = GetStage();

	if (!Stage)
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerItem> SlotLayerItem = SlotLayerItemWeak.Pin();

	if (!SlotLayerItem.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotLayerView> SlotLayerView = SlotLayerItem->GetSlotLayerView();

	if (!SlotLayerView.IsValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialSlotEditor> SlotEditorWidget = SlotLayerView->GetSlotEditorWidget();

	if (!SlotEditorWidget.IsValid())
	{
		return;
	}

	FSlateApplication& SlateApplication = FSlateApplication::Get();

	SlateApplication.PushMenu(
		SharedThis(this),
		FWidgetPath(),
		FDMMaterialStageMenus::GenerateStageMenu(SlotEditorWidget.ToSharedRef(), SharedThis(this)),
		SlateApplication.GetCursorPos(),
		FPopupTransitionEffect::ContextMenu
	);
}

#undef LOCTEXT_NAMESPACE
