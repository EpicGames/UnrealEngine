// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class SDMMaterialComponentPreview;
class SDMMaterialEditor;
class SDMMaterialSlotLayerItem;
class UDMMaterialStage;
class UTexture;
struct FPropertyChangedEvent;

class SDMMaterialStage : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SDMMaterialStage, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialStage) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialStage() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialSlotLayerItem>& InSlotLayerItem, UDMMaterialStage* InStage);

	TSharedPtr<SDMMaterialSlotLayerItem> GetSlotLayerView() const;

	UDMMaterialStage* GetStage() const;

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

protected:
	TWeakPtr<SDMMaterialSlotLayerItem> SlotLayerItemWeak;
	TWeakObjectPtr<UDMMaterialStage> StageWeak;
	TSharedPtr<SDMMaterialComponentPreview> PreviewImage;
	TSharedPtr<SDMMaterialComponentPreview> ToolTipImage;

	bool IsStageSelected() const;

	const FSlateBrush* GetBorderBrush() const;

	EVisibility GetDisabledOverlayVisibility() const;

	void OnSettingsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent);

	/** Drag and drop. */
	bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets);

	void OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets);

	void HandleDrop_Texture(UTexture* InTexture);

	void OnMouseButtonDown_Left();

	void OnMouseButtonUp_Right();
};
