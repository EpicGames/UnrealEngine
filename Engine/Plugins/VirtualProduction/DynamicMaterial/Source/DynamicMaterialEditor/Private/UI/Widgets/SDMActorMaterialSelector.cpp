// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/SDMActorMaterialSelector.h"

#include "AssetThumbnail.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/Array.h"
#include "DMObjectMaterialProperty.h"
#include "DynamicMaterialEditorStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Materials/MaterialInterface.h"
#include "Model/DynamicMaterialModel.h"
#include "Styling/StyleColors.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UI/Widgets/SDMMaterialDesigner.h"
#include "Utils/DMMaterialInstanceFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMActorMaterialSelector"

void SDMActorMaterialSelector::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SDMActorMaterialSelector::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget, AActor* InActor,
	TArray<FDMObjectMaterialProperty>&& InActorProperties)
{
	DesignerWidgetWeak = InDesignerWidget;
	ActorWeak = InActor;
	ActorProperties = MoveTemp(InActorProperties);

	SetCanTick(false);

	ChildSlot
	[
		SNew(SBox)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(10.f)
		[
			ActorProperties.IsEmpty()
				? CreateNoPropertiesLayout()
				: CreateSelectorLayout()
		]
	];
}

TSharedPtr<SDMMaterialDesigner> SDMActorMaterialSelector::GetDesignerWidget() const
{
	return DesignerWidgetWeak.Pin();
}

TSharedRef<SWidget> SDMActorMaterialSelector::CreateSelectorLayout()
{
	AActor* Actor = ActorWeak.Get();

	TSharedRef<SVerticalBox> ListOuter =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.Padding(0.0f, 20.0f, 0.0f, 20.0f)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "ActorNameBig")
			.Text(Actor ? FText::FromString(Actor->GetActorLabel()) : FText::GetEmpty())
		];

	const UObject* CurrentOuter = nullptr;

	for (int32 Index = 0; Index < ActorProperties.Num(); ++Index)
	{
		const FDMObjectMaterialProperty& ActorMaterialProperty = ActorProperties[Index];

		if (!ActorMaterialProperty.IsValid())
		{
			continue;
		}

		const UObject* Outer = ActorMaterialProperty.GetOuter();

		if (!IsValid(Outer))
		{
			continue;
		}

		if (Outer != CurrentOuter)
		{
			ListOuter->AddSlot()
				.AutoHeight()
				.Padding(0.f, CurrentOuter == nullptr ? 0.f : 10.f, 0.f, 5.f)
				[
					SNew(STextBlock)
						.TextStyle(FDynamicMaterialEditorStyle::Get(), "ComponentNameBig")
						.Text(FText::FromString(Outer->GetName()))
				];

			CurrentOuter = Outer;
		}

		ListOuter->AddSlot()
			.AutoHeight()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				CreateActorMaterialPropertyEntry(Index, Cast<UPrimitiveComponent>(ActorMaterialProperty.GetOuter()))
			];
	}

	if (ListOuter->NumSlots() == 1)
	{
		return CreateNoPropertiesLayout();
	}

	return SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)
		+ SScrollBox::Slot()
		[
			ListOuter
		];
}

TSharedRef<SWidget> SDMActorMaterialSelector::CreateNoPropertiesLayout()
{
	return SNew(STextBlock)
		.Justification(ETextJustify::Center)
		.AutoWrapText(true)
		.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
		.Text(LOCTEXT("NoMaterialSlot", "\n\nThe selected actor contains no primitive components with material slots."));
}

TSharedRef<SWidget> SDMActorMaterialSelector::CreateActorMaterialPropertyEntry(int32 InActorPropertyIndex, UPrimitiveComponent* InPrimitiveComponent)
{
	const FDMObjectMaterialProperty& ActorMaterialProperty = ActorProperties[InActorPropertyIndex];

	constexpr int32 ThumbnailSize = 48;

	UMaterialInterface* Material = InPrimitiveComponent
		? InPrimitiveComponent->GetMaterial(ActorMaterialProperty.GetIndex())
		: ActorMaterialProperty.GetMaterialInterface();

	TSharedRef<FAssetThumbnail> Thumbnail = MakeShared<FAssetThumbnail>(
		Material,
		ThumbnailSize,
		ThumbnailSize, 
		UThumbnailManager::Get().GetSharedThumbnailPool()
	);

	FAssetThumbnailConfig ThumbnailConfig;
	ThumbnailConfig.GenericThumbnailSize = ThumbnailSize;

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 5.f, 5.f, 5.f)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			Thumbnail->MakeThumbnailWidget(ThumbnailConfig)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 5.f, 0.5, 5.f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 5.f, 0.f, 5.f)
			[
				SNew(STextBlock)
				.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
				.Text(ActorMaterialProperty.GetPropertyName(true))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(2.f, 2.f, 2.f, 2.f))
				.OnClicked(this, &SDMActorMaterialSelector::OnCreateMaterialButtonClicked, InActorPropertyIndex)
				[
					SNew(STextBlock)
					.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
					.Text(LOCTEXT("CreateMaterial", "Create Material"))
				]
			]
		];
}

FReply SDMActorMaterialSelector::OnCreateMaterialButtonClicked(int32 InActorPropertyIndex)
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = DesignerWidgetWeak.Pin();

	if (!DesignerWidget.IsValid())
	{
		return FReply::Handled();
	}

	if (!ActorProperties.IsValidIndex(InActorPropertyIndex))
	{
		return FReply::Handled();
	}

	const FDMScopedUITransaction Transaction(LOCTEXT("CreateMaterialDesignerMaterialInActor", "Create Material Designer Material in Actor"));

	if (UObject* Outer = ActorProperties[InActorPropertyIndex].GetOuter())
	{
		Outer->Modify();
	}

	UDynamicMaterialModel* NewMaterialModel = UDMMaterialInstanceFunctionLibrary::CreateMaterialInObject(ActorProperties[InActorPropertyIndex]);

	if (NewMaterialModel)
	{
		DesignerWidget->OnObjectMaterialPropertySelected(ActorProperties[InActorPropertyIndex]);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
