// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class SDMMaterialDesigner;
struct FDMObjectMaterialProperty;

class SDMActorMaterialSelector : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET(SDMActorMaterialSelector, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMActorMaterialSelector) {}
	SLATE_END_ARGS()

	virtual ~SDMActorMaterialSelector() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget, AActor* InActor,
		TArray<FDMObjectMaterialProperty>&& InActorProperties);

	TSharedPtr<SDMMaterialDesigner> GetDesignerWidget() const;

protected:
	TWeakPtr<SDMMaterialDesigner> DesignerWidgetWeak;
	TWeakObjectPtr<AActor> ActorWeak;
	TArray<FDMObjectMaterialProperty> ActorProperties;

	TSharedRef<SWidget> CreateSelectorLayout();

	TSharedRef<SWidget> CreateNoPropertiesLayout();

	TSharedRef<SWidget> CreateActorMaterialPropertyEntry(int32 InActorPropertyIndex, UPrimitiveComponent* InPrimitiveComponent = nullptr);

	FReply OnCreateMaterialButtonClicked(int32 InActorPropertyIndex);
};
