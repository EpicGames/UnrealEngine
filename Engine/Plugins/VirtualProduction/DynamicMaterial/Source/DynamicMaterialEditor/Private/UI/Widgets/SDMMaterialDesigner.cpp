// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/SDMMaterialDesigner.h"

#include "Containers/Set.h"
#include "DMTextureSet.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialModule.h"
#include "Editor/SDMMaterialComponentEditor.h"
#include "Editor/SDMMaterialSlotEditor.h"
#include "GameFramework/Actor.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "SAssetDropTarget.h"
#include "UI/Utils/DMDropTargetPrivateSetter.h"
#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_Left.h"
#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopSlim.h"
#include "UI/Widgets/Editor/EditorLayouts/SDMMaterialEditor_TopVertical.h"
#include "UI/Widgets/SDMActorMaterialSelector.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/SDMMaterialSelectPrompt.h"
#include "UI/Widgets/SDMMaterialWizard.h"
#include "Utils/DMMaterialInstanceFunctionLibrary.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Widgets/SNullWidget.h"

void SDMMaterialDesigner::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialDesigner::~SDMMaterialDesigner()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Settings->GetOnSettingsChanged().RemoveAll(this);
	}
}

void SDMMaterialDesigner::Construct(const FArguments& InArgs)
{
	SetCanTick(true);

	ContentSlot = TDMWidgetSlot<SWidget>(SharedThis(this), 0, SNullWidget::NullWidget);

	SetSelectPromptView();

	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Settings->GetOnSettingsChanged().AddSP(this, &SDMMaterialDesigner::OnSettingsChanged);
	}
}

bool SDMMaterialDesigner::OpenMaterialModelBase(UDynamicMaterialModelBase* InMaterialModelBase)
{
	if (IsValid(InMaterialModelBase) && UDMMaterialModelFunctionLibrary::IsModelValid(InMaterialModelBase))
	{
		OpenMaterialModelBase_Internal(InMaterialModelBase);
		return true;
	}

	return false;
}

bool SDMMaterialDesigner::OpenMaterialInstance(UDynamicMaterialInstance* InMaterialInstance)
{
	if (IsValid(InMaterialInstance))
	{
		if (UDynamicMaterialModelBase* MaterialModelBase = InMaterialInstance->GetMaterialModelBase())
		{
			OpenMaterialModelBase(MaterialModelBase);
			return true;
		}
	}

	return false;
}

bool SDMMaterialDesigner::OpenObjectMaterialProperty(const FDMObjectMaterialProperty& InObjectMaterialProperty)
{
	if (InObjectMaterialProperty.IsValid())
	{
		OpenObjectMaterialProperty_Internal(InObjectMaterialProperty);
		return true;
	}

	return false;
}

bool SDMMaterialDesigner::OpenActor(AActor* InActor)
{
	if (IsValid(InActor))
	{
		OpenActor_Internal(InActor);
		return true;
	}

	return false;
}

void SDMMaterialDesigner::ShowSelectPrompt()
{
	SetSelectPromptView();	
}

void SDMMaterialDesigner::Empty()
{
	SetEmptyView();
}

void SDMMaterialDesigner::OnMaterialModelBaseSelected(UDynamicMaterialModelBase* InMaterialModelBase)
{
	if (ShouldFollowSelection())
	{
		OpenMaterialModelBase(InMaterialModelBase);
	}
}

void SDMMaterialDesigner::OnMaterialInstanceSelected(UDynamicMaterialInstance* InMaterialInstance)
{
	if (ShouldFollowSelection())
	{
		OpenMaterialModelBase(InMaterialInstance->GetMaterialModel());
	}
}

void SDMMaterialDesigner::OnObjectMaterialPropertySelected(const FDMObjectMaterialProperty& InObjectMaterialProperty)
{
	if (ShouldFollowSelection())
	{
		OpenObjectMaterialProperty(InObjectMaterialProperty);
	}
}

void SDMMaterialDesigner::OnActorSelected(AActor* InActor)
{
	if (ShouldFollowSelection())
	{
		OpenActor(InActor);
	}
}

UDynamicMaterialModelBase* SDMMaterialDesigner::GetOriginalMaterialModelBase() const
{
	if (!Content.IsValid())
	{
		return nullptr;
	}

	if (Content->GetWidgetClass().GetWidgetType() == SDMMaterialEditor::StaticWidgetClass().GetWidgetType())
	{
		return StaticCastSharedPtr<SDMMaterialEditor>(Content)->GetOriginalMaterialModelBase();
	}

	if (Content->GetWidgetClass().GetWidgetType() == SDMMaterialWizard::StaticWidgetClass().GetWidgetType())
	{
		return StaticCastSharedPtr<SDMMaterialWizard>(Content)->GetMaterialModel();
	}

	return nullptr;
}

void SDMMaterialDesigner::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (Content.IsValid() && Content->GetWidgetClass().GetWidgetType() == SDMMaterialEditor::StaticWidgetClass().GetWidgetType())
	{
		StaticCastSharedPtr<SDMMaterialEditor>(Content)->Validate();
	}

	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);
}

void SDMMaterialDesigner::OpenMaterialModelBase_Internal(UDynamicMaterialModelBase* InMaterialModelBase)
{
	if (NeedsWizard(InMaterialModelBase))
	{
		SetWizardView(Cast<UDynamicMaterialModel>(InMaterialModelBase));
		return;
	}

	SetEditorView(InMaterialModelBase);
}

void SDMMaterialDesigner::OpenObjectMaterialProperty_Internal(const FDMObjectMaterialProperty& InObjectMaterialProperty)
{
	if (UDynamicMaterialModelBase* ModelBase = InObjectMaterialProperty.GetMaterialModelBase())
	{
		if (NeedsWizard(ModelBase))
		{
			SetWizardView(InObjectMaterialProperty);
		}
		else
		{
			SetEditorView(InObjectMaterialProperty);
		}
	}
	else
	{
		SetWizardView(InObjectMaterialProperty);
	}
}

void SDMMaterialDesigner::OpenActor_Internal(AActor* InActor)
{
	SetWidget(SNullWidget::NullWidget, /* Include Drop Target */ true);

	TArray<FDMObjectMaterialProperty> ActorProperties = UDMMaterialInstanceFunctionLibrary::GetActorMaterialProperties(InActor);

	if (ActorProperties.IsEmpty())
	{
		SetSelectPromptView();
		return;
	}

	for (const FDMObjectMaterialProperty& MaterialProperty : ActorProperties)
	{
		if (MaterialProperty.GetMaterialModelBase())
		{
			OpenObjectMaterialProperty(MaterialProperty);
			return;
		}
	}

	if (ActorProperties.Num() == 1)
	{
		OpenObjectMaterialProperty_Internal(ActorProperties[0]);
	}
	else
	{
		SetMaterialSelectorView(InActor, MoveTemp(ActorProperties));
	}
}

void SDMMaterialDesigner::SetEmptyView()
{
	SetWidget(SNullWidget::NullWidget, /* Include drop target */ true);
}

void SDMMaterialDesigner::SetSelectPromptView()
{
	SetWidget(SNew(SDMMaterialSelectPrompt), /* Include Drop Target */ true);
}

void SDMMaterialDesigner::SetMaterialSelectorView(AActor* InActor, TArray<FDMObjectMaterialProperty>&& InActorProperties)
{
	TSharedRef<SDMActorMaterialSelector> Selector = SNew(
		SDMActorMaterialSelector, 
		SharedThis(this), 
		InActor, 
		Forward<TArray<FDMObjectMaterialProperty>>(InActorProperties)
	);

	SetWidget(Selector, /* Include Drop Target */ true);
}

void SDMMaterialDesigner::SetWizardView(UDynamicMaterialModel* InMaterialModel)
{
	TSharedRef<SDMMaterialWizard> Wizard = SNew(SDMMaterialWizard, SharedThis(this))
		.MaterialModel(InMaterialModel);

	SetWidget(Wizard, /* Include Drop Target */ true);
}

void SDMMaterialDesigner::SetWizardView(const FDMObjectMaterialProperty& InObjectMaterialProperty)
{
	TSharedRef<SDMMaterialWizard> Wizard = SNew(SDMMaterialWizard, SharedThis(this))
		.MaterialProperty(InObjectMaterialProperty);

	SetWidget(Wizard, /* Include Drop Target */ true);
}

void SDMMaterialDesigner::SetEditorView(UDynamicMaterialModelBase* InMaterialModelBase)
{
	EDMMaterialEditorLayout Layout = EDMMaterialEditorLayout::Left;

	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Layout = Settings->Layout;
	}

	SetEditorLayout(Layout, InMaterialModelBase, nullptr);
}

void SDMMaterialDesigner::SetEditorView(const FDMObjectMaterialProperty& InObjectMaterialProperty)
{
	EDMMaterialEditorLayout Layout = EDMMaterialEditorLayout::Left;

	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Layout = Settings->Layout;
	}

	SetEditorLayout(Layout, InObjectMaterialProperty, nullptr);
}

void SDMMaterialDesigner::SetWidget(const TSharedRef<SWidget>& InWidget, bool bInIncludeAssetDropTarget)
{
	Content = InWidget;

	if (!bInIncludeAssetDropTarget)
	{
		ContentSlot << InWidget;
	}
	else
	{
		TSharedRef<SAssetDropTarget> DropTarget = SNew(SAssetDropTarget)
			.OnAreAssetsAcceptableForDrop(this, &SDMMaterialDesigner::OnAssetDraggedOver)
			.OnAssetsDropped(this, &SDMMaterialDesigner::OnAssetsDropped)
			.bSupportsMultiDrop(true)
			[
				InWidget
			];

		using namespace UE::DynamicMaterialEditor::Private;
		DropTarget::SetInvalidColor(&DropTarget.Get(), FStyleColors::Transparent);

		ContentSlot << DropTarget;
	}
}

bool SDMMaterialDesigner::IsFollowingSelection()
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		return Settings->bFollowSelection;
	}

	return false;
}

bool SDMMaterialDesigner::NeedsWizard(UDynamicMaterialModelBase* InMaterialModelBase) const
{
	if (UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(InMaterialModelBase))
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
		{
			if (EditorOnlyData->NeedsWizard())
			{
				return true;
			}
		}
	}

	return false;
}

bool SDMMaterialDesigner::ShouldFollowSelection() const
{
	return IsFollowingSelection() || !GetOriginalMaterialModelBase();
}

bool SDMMaterialDesigner::OnAssetDraggedOver(TArrayView<FAssetData> InAssets)
{
	TArray<UClass*> AllowedClasses = {
		AActor::StaticClass(),
		UDynamicMaterialModelBase::StaticClass(),
		UDynamicMaterialInstance::StaticClass()
	};

	const bool bIsEditor = Content.IsValid()
		&& Content->GetWidgetClass().GetWidgetType() == SDMMaterialEditor::StaticWidgetClass().GetWidgetType();

	const bool bIsWizard = Content.IsValid()
		&& Content->GetWidgetClass().GetWidgetType() == SDMMaterialWizard::StaticWidgetClass().GetWidgetType();

	if (bIsEditor || bIsWizard)
	{
		AllowedClasses.Add(UDMTextureSet::StaticClass());
	}

	TArray<FAssetData> DroppedTextures;

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		for (UClass* AllowedClass : AllowedClasses)
		{
			if (AssetClass->IsChildOf(AllowedClass))
			{
				return true;
			}
		}

		if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			DroppedTextures.Add(Asset);
		}
	}

	if ((bIsEditor || bIsWizard) && DroppedTextures.Num() > 1)
	{
		return true;
	}

	return false;
}

void SDMMaterialDesigner::OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets)
{
	TArray<FAssetData> DroppedTextures;

	const bool bIsEditor = Content.IsValid()
		&& Content->GetWidgetClass().GetWidgetType() == SDMMaterialEditor::StaticWidgetClass().GetWidgetType();

	const bool bIsWizard = Content.IsValid()
		&& Content->GetWidgetClass().GetWidgetType() == SDMMaterialWizard::StaticWidgetClass().GetWidgetType();

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass)
		{
			continue;
		}

		if (AssetClass->IsChildOf(AActor::StaticClass()))
		{
			if (OpenActor(Cast<AActor>(Asset.GetAsset())))
			{
				return;
			}
		}
		else if (AssetClass->IsChildOf(UDynamicMaterialModelBase::StaticClass()))
		{
			if (OpenMaterialModelBase(Cast<UDynamicMaterialModelBase>(Asset.GetAsset())))
			{
				return;
			}
		}
		else if (AssetClass->IsChildOf(UDynamicMaterialInstance::StaticClass()))
		{
			if (OpenMaterialInstance(Cast<UDynamicMaterialInstance>(Asset.GetAsset())))
			{
				return;
			}
		}
		else if (AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			DroppedTextures.Add(Asset);
		}
		else if (AssetClass->IsChildOf(UDMTextureSet::StaticClass()))
		{
			if (bIsEditor)
			{
				StaticCastSharedPtr<SDMMaterialEditor>(Content)->HandleDrop_TextureSet(Cast<UDMTextureSet>(Asset.GetAsset()));
				return;
			}

			if (bIsWizard)
			{
				StaticCastSharedPtr<SDMMaterialWizard>(Content)->HandleDrop_TextureSet(Cast<UDMTextureSet>(Asset.GetAsset()));
				return;
			}
		}
	}

	if (DroppedTextures.Num() > 1)
	{
		if (bIsEditor)
		{
			StaticCastSharedPtr<SDMMaterialEditor>(Content)->HandleDrop_CreateTextureSet(DroppedTextures);
		}
		else if (bIsWizard)
		{
			StaticCastSharedPtr<SDMMaterialWizard>(Content)->HandleDrop_CreateTextureSet(DroppedTextures);
		}
	}
}

void SDMMaterialDesigner::OnSettingsChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!Content.IsValid() || Content->GetWidgetClass().GetWidgetType() != SDMMaterialEditor::StaticWidgetClass().GetWidgetType())
	{
		return;
	}

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, Layout))
	{
		OnLayoutChanged();
	}
}

void SDMMaterialDesigner::OnLayoutChanged()
{
	UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>();

	if (!Settings)
	{
		return;
	}

	// Already assured
	TSharedPtr<SDMMaterialEditor> CurrentEditor = StaticCastSharedRef<SDMMaterialEditor>(Content.ToSharedRef());

	UDynamicMaterialModelBase* OriginalMaterialModelBase = CurrentEditor->GetOriginalMaterialModelBase();
	UDynamicMaterialModelBase* PreviewMaterialModelBase = CurrentEditor->GetPreviewMaterialModelBase();
	const FDMObjectMaterialProperty* MaterialObjectProperty = CurrentEditor->GetMaterialObjectProperty();
	EDMMaterialEditorMode EditorMode = CurrentEditor->GetEditMode();
	EDMMaterialPropertyType SelectedProperty = CurrentEditor->GetSelectedPropertyType();
	UDMMaterialComponent* EditedComponent = nullptr;

	if (EditorMode == EDMMaterialEditorMode::EditSlot)
	{
		if (TSharedPtr<SDMMaterialComponentEditor> ComponentEditorWidget = CurrentEditor->GetComponentEditorWidget())
		{
			EditedComponent = ComponentEditorWidget->GetComponent();
		}
	}

	if (MaterialObjectProperty)
	{
		if (!SetEditorLayout(Settings->Layout, *MaterialObjectProperty, PreviewMaterialModelBase))
		{
			return;
		}
	}
	else
	{
		if (!SetEditorLayout(Settings->Layout, OriginalMaterialModelBase, PreviewMaterialModelBase))
		{
			return;
		}
	}

	TSharedRef<SDMMaterialEditor> NewEditor = StaticCastSharedRef<SDMMaterialEditor>(Content.ToSharedRef());

	switch (EditorMode)
	{
		default:
		case EDMMaterialEditorMode::GlobalSettings:
			NewEditor->EditGlobalSettings();
			break;

		case EDMMaterialEditorMode::Properties:
			NewEditor->EditProperties();
			break;

		case EDMMaterialEditorMode::EditSlot:
			NewEditor->SelectProperty(SelectedProperty);

			if (EditedComponent)
			{
				NewEditor->EditComponent(EditedComponent);
			}

			break;
	}
}

bool SDMMaterialDesigner::SetEditorLayout(EDMMaterialEditorLayout InLayout, UDynamicMaterialModelBase* InMaterialModelBase,
	UDynamicMaterialModelBase* InCurrentPreviewMaterial)
{
	TSharedPtr<SDMMaterialEditor> NewEditor;

	switch (InLayout)
	{
		case EDMMaterialEditorLayout::Left:
			NewEditor = SNew(SDMMaterialEditor_Left, SharedThis(this))
				.MaterialModelBase(InMaterialModelBase)
				.PreviewMaterialModelBase(InCurrentPreviewMaterial);
			break;

		case EDMMaterialEditorLayout::Top:
			NewEditor = SNew(SDMMaterialEditor_TopVertical, SharedThis(this))
				.MaterialModelBase(InMaterialModelBase)
				.PreviewMaterialModelBase(InCurrentPreviewMaterial);
			break;

		case EDMMaterialEditorLayout::TopSlim:
			NewEditor = SNew(SDMMaterialEditor_TopSlim, SharedThis(this))
				.MaterialModelBase(InMaterialModelBase)
				.PreviewMaterialModelBase(InCurrentPreviewMaterial);
			break;

		default:
			break;
	}

	if (!NewEditor.IsValid())
	{
		return false;
	}

	SetWidget(NewEditor.ToSharedRef(), /* Include Drop Target */ true);

	return true;
}

bool SDMMaterialDesigner::SetEditorLayout(EDMMaterialEditorLayout InLayout, const FDMObjectMaterialProperty& InObjectMaterialProperty,
	UDynamicMaterialModelBase* InCurrentPreviewMaterial)
{
	TSharedPtr<SDMMaterialEditor> NewEditor;

	switch (InLayout)
	{
		case EDMMaterialEditorLayout::Left:
			NewEditor = SNew(SDMMaterialEditor_Left, SharedThis(this))
				.MaterialProperty(InObjectMaterialProperty)
				.PreviewMaterialModelBase(InCurrentPreviewMaterial);
			break;

		case EDMMaterialEditorLayout::Top:
			NewEditor = SNew(SDMMaterialEditor_TopVertical, SharedThis(this))
				.MaterialProperty(InObjectMaterialProperty)
				.PreviewMaterialModelBase(InCurrentPreviewMaterial);
			break;

		case EDMMaterialEditorLayout::TopSlim:
			NewEditor = SNew(SDMMaterialEditor_TopSlim, SharedThis(this))
				.MaterialProperty(InObjectMaterialProperty)
				.PreviewMaterialModelBase(InCurrentPreviewMaterial);
			break;

		default:
			break;
	}

	if (!NewEditor.IsValid())
	{
		return false;
	}

	SetWidget(NewEditor.ToSharedRef(), /* Include Drop Target */ true);

	return true;
}
