// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UI/Utils/DMWidgetSlot.h"

class AActor;
class SBox;
class SDMActorMaterialSelector;
class SDMMaterialEditor;
class SDMMaterialWizard;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
class UDynamicMaterialModelBase;
enum class EDMMaterialEditorLayout : uint8;
struct FDMObjectMaterialProperty;
struct FPropertyChangedEvent;

class SDMMaterialDesigner : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SDMMaterialDesigner, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialDesigner) {}
	SLATE_END_ARGS()

public:
	static bool IsFollowingSelection();

	virtual ~SDMMaterialDesigner() override;

	void Construct(const FArguments& InArgs);

	/** Open specific types of objects. */
	bool OpenMaterialModelBase(UDynamicMaterialModelBase* InMaterialModelBase);

	bool OpenMaterialInstance(UDynamicMaterialInstance* InMaterialInstance);

	bool OpenObjectMaterialProperty(const FDMObjectMaterialProperty& InObjectMaterialProperty);

	bool OpenActor(AActor* InActor);

	/** Displays a "select something" message. */
	void ShowSelectPrompt();

	/** Displays nothing. */
	void Empty();

	/** Events based on editor selection. */
	void OnMaterialModelBaseSelected(UDynamicMaterialModelBase* InMaterialModelBase);

	void OnMaterialInstanceSelected(UDynamicMaterialInstance* InMaterialInstance);

	void OnObjectMaterialPropertySelected(const FDMObjectMaterialProperty& InObjectMaterialProperty);

	void OnActorSelected(AActor* InActor);

	/** Getters */
	UDynamicMaterialModelBase* GetOriginalMaterialModelBase() const;

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	TDMWidgetSlot<SWidget> ContentSlot;
	TSharedPtr<SWidget> Content;

	/** Opens assets in specific widget modes. */
	void OpenMaterialModelBase_Internal(UDynamicMaterialModelBase* InMaterialModelBase);

	void OpenObjectMaterialProperty_Internal(const FDMObjectMaterialProperty& InObjectMaterialProperty);

	void OpenActor_Internal(AActor* InActor);

	/** Widget modes. */
	void SetEmptyView();

	void SetSelectPromptView();

	void SetMaterialSelectorView(AActor* InActor, TArray<FDMObjectMaterialProperty>&& InActorProperties);

	void SetWizardView(UDynamicMaterialModel* InMaterialModel);

	void SetWizardView(const FDMObjectMaterialProperty& InObjectMaterialProperty);

	void SetEditorView(UDynamicMaterialModelBase* InMaterialModelBase);

	void SetEditorView(const FDMObjectMaterialProperty& InObjectMaterialProperty);

	/** Utility methods. */
	void SetWidget(const TSharedRef<SWidget>& InWidget, bool bInIncludeAssetDropTarget);

	bool NeedsWizard(UDynamicMaterialModelBase* InMaterialModelBase) const;

	bool ShouldFollowSelection() const;

	/** Drag and drop. */
	bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets);

	void OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets);

	/** Events */
	void OnSettingsChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	void OnLayoutChanged();

	bool SetEditorLayout(EDMMaterialEditorLayout InLayout, UDynamicMaterialModelBase* InMaterialModelBase, 
		UDynamicMaterialModelBase* InCurrentPreviewMaterial);

	bool SetEditorLayout(EDMMaterialEditorLayout InLayout, const FDMObjectMaterialProperty& InObjectMaterialProperty,
		UDynamicMaterialModelBase* InCurrentPreviewMaterial);
};
