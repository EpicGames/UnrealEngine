// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/SDMMaterialEditor.h"

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "DMTextureSetBlueprintFunctionLibrary.h"
#include "DynamicMaterialEditorCommands.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialModule.h"
#include "Editor/SlotEditor/SDMMaterialSlotLayerView.h"
#include "Engine/Texture.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Misc/MessageDialog.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "PackageTools.h"
#include "Styling/SlateIconFinder.h"
#include "UI/Utils/DMEditorSelectionContext.h"
#include "UI/Utils/DMPreviewMaterialManager.h"
#include "UI/Widgets/Editor/SDMMaterialComponentEditor.h"
#include "UI/Widgets/Editor/SDMMaterialGlobalSettingsEditor.h"
#include "UI/Widgets/Editor/SDMMaterialPreview.h"
#include "UI/Widgets/Editor/SDMMaterialProperties.h"
#include "UI/Widgets/Editor/SDMMaterialPropertySelector.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UI/Widgets/Editor/SDMStatusBar.h"
#include "UI/Widgets/Editor/SDMToolBar.h"
#include "UI/Widgets/SDMMaterialDesigner.h"
#include "UObject/Linker.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMTextureSetFunctionLibrary.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialEditor"

/**
 * This is used to track a key, similar to how modifier keys are tracked by the engine...
 * because non-modifier keys are not tracked.
 */
class FDMKeyTracker : public IInputProcessor
{
public:
	FDMKeyTracker(const FKey& InTrackedKey)
		: TrackedKey(InTrackedKey)
		, bKeyDown(false)
	{
		
	}

	const FKey& GetTrackedKey() const
	{
		return TrackedKey;
	}

	bool IsKeyDown() const
	{
		return bKeyDown;
	}

	//~ Begin IInputProcessor
	virtual void Tick(const float InDeltaTime, FSlateApplication& InSlateApp, TSharedRef<ICursor> InCursor) override
	{		
	}

	virtual bool HandleKeyDownEvent(FSlateApplication& InSlateApp, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == TrackedKey)
		{
			bKeyDown = true;
		}

		return false;
	}

	/** Key up input */
	virtual bool HandleKeyUpEvent(FSlateApplication& InSlateApp, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == TrackedKey)
		{
			bKeyDown = false;
		}

		return false;
	}

	virtual const TCHAR* GetDebugName() const override
	{
		return TEXT("FDMKeyTracker");
	}
	//~ End IInputProcessor

private:
	const FKey& TrackedKey;
	bool bKeyDown;
};

void SDMMaterialEditor::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialEditor::SDMMaterialEditor()
	: SplitterSlot(nullptr)
	, CommandList(MakeShared<FUICommandList>())
	, PreviewMaterialManager(MakeShared<FDMPreviewMaterialManager>())
	, bSkipApplyOnCompile(false)
{
	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->ShouldAutomaticallyApplyToSourceOnPreviewCompile())
		{
			// If we are automatically applying, we need to skip the initial compile event.
			bSkipApplyOnCompile = true;
		}
	}
}

SDMMaterialEditor::~SDMMaterialEditor()
{
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	CloseMaterialPreviewTab();
	DestroyMaterialPreviewToolTip();

	if (KeyTracker_V.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(KeyTracker_V.ToSharedRef());
	}

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	UnbindEditorOnlyDataUpdate();

	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Settings->GetOnSettingsChanged().RemoveAll(this);
	}
}

void SDMMaterialEditor::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	DesignerWidgetWeak = InDesignerWidget;
	
	SelectionContext.EditorMode = EDMMaterialEditorMode::GlobalSettings;
	SelectionContext.Property = EDMMaterialPropertyType::None;

	SelectionContext.PageHistory.Add(FDMMaterialEditorPage::GlobalSettings);
	SelectionContext.PageHistoryCount = 1;

	// Some small number to get us going
	SelectionContext.PageHistory.Reserve(20);

	SetCanTick(false);

	ContentSlot = TDMWidgetSlot<SWidget>(SharedThis(this), 0, SNullWidget::NullWidget);

	if (InArgs._MaterialProperty.IsSet())
	{
		SetObjectMaterialProperty(InArgs._MaterialProperty.GetValue(), InArgs._PreviewMaterialModelBase);
	}
	else if (IsValid(InArgs._MaterialModelBase))
	{
		SetMaterialModelBase(InArgs._MaterialModelBase, InArgs._PreviewMaterialModelBase);
	}
	else
	{
		ensureMsgf(false, TEXT("No valid material model passed to Material DesignerWidget Editor."));
	}

	FCoreDelegates::OnEnginePreExit.AddSP(this, &SDMMaterialEditor::OnEnginePreExit);

	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Settings->GetOnSettingsChanged().AddSP(this, &SDMMaterialEditor::OnSettingsChanged);
	}

	KeyTracker_V = MakeShared<FDMKeyTracker>(EKeys::V);
	FSlateApplication::Get().RegisterInputPreProcessor(KeyTracker_V);
}

TSharedPtr<SDMMaterialDesigner> SDMMaterialEditor::GetDesignerWidget() const
{
	return DesignerWidgetWeak.Pin();
}

UDynamicMaterialModelBase* SDMMaterialEditor::GetOriginalMaterialModelBase() const
{
	return OriginalMaterialModelBaseWeak.Get();
}

UDynamicMaterialModelBase* SDMMaterialEditor::GetPreviewMaterialModelBase() const
{
	return PreviewMaterialModelBase.Get();
}

void SDMMaterialEditor::SetMaterialModelBase(UDynamicMaterialModelBase* InMaterialModelBase, UDynamicMaterialModelBase* InPreviewMaterialModelBase)
{
	OriginalMaterialModelBaseWeak = InMaterialModelBase;

	if (!InMaterialModelBase)
	{
		PreviewMaterialModelBase = nullptr;
	}
	else if (InPreviewMaterialModelBase)
	{
		PreviewMaterialModelBase = InPreviewMaterialModelBase;
		PreviewMaterialModelBase->MarkOriginalUpdated();
	}
	else
	{
		PreviewMaterialModelBase = UDMMaterialModelFunctionLibrary::CreatePreviewModel(InMaterialModelBase);

		UDynamicMaterialInstanceFactory* Factory = NewObject<UDynamicMaterialInstanceFactory>();

		UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(Factory->FactoryCreateNew(
			UDynamicMaterialInstance::StaticClass(),
			GetTransientPackage(),
			NAME_None,
			RF_Transactional,
			PreviewMaterialModelBase,
			nullptr
		));

		PreviewMaterialInstance = NewInstance;

		PreviewMaterialModelBase->MarkOriginalUpdated();
	}

	EditGlobalSettings();

	CreateLayout();

	UnbindEditorOnlyDataUpdate();

	if (PreviewMaterialModelBase)
	{
		BindEditorOnlyDataUpdate(PreviewMaterialModelBase);
	}
}

UDynamicMaterialModel* SDMMaterialEditor::GetPreviewMaterialModel() const
{
	if (UDynamicMaterialModelBase* MaterialModelBase = GetPreviewMaterialModelBase())
	{
		return MaterialModelBase->ResolveMaterialModel();
	}
	
	return nullptr;
}

bool SDMMaterialEditor::IsDynamicModel() const
{
	return !!Cast<UDynamicMaterialModelDynamic>(GetPreviewMaterialModelBase());
}

const FDMObjectMaterialProperty* SDMMaterialEditor::GetMaterialObjectProperty() const
{
	if (ObjectMaterialPropertyOpt.IsSet())
	{
		return &ObjectMaterialPropertyOpt.GetValue();
	}

	return nullptr;
}

void SDMMaterialEditor::SetObjectMaterialProperty(const FDMObjectMaterialProperty& InObjectProperty, UDynamicMaterialModelBase* InPreviewMaterialModelBase)
{
	UDynamicMaterialModelBase* MaterialModelBase = InObjectProperty.GetMaterialModelBase();

	if (!ensureMsgf(MaterialModelBase, TEXT("Invalid object material property value.")))
	{
		ClearSlots();
		return;
	}

	ObjectMaterialPropertyOpt = InObjectProperty;
	SetMaterialModelBase(MaterialModelBase, InPreviewMaterialModelBase);
}

AActor* SDMMaterialEditor::GetMaterialActor() const
{
	if (ObjectMaterialPropertyOpt.IsSet())
	{
		return ObjectMaterialPropertyOpt.GetValue().GetTypedOuter<AActor>();
	}

	return nullptr;
}

UDMMaterialComponent* SDMMaterialEditor::GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const
{
	if (!PreviewMaterialModelBase)
	{
		return nullptr;
	}

	UDynamicMaterialModelBase* OriginalMaterialModelBase = GetOriginalMaterialModelBase();

	if (!OriginalMaterialModelBase)
	{
		return nullptr;
	}

	const FString RelativePath = InPreviewComponent->GetPathName(PreviewMaterialModelBase);

	return UDMMaterialModelFunctionLibrary::FindSubobject<UDMMaterialComponent>(OriginalMaterialModelBase, *RelativePath);
}

EDMMaterialEditorMode SDMMaterialEditor::GetEditMode() const
{
	return SelectionContext.EditorMode;
}

void SDMMaterialEditor::SetMaterialActor(AActor* InActor)
{
	if (GetMaterialActor() == InActor)
	{
		return;
	}

	TSharedRef<SDMToolBar> NewToolBar = SNew(SDMToolBar, SharedThis(this), FDMObjectMaterialProperty(InActor, 0));

	ToolBarSlot << NewToolBar;
}

TSharedPtr<SDMMaterialSlotEditor> SDMMaterialEditor::GetSlotEditorWidget() const
{
	return &SlotEditorSlot;
}

TSharedPtr<SDMMaterialComponentEditor> SDMMaterialEditor::GetComponentEditorWidget() const
{
	return &ComponentEditorSlot;
}

UDMMaterialSlot* SDMMaterialEditor::SlotSelectedSlot() const
{
	return SelectionContext.Slot.Get();
}

UDMMaterialComponent* SDMMaterialEditor::GetSelectedComponent() const
{
	return SelectionContext.Component.Get();
}

EDMMaterialPropertyType SDMMaterialEditor::GetSelectedPropertyType() const
{
	return SelectionContext.Property;
}

void SDMMaterialEditor::SelectProperty(EDMMaterialPropertyType InProperty, bool bInForceRefresh)
{
	if (SelectionContext.EditorMode == EDMMaterialEditorMode::EditSlot && SelectionContext.Property == InProperty && !bInForceRefresh)
	{
		return;
	}

	SelectionContext.bModeChanged = bInForceRefresh || (SelectionContext.EditorMode != EDMMaterialEditorMode::EditSlot);

	SelectProperty_Impl(InProperty);

	PageHistoryAdd({EDMMaterialEditorMode::EditSlot, InProperty});

	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(GetPreviewMaterialModelBase()))
	{
		UDMMaterialSlot* CurrentSlot = SelectionContext.Slot.Get();

		if (!CurrentSlot || !EditorOnlyData->GetMaterialPropertiesForSlot(CurrentSlot).Contains(InProperty))
		{
			if (UDMMaterialSlot* PropertySlot = EditorOnlyData->GetSlotForMaterialProperty(InProperty))
			{
				EditSlot(PropertySlot);
			}
		}
	}
}

const TSharedRef<FUICommandList>& SDMMaterialEditor::GetCommandList() const
{
	return CommandList;
}

TSharedRef<FDMPreviewMaterialManager> SDMMaterialEditor::GetPreviewMaterialManager() const
{
	return PreviewMaterialManager;
}

void SDMMaterialEditor::EditSlot(UDMMaterialSlot* InSlot, bool bInForceRefresh)
{
	if (!bInForceRefresh && SlotEditorSlot.IsValid() && SlotEditorSlot->GetSlot() == InSlot)
	{
		return;
	}

	SelectionContext.bModeChanged = bInForceRefresh || (SelectionContext.EditorMode != EDMMaterialEditorMode::EditSlot);

	EditSlot_Impl(InSlot);

	SelectionContext.Component.Reset();

	if (!InSlot)
	{
		return;
	}

	UDMMaterialLayerObject* SelectedLayer = SelectionContext.Layer.Get();

	if (SelectedLayer && SelectedLayer->GetSlot() == InSlot)
	{
		return;
	}

	for (const TObjectPtr<UDMMaterialLayerObject>& Layer : InSlot->GetLayers())
	{
		if (UDMMaterialStage* Stage = Layer->GetFirstValidStage(EDMMaterialLayerStage::All))
		{
			EditComponent(Stage);
			break;
		}
	}
}

void SDMMaterialEditor::EditComponent(UDMMaterialComponent* InComponent, bool bInForceRefresh)
{
	if (!bInForceRefresh && ComponentEditorSlot.IsValid() && ComponentEditorSlot->GetComponent() == InComponent)
	{
		return;
	}

	EditComponent_Impl(InComponent);
}

void SDMMaterialEditor::EditGlobalSettings(bool bInForceRefresh)
{
	if (SelectionContext.EditorMode == EDMMaterialEditorMode::GlobalSettings && !bInForceRefresh)
	{
		return;
	}

	SelectionContext.bModeChanged = bInForceRefresh || (SelectionContext.EditorMode != EDMMaterialEditorMode::GlobalSettings);

	PageHistoryAdd(FDMMaterialEditorPage::GlobalSettings);

	EditGlobalSettings_Impl();
}

void SDMMaterialEditor::EditProperties(bool bInForceRefresh)
{
	if (SelectionContext.EditorMode == EDMMaterialEditorMode::Properties && !bInForceRefresh)
	{
		return;
	}

	SelectionContext.bModeChanged = bInForceRefresh || (SelectionContext.EditorMode != EDMMaterialEditorMode::Properties);

	PageHistoryAdd(FDMMaterialEditorPage::Properties);

	EditProperties_Impl();
}

void SDMMaterialEditor::OnLayerSelected(const TSharedRef<SDMMaterialSlotLayerView>& InSlotView, 
	const TSharedPtr<FDMMaterialLayerReference>& InLayerView)
{
	SelectionContext.Layer = InLayerView.IsValid() ? InLayerView->GetLayer() : nullptr;
}

void SDMMaterialEditor::OnStageSelected(const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItem, UDMMaterialStage* InStage)
{
	EditComponent(InStage);
}

void SDMMaterialEditor::OnEffectSelected(const TSharedRef<SDMMaterialSlotLayerEffectView>& InEffectView, UDMMaterialEffect* InEffect)
{
	EditComponent(InEffect);
}

void SDMMaterialEditor::OpenMaterialPreviewTab()
{
	if (!PreviewMaterialModelBase)
	{
		return;
	}

	CloseMaterialPreviewTab();

	FSlateApplication::Get().CloseToolTip();

	const FName TabId = TEXT("MaterialPreviewTab");

	if (!FGlobalTabmanager::Get()->HasTabSpawner(TabId))
	{
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
			TabId,
			FOnSpawnTab::CreateLambda(
				[TabId](const FSpawnTabArgs& InArgs)
				{
					TSharedRef<SDockTab> DockTab = SNew(SDockTab)
						.Label(FText::FromName(TabId))
						.LabelSuffix(LOCTEXT("TabSuffix", "Material Preview"));

					DockTab->SetTabIcon(FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()).GetIcon());

					return DockTab;
				}
			)
		);
	}

	MaterialPreviewTab = FGlobalTabmanager::Get()->TryInvokeTab(TabId);
	MaterialPreviewTab->ActivateInParent(ETabActivationCause::SetDirectly);
	MaterialPreviewTab->SetLabel(FText::FromString(PreviewMaterialModelBase->GetPathName()));
	MaterialPreviewTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSPLambda(
		this,
		[this](TSharedRef<SDockTab> InDockTab)
		{
			MaterialPreviewTabSlot.ClearWidget();
		}));

	TSharedRef<SBox> Wrapper = SNew(SBox);

	MaterialPreviewTabSlot = TDMWidgetSlot<SWidget>(
		Wrapper, 
		0, 
		SNew(SDMMaterialPreview, SharedThis(this), PreviewMaterialModelBase)
		.IsPopout(true)
	);

	MaterialPreviewTab->SetContent(Wrapper);
}

void SDMMaterialEditor::CloseMaterialPreviewTab()
{
	if (MaterialPreviewTab.IsValid())
	{
		MaterialPreviewTabSlot.ClearWidget();
		MaterialPreviewTab->RequestCloseTab();
		MaterialPreviewTab.Reset();
	}
}

TSharedPtr<IToolTip> SDMMaterialEditor::GetMaterialPreviewToolTip()
{
	if (!PreviewMaterialModelBase)
	{
		return nullptr;
	}

	UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get();

	if (!Settings)
	{
		return nullptr;
	}

	DestroyMaterialPreviewToolTip();

	TSharedRef<SBox> Wrapper = SNew(SBox)
		.WidthOverride(TAttribute<FOptionalSize>::CreateWeakLambda(
			Settings,
			[Settings]
			{
				return Settings->ThumbnailSize;
			}
		))
		.HeightOverride(TAttribute<FOptionalSize>::CreateWeakLambda(
			Settings,
			[Settings]
			{
				return Settings->ThumbnailSize;
			}
		));

	MaterialPreviewToolTipSlot = TDMWidgetSlot<SWidget>(
		Wrapper,
		0,
		SNew(SDMMaterialPreview, SharedThis(this), PreviewMaterialModelBase)
		.ShowMenu(false)
	);

	MaterialPreviewToolTip = SNew(SToolTip)
		.IsInteractive(false)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.Background"))
		[
			Wrapper
		];

	return MaterialPreviewToolTip.ToSharedRef();
}

void SDMMaterialEditor::DestroyMaterialPreviewToolTip()
{
	if (MaterialPreviewToolTip.IsValid())
	{
		MaterialPreviewToolTipSlot.ClearWidget();
		MaterialPreviewToolTip.Reset();
	}
}

void SDMMaterialEditor::Validate()
{
	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	UDynamicMaterialModelBase* OriginalMaterialModelBase = GetOriginalMaterialModelBase();

	if (!IsValid(OriginalMaterialModelBase) || !IsValid(PreviewMaterialModelBase))
	{
		Close();
		return;
	}

	if (ObjectMaterialPropertyOpt.IsSet() && ObjectMaterialPropertyOpt->IsValid())
	{
		const FDMObjectMaterialProperty& ObjectMaterialProperty = ObjectMaterialPropertyOpt.GetValue();
		UDynamicMaterialModelBase* MaterialModelBaseFromProperty = ObjectMaterialProperty.GetMaterialModelBase();

		if (!UDMMaterialModelFunctionLibrary::IsModelValid(OriginalMaterialModelBase))
		{
			MaterialModelBaseFromProperty = nullptr;
		}

		if (OriginalMaterialModelBase != MaterialModelBaseFromProperty)
		{
			if (TSharedPtr<SDMMaterialDesigner> DesignerWidget = DesignerWidgetWeak.Pin())
			{
				DesignerWidget->OpenObjectMaterialProperty(ObjectMaterialProperty);
				return;
			}
		}
	}
	else if (!UDMMaterialModelFunctionLibrary::IsModelValid(OriginalMaterialModelBase))
	{
		Close();
		return;
	}

	ValidateSlots();
}

SDMMaterialEditor::FOnEditedSlotChanged::RegistrationType& SDMMaterialEditor::GetOnEditedSlotChanged()
{
	return OnEditedSlotChanged;
}

SDMMaterialEditor::FOnEditedComponentChanged::RegistrationType& SDMMaterialEditor::GetOnEditedComponentChanged()
{
	return OnEditedComponentChanged;
}

bool SDMMaterialEditor::SupportsKeyboardFocus() const
{
	return true;
}

FReply SDMMaterialEditor::OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent)
{
	// Cannot make a key bind that has 2 buttons, so hard code that here.
	if (CheckOpacityInput(InKeyEvent))
	{
		return FReply::Handled();
	}

	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	// We accept the delete key bind, so we don't want this accidentally deleting actors and such.
	// Always return handled to stop the event bubbling.
	const TArray<TSharedRef<const FInputChord>> DeleteChords = {
		FGenericCommands::Get().Delete->GetActiveChord(EMultipleKeyBindingIndex::Primary),
		FGenericCommands::Get().Delete->GetActiveChord(EMultipleKeyBindingIndex::Secondary)
	};

	for (const TSharedRef<const FInputChord>& DeleteChord : DeleteChords)
	{
		if (DeleteChord->Key == InKeyEvent.GetKey())
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SDMMaterialEditor::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (CommandList->ProcessCommandBindings(InPointerEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnMouseButtonDown(InGeometry, InPointerEvent);
}

void SDMMaterialEditor::PostUndo(bool bInSuccess)
{
	OnUndo();
}

void SDMMaterialEditor::PostRedo(bool bInSuccess)
{
	OnUndo();
}

void SDMMaterialEditor::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (PreviewMaterialInstance)
	{
		InCollector.AddReferencedObject(PreviewMaterialInstance);
	}

	if (PreviewMaterialModelBase)
	{
		InCollector.AddReferencedObject(PreviewMaterialModelBase);
	}
}

FString SDMMaterialEditor::GetReferencerName() const
{
	return TEXT("SDMMaterialEditor");
}

UPackage* SDMMaterialEditor::GetSaveablePackage(UObject* InObject)
{
	if (!IsValid(InObject))
	{
		return nullptr;
	}

	UPackage* Package = InObject->GetPackage();

	if (!Package || Package->HasAllFlags(RF_Transient))
	{
		return nullptr;
	}

	return Package;
}

void SDMMaterialEditor::BindCommands(SDMMaterialSlotEditor* InSlotEditor)
{
	const FGenericCommands& GenericCommands = FGenericCommands::Get();
	const FDynamicMaterialEditorCommands& DMEditorCommands = FDynamicMaterialEditorCommands::Get();

	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(
		DMEditorCommands.NavigateForward,
		FExecuteAction::CreateSP(this, &SDMMaterialEditor::NavigateForward_Execute),
		FCanExecuteAction::CreateSP(this, &SDMMaterialEditor::NavigateForward_CanExecute)
	);

	CommandList->MapAction(
		DMEditorCommands.NavigateBack,
		FExecuteAction::CreateSP(this, &SDMMaterialEditor::NavigateBack_Execute),
		FCanExecuteAction::CreateSP(this, &SDMMaterialEditor::NavigateBack_CanExecute)
	);

	CommandList->MapAction(
		DMEditorCommands.AddDefaultLayer,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::AddNewLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanAddNewLayer)
	);

	CommandList->MapAction(
		DMEditorCommands.InsertDefaultLayerAbove,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::InsertNewLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanInsertNewLayer)
	);

	for (const TPair<FKey, FDynamicMaterialEditorCommands::FOpacityCommand>& OpacityCommandPair : DMEditorCommands.SetOpacities)
	{
		const float Opacity = OpacityCommandPair.Value.Opacity;
		const TSharedPtr<FUICommandInfo>& OpacityCommand = OpacityCommandPair.Value.Command;

		CommandList->MapAction(
			OpacityCommand,
			FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::SetOpacity_Execute, Opacity),
			FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::SetOpacity_CanExecute)
		);
	}

	CommandList->MapAction(
		GenericCommands.Copy,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CopySelectedLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanCopySelectedLayer)
	);

	CommandList->MapAction(
		GenericCommands.Cut,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CutSelectedLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanCutSelectedLayer)
	);

	CommandList->MapAction(
		GenericCommands.Paste,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::PasteLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanPasteLayer)
	);

	CommandList->MapAction(
		GenericCommands.Duplicate,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::DuplicateSelectedLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanDuplicateSelectedLayer)
	);

	CommandList->MapAction(
		GenericCommands.Delete,
		FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::DeleteSelectedLayer),
		FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::CanDeleteSelectedLayer)
	);

	for (int32 LayerIndex = 0; LayerIndex < DMEditorCommands.SelectLayers.Num(); ++LayerIndex)
	{
		CommandList->MapAction(
			DMEditorCommands.SelectLayers[LayerIndex],
			FExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::SelectLayer_Execute, LayerIndex),
			FCanExecuteAction::CreateSP(InSlotEditor, &SDMMaterialSlotEditor::SelectLayer_CanExecute, LayerIndex)
		);
	}
}

bool SDMMaterialEditor::IsPropertyValidForModel(EDMMaterialPropertyType InProperty) const
{
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(GetPreviewMaterialModelBase());

	if (!EditorOnlyData)
	{
		return false;
	}

	if (UDMMaterialProperty* Property = EditorOnlyData->GetMaterialProperty(InProperty))
	{
		if (Property->IsValidForModel(*EditorOnlyData))
		{
			return true;
		}
	}

	if (InProperty == EDMMaterialPropertyType::Opacity)
	{
		if (UDMMaterialProperty* Property = EditorOnlyData->GetMaterialProperty(EDMMaterialPropertyType::OpacityMask))
		{
			return Property->IsValidForModel(*EditorOnlyData);
		}
	}

	return false;
}

void SDMMaterialEditor::Close()
{
	if (TSharedPtr<SDMMaterialDesigner> DesignerWidget = DesignerWidgetWeak.Pin())
	{
		DesignerWidget->ShowSelectPrompt();
	}
}

void SDMMaterialEditor::ValidateSlots()
{
	if (ContentSlot.HasBeenInvalidated())
	{
		CreateLayout();
		return;
	}

	if (ToolBarSlot.HasBeenInvalidated())
	{
		ToolBarSlot << CreateSlot_ToolBar();
	}

	if (MainSlot.HasBeenInvalidated())
	{
		MainSlot << CreateSlot_Main();
	}
	else
	{
		ValidateSlots_Main();

		if (MaterialPreviewSlot.HasBeenInvalidated())
		{
			MaterialPreviewSlot << CreateSlot_Preview();
		}

		if (PropertySelectorSlot.HasBeenInvalidated())
		{
			PropertySelectorSlot << CreateSlot_PropertySelector();
		}

		switch (SelectionContext.EditorMode)
		{
			case EDMMaterialEditorMode::GlobalSettings:
				if (GlobalSettingsEditorSlot.HasBeenInvalidated())
				{
					GlobalSettingsEditorSlot << CreateSlot_GlobalSettingsEditor();
				}
				else
				{
					GlobalSettingsEditorSlot->Validate();
				}
				break;

			case EDMMaterialEditorMode::Properties:
				if (MaterialPropertiesSlot.HasBeenInvalidated())
				{
					MaterialPropertiesSlot << CreateSlot_MaterialProperties();
				}
				else
				{
					MaterialPropertiesSlot->Validate();
				}
				break;

			default:
				if (SlotEditorSlot.HasBeenInvalidated())
				{
					SlotEditorSlot << CreateSlot_SlotEditor();
				}
				else
				{
					SlotEditorSlot->ValidateSlots();
				}

				if (ComponentEditorSlot.HasBeenInvalidated())
				{
					ComponentEditorSlot << CreateSlot_ComponentEditor();
				}
				else
				{
					ComponentEditorSlot->Validate();
				}
				break;
		}
	}

	if (StatusBarSlot.HasBeenInvalidated())
	{
		StatusBarSlot << CreateSlot_StatusBar();
	}

	SelectionContext.bModeChanged = false;
}

void SDMMaterialEditor::ClearSlots()
{
	ContentSlot.ClearWidget();
	ToolBarSlot.ClearWidget();
	MainSlot.ClearWidget();
	SlotEditorSlot.ClearWidget();
	MaterialPreviewSlot.ClearWidget();
	PropertySelectorSlot.ClearWidget();
	GlobalSettingsEditorSlot.ClearWidget();
	SplitterSlot = nullptr;
	ComponentEditorSlot.ClearWidget();
	StatusBarSlot.ClearWidget();

	ClearSlots_Main();
}

void SDMMaterialEditor::PageHistoryAdd(const FDMMaterialEditorPage& InPage)
{
	if (SelectionContext.PageHistory.IsValidIndex(SelectionContext.PageHistoryActive) && SelectionContext.PageHistory[SelectionContext.PageHistoryActive] == InPage)
	{
		return;
	}

	const int32 NewPageIndex = SelectionContext.PageHistoryActive + 1;

	if (!SelectionContext.PageHistory.IsValidIndex(NewPageIndex))
	{
		SelectionContext.PageHistory.Add(InPage);
	}
	else
	{
		SelectionContext.PageHistory[NewPageIndex] = InPage;
	}	

	SelectionContext.PageHistoryActive = NewPageIndex;
	SelectionContext.PageHistoryCount = NewPageIndex + 1;
}

void SDMMaterialEditor::ApplyToOriginal()
{
	UDynamicMaterialModelBase* Preview = GetPreviewMaterialModelBase();

	if (!Preview)
	{
		return;
	}

	UDynamicMaterialModelBase* Original = GetOriginalMaterialModelBase();

	if (!Original)
	{
		return;
	}

	if (UDynamicMaterialModel* PreviewMaterialModel = Cast<UDynamicMaterialModel>(Preview))
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(PreviewMaterialModel))
		{
			if (EditorOnlyData->HasBuildBeenRequested())
			{
				EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Immediate);
			}
		}		
	}

	UDynamicMaterialInstance* OriginalMaterialInstance = Original->GetDynamicMaterialInstance();

	if (GUndo)
	{
		Original->Modify();
	}

	UDMMaterialModelFunctionLibrary::MirrorMaterialModel(Preview, Original);

	OriginalMaterialModelBaseWeak = Original;

	if (OriginalMaterialInstance)
	{
		OriginalMaterialInstance->SetMaterialModel(Original);
		Original->SetDynamicMaterialInstance(OriginalMaterialInstance);
	}

	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(Original))
	{
		EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Immediate);
	}

	if (OriginalMaterialInstance)
	{
		OriginalMaterialInstance->InitializeMIDPublic();
		Original->ApplyComponents(OriginalMaterialInstance);
	}

	MainSlot.Invalidate();

	if (PreviewMaterialModelBase)
	{
		PreviewMaterialModelBase->MarkOriginalUpdated();
	}
}

void SDMMaterialEditor::Compile()
{
	if (PreviewMaterialModelBase && !PreviewMaterialModelBase->IsA<UDynamicMaterialModelDynamic>())
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(PreviewMaterialModelBase.Get()))
		{
			EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Immediate);
		}
	}
}

bool SDMMaterialEditor::SetActivePage(const FDMMaterialEditorPage& InPage)
{
	switch (InPage.EditorMode)
	{
		// This is not a valid page
		case EDMMaterialEditorMode::MaterialPreview:
		default:
			return false;

		case EDMMaterialEditorMode::GlobalSettings:
			EditGlobalSettings();
			return true;

		case EDMMaterialEditorMode::Properties:
			EditProperties();
			return true;

		case EDMMaterialEditorMode::EditSlot:
			SelectProperty(InPage.MaterialProperty);
			return true;
	}
}

void SDMMaterialEditor::SaveOriginal()
{
	if (UDynamicMaterialModelBase* MaterialModelBase = GetOriginalMaterialModelBase())
	{
		ApplyToOriginal();

		if (!!GetSaveablePackage(MaterialModelBase))
		{
			TArray<UObject*> AssetsToSave;
			AssetsToSave.Add(MaterialModelBase);
			UPackageTools::SavePackagesForObjects(AssetsToSave);
		}
	}
}

void SDMMaterialEditor::HandleDrop_CreateTextureSet(const TArray<FAssetData>& InTextureAssets)
{
	if (InTextureAssets.Num() < 2)
	{
		return;
	}

	UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssetsInteractive(
		InTextureAssets,
		FDMTextureSetBuilderOnComplete::CreateSPLambda(
			this,
			[this](UDMTextureSet* InTextureSet, bool bInWasAccepted)
			{
				if (bInWasAccepted)
				{
					HandleDrop_TextureSet(InTextureSet);
				}
			}
		)
	);
}

void SDMMaterialEditor::HandleDrop_TextureSet(UDMTextureSet* InTextureSet)
{
	if (!InTextureSet)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = GetPreviewMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	const EAppReturnType::Type Result = FMessageDialog::Open(
		EAppMsgType::YesNoCancel,
		LOCTEXT("ReplaceSlotsTextureSet",
			"Material Designer Texture Set.\n\n"
			"Replace Slots?\n\n"
			"- Yes: Delete Layers.\n"
			"- No: Add Layers.\n"
			"- Cancel")
	);

	FDMScopedUITransaction Transaction(LOCTEXT("DropTextureSet", "Drop Texture Set"));

	switch (Result)
	{
		case EAppReturnType::No:
			EditorOnlyData->Modify();
			UDMTextureSetFunctionLibrary::AddTextureSetToModel(EditorOnlyData, InTextureSet, /* Replace */ false);
			break;

		case EAppReturnType::Yes:
			EditorOnlyData->Modify();
			UDMTextureSetFunctionLibrary::AddTextureSetToModel(EditorOnlyData, InTextureSet, /* Replace */ true);
			break;

		default:
			Transaction.Transaction.Cancel();
			break;
	}
}

bool SDMMaterialEditor::PageHistoryBack()
{
	const int32 NewPageIndex = SelectionContext.PageHistoryActive - 1;

	if (!SelectionContext.PageHistory.IsValidIndex(NewPageIndex))
	{
		return false;
	}

	const int32 OldPageIndex = SelectionContext.PageHistoryActive;
	SelectionContext.PageHistoryActive = NewPageIndex;

	if (!SetActivePage(SelectionContext.PageHistory[NewPageIndex]))
	{
		SelectionContext.PageHistoryActive = OldPageIndex;
		return false;
	}

	return true;
}

bool SDMMaterialEditor::PageHistoryForward()
{
	const int32 NewPageIndex = SelectionContext.PageHistoryActive + 1;

	if (NewPageIndex >= SelectionContext.PageHistoryCount || !SelectionContext.PageHistory.IsValidIndex(NewPageIndex))
	{
		return false;
	}

	const int32 OldPageIndex = SelectionContext.PageHistoryActive;
	SelectionContext.PageHistoryActive = NewPageIndex;

	if (!SetActivePage(SelectionContext.PageHistory[NewPageIndex]))
	{
		SelectionContext.PageHistoryActive = OldPageIndex;
		return false;
	}

	return true;
}

void SDMMaterialEditor::CreateLayout()
{
	ContentSlot << CreateSlot_Container();
}

TSharedRef<SWidget> SDMMaterialEditor::CreateSlot_Container()
{
	SVerticalBox::FSlot* ToolBarSlotPtr = nullptr;
	SVerticalBox::FSlot* MainSlotPtr = nullptr;
	SVerticalBox::FSlot* StatusBarSlotPtr = nullptr;

	TSharedRef<SVerticalBox> NewContainer = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Expose(ToolBarSlotPtr)
		.AutoHeight()
		[
			SNullWidget::NullWidget
		]

		+ SVerticalBox::Slot()
		.Expose(MainSlotPtr)
		.FillHeight(1.0f)
		[
			SNullWidget::NullWidget
		]

		+ SVerticalBox::Slot()
		.Expose(StatusBarSlotPtr)
		.AutoHeight()
		[
			SNullWidget::NullWidget
		];

	ToolBarSlot = TDMWidgetSlot<SDMToolBar>(ToolBarSlotPtr, CreateSlot_ToolBar());
	MainSlot = TDMWidgetSlot<SWidget>(MainSlotPtr, CreateSlot_Main());
	StatusBarSlot = TDMWidgetSlot<SDMStatusBar>(StatusBarSlotPtr, CreateSlot_StatusBar());

	return NewContainer;
}

TSharedRef<SDMToolBar> SDMMaterialEditor::CreateSlot_ToolBar()
{
	return SNew(
		SDMToolBar, 
		SharedThis(this), 
		ObjectMaterialPropertyOpt.IsSet()
			? ObjectMaterialPropertyOpt.GetValue()
			: FDMObjectMaterialProperty(GetMaterialActor(), 0)
	);
}

TSharedRef<SDMMaterialGlobalSettingsEditor> SDMMaterialEditor::CreateSlot_GlobalSettingsEditor()
{
	return SNew(SDMMaterialGlobalSettingsEditor, SharedThis(this), GetPreviewMaterialModelBase());
}

TSharedRef<SDMMaterialProperties> SDMMaterialEditor::CreateSlot_MaterialProperties()
{
	return SNew(SDMMaterialProperties, SharedThis(this));
}

TSharedRef<SWidget> SDMMaterialEditor::CreateSlot_Preview()
{
	return SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SDMMaterialPreview, SharedThis(this), GetPreviewMaterialModelBase())
		]
		+ SOverlay::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Bottom)
		.Padding(3.f, 2.f)
		[
			SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("TinyText"))
				.Text(IsDynamicModel() 
					? LOCTEXT("MaterialInstance", "Instance")
					: LOCTEXT("MaterialTemplate", "Material"))
				.ShadowColorAndOpacity(FLinearColor::Black)
				.ShadowOffset(FVector2D(1.0))
		];
}

TSharedRef<SDMMaterialPropertySelector> SDMMaterialEditor::CreateSlot_PropertySelector()
{
	TSharedRef<SDMMaterialPropertySelector> NewPropertySelector = CreateSlot_PropertySelector_Impl();

	if (SelectionContext.EditorMode == EDMMaterialEditorMode::EditSlot && SelectionContext.Property == EDMMaterialPropertyType::None)
	{
		if (UDynamicMaterialModel* MaterialModel = GetPreviewMaterialModel())
		{
			if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
			{
				for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
				{
					if (PropertyPair.Value->IsEnabled() && PropertyPair.Value->IsValidForModel(*EditorOnlyData))
					{
						SelectionContext.Property = PropertyPair.Key;
						break;
					}
				}
			}
		}
	}

	return NewPropertySelector;
}

TSharedRef<SDMMaterialSlotEditor> SDMMaterialEditor::CreateSlot_SlotEditor()
{
	UDMMaterialSlot* Slot = SelectionContext.Slot.Get();

	TSharedRef<SDMMaterialSlotEditor> NewSlotEditor = SNew(SDMMaterialSlotEditor, SharedThis(this), Slot);

	if (UDMMaterialLayerObject* Layer = SelectionContext.Layer.Get())
	{
		NewSlotEditor->SetSelectedLayer(Layer);
	}

	NewSlotEditor->GetOnLayerSelectionChanged().AddSP(this, &SDMMaterialEditor::OnLayerSelected);
	NewSlotEditor->GetOnStageSelectionChanged().AddSP(this, &SDMMaterialEditor::OnStageSelected);
	NewSlotEditor->GetOnEffectSelectionChanged().AddSP(this, &SDMMaterialEditor::OnEffectSelected);

	BindCommands(&*NewSlotEditor);

	OnEditedSlotChanged.Broadcast(NewSlotEditor, Slot);

	return NewSlotEditor;
}

TSharedRef<SDMMaterialComponentEditor> SDMMaterialEditor::CreateSlot_ComponentEditor()
{
	UDMMaterialComponent* Component = SelectionContext.Component.Get();

	TSharedRef<SDMMaterialComponentEditor> NewComponentEditor = SNew(SDMMaterialComponentEditor, SharedThis(this), Component);

	OnEditedComponentChanged.Broadcast(NewComponentEditor, Component);

	return NewComponentEditor;
}

TSharedRef<SDMStatusBar> SDMMaterialEditor::CreateSlot_StatusBar()
{
	return SNew(SDMStatusBar, SharedThis(this), GetPreviewMaterialModelBase());
}

void SDMMaterialEditor::OnUndo()
{
	UDynamicMaterialModelBase* OriginalMaterialModel = GetOriginalMaterialModelBase();

	if (!IsValid(OriginalMaterialModel))
	{
		Close();
		return;
	}

	ApplySelectionContext();
}

void SDMMaterialEditor::ApplySelectionContext()
{
	switch (SelectionContext.EditorMode)
	{
		case EDMMaterialEditorMode::MaterialPreview:
			// do nothing
			break;

		case EDMMaterialEditorMode::EditSlot:
			EditSlot(SelectionContext.Slot.Get());
			break;

		case EDMMaterialEditorMode::GlobalSettings:
			EditGlobalSettings();
			break;

		case EDMMaterialEditorMode::Properties:
			EditProperties();
			break;
	}

	if (SelectionContext.EditorMode == EDMMaterialEditorMode::EditSlot)
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(GetPreviewMaterialModelBase()))
		{
			if (UDMMaterialSlot* Slot = SelectionContext.Slot.Get())
			{
				const TArray<EDMMaterialPropertyType> SlotProperties = EditorOnlyData->GetMaterialPropertiesForSlot(Slot);

				if (!SlotProperties.IsEmpty())
				{
					SelectProperty(SlotProperties[0]);
					return;
				}
			}

			for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& PropertyPair : EditorOnlyData->GetMaterialProperties())
			{
				if (PropertyPair.Value->IsEnabled() && PropertyPair.Value->IsValidForModel(*EditorOnlyData))
				{
					SelectProperty(PropertyPair.Key);
					break;
				}
			}
		}
	}
}

void SDMMaterialEditor::OnEnginePreExit()
{
	MaterialPreviewSlot.ClearWidget();
	CloseMaterialPreviewTab();
	DestroyMaterialPreviewToolTip();
}

void SDMMaterialEditor::OnEditorSplitterResized()
{
	if (SplitterSlot)
	{
		if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
		{
			const float SplitterLocation = static_cast<SSplitter::FSlot*>(SplitterSlot)->GetSizeValue();
			Settings->SplitterLocation = SplitterLocation;
			Settings->SaveConfig();
		}
	}
}

void SDMMaterialEditor::BindEditorOnlyDataUpdate(UDynamicMaterialModelBase* InMaterialModelBase)
{
	if (UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(InMaterialModelBase))
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
		{
			EditorOnlyDataUpdateObject = EditorOnlyData;
			EditorOnlyData->GetOnMaterialBuiltDelegate().AddSP(this, &SDMMaterialEditor::OnMaterialBuilt);
			EditorOnlyData->GetOnPropertyUpdateDelegate().AddSP(this, &SDMMaterialEditor::OnPropertyUpdate);
			EditorOnlyData->GetOnSlotListUpdateDelegate().AddSP(this, &SDMMaterialEditor::OnSlotListUpdate);
		}
	}
}

void SDMMaterialEditor::UnbindEditorOnlyDataUpdate()
{
	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = EditorOnlyDataUpdateObject.Get())
	{
		EditorOnlyData->GetOnMaterialBuiltDelegate().RemoveAll(this);
		EditorOnlyData->GetOnPropertyUpdateDelegate().RemoveAll(this);
		EditorOnlyData->GetOnSlotListUpdateDelegate().RemoveAll(this);
	}
}

void SDMMaterialEditor::OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModelBase)
{
	PropertySelectorSlot.Invalidate();

	if (bSkipApplyOnCompile)
	{
		bSkipApplyOnCompile = false;
		return;
	}

	if (UDynamicMaterialEditorSettings* Settings = UDynamicMaterialEditorSettings::Get())
	{
		if (Settings->ShouldAutomaticallyApplyToSourceOnPreviewCompile())
		{
			ApplyToOriginal();
		}
	}
}

void SDMMaterialEditor::OnPropertyUpdate(UDynamicMaterialModelBase* InMaterialModelBase)
{
	PropertySelectorSlot.Invalidate();
}

void SDMMaterialEditor::OnSlotListUpdate(UDynamicMaterialModelBase* InMaterialModelBase)
{
	PropertySelectorSlot.Invalidate();
}

void SDMMaterialEditor::OnSettingsChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!PropertySelectorSlot.IsValid())
	{
		return;
	}

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UDynamicMaterialEditorSettings, bUseFullChannelNamesInTopSlimLayout))
	{
		PropertySelectorSlot.Invalidate();
	}
}

void SDMMaterialEditor::NavigateForward_Execute()
{
	PageHistoryForward();
}

bool SDMMaterialEditor::NavigateForward_CanExecute()
{
	return (SelectionContext.PageHistoryActive + 1) < SelectionContext.PageHistoryCount;
}

void SDMMaterialEditor::NavigateBack_Execute()
{
	PageHistoryBack();
}

bool SDMMaterialEditor::NavigateBack_CanExecute()
{
	return SelectionContext.PageHistoryActive > 0;
}

bool SDMMaterialEditor::CheckOpacityInput(const FKeyEvent& InKeyEvent)
{
	if (!KeyTracker_V.IsValid() || !KeyTracker_V->IsKeyDown() || InKeyEvent.GetKey() == KeyTracker_V->GetTrackedKey())
	{
		return false;
	}

	const FDynamicMaterialEditorCommands& DMEditorCommands = FDynamicMaterialEditorCommands::Get();

	if (const FDynamicMaterialEditorCommands::FOpacityCommand* OpacityCommandPair = DMEditorCommands.SetOpacities.Find(InKeyEvent.GetKey()))
	{
		const TSharedRef<FUICommandInfo>& OpacityCommand = OpacityCommandPair->Command;
		return CommandList->TryExecuteAction(OpacityCommand);
	}

	return false;
}

void SDMMaterialEditor::SelectProperty_Impl(EDMMaterialPropertyType InProperty)
{
	SlotEditorSlot.Invalidate();
	SplitterSlot = nullptr;

	if (SelectionContext.EditorMode != EDMMaterialEditorMode::EditSlot)
	{
		SelectionContext.bModeChanged = true;
	}

	SelectionContext.EditorMode = EDMMaterialEditorMode::EditSlot;
	SelectionContext.Property = InProperty;
}

void SDMMaterialEditor::EditSlot_Impl(UDMMaterialSlot* InSlot)
{
	SlotEditorSlot.Invalidate();
	SplitterSlot = nullptr;

	ComponentEditorSlot.Invalidate();

	SelectionContext.EditorMode = EDMMaterialEditorMode::EditSlot;
	SelectionContext.Slot = InSlot;
}

void SDMMaterialEditor::EditComponent_Impl(UDMMaterialComponent* InComponent)
{
	if (SelectionContext.EditorMode != EDMMaterialEditorMode::EditSlot)
	{
		SlotEditorSlot.Invalidate();
		SplitterSlot = nullptr;
		GlobalSettingsEditorSlot.Invalidate();
		MaterialPropertiesSlot.Invalidate();
	}

	ComponentEditorSlot.Invalidate();

	SelectionContext.EditorMode = EDMMaterialEditorMode::EditSlot;
	SelectionContext.Component = InComponent;
}

void SDMMaterialEditor::EditGlobalSettings_Impl()
{
	if (SelectionContext.EditorMode != EDMMaterialEditorMode::GlobalSettings)
	{
		SlotEditorSlot.Invalidate();
		SplitterSlot = nullptr;
		ComponentEditorSlot.Invalidate();
		MaterialPropertiesSlot.Invalidate();
	}

	SelectionContext.EditorMode = EDMMaterialEditorMode::GlobalSettings;
	SelectionContext.Property = EDMMaterialPropertyType::None;
	SelectionContext.Slot.Reset();
	SelectionContext.Layer.Reset();

	GlobalSettingsEditorSlot.Invalidate();

}

void SDMMaterialEditor::EditProperties_Impl()
{
	if (SelectionContext.EditorMode != EDMMaterialEditorMode::Properties)
	{
		SlotEditorSlot.Invalidate();
		SplitterSlot = nullptr;
		ComponentEditorSlot.Invalidate();
		GlobalSettingsEditorSlot.Invalidate();
	}

	SelectionContext.EditorMode = EDMMaterialEditorMode::Properties;
	SelectionContext.Property = EDMMaterialPropertyType::None;
	SelectionContext.Slot.Reset();
	SelectionContext.Layer.Reset();

	MaterialPropertiesSlot.Invalidate();
}

#undef LOCTEXT_NAMESPACE
