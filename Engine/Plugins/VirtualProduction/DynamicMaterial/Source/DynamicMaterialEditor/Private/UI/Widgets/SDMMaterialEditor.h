// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "DMObjectMaterialProperty.h"
#include "Misc/Optional.h"
#include "UI/Utils/DMEditorSelectionContext.h"
#include "UI/Utils/DMWidgetSlot.h"
#include "UObject/StrongObjectPtr.h"

class FDMKeyTracker;
class FDMPreviewMaterialManager;
class FSlotBase;
class FUICommandList;
class IToolTip;
class SDMMaterialComponentEditor;
class SDMMaterialDesigner;
class SDMMaterialGlobalSettingsEditor;
class SDMMaterialPreview;
class SDMMaterialProperties;
class SDMMaterialPropertySelector;
class SDMMaterialSlotEditor;
class SDMMaterialSlotLayerEffectView;
class SDMMaterialSlotLayerItem;
class SDMMaterialSlotLayerView;
class SDMStatusBar;
class SDMToolBar;
class SDockTab;
class SSplitter;
class UDMMaterialComponent;
class UDMMaterialEffect;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMTextureSet;
class UDynamicMaterialModelBase;
class UDynamicMaterialModelEditorOnlyData;
enum class EDMMaterialPropertyType : uint8;
enum class EDMUpdateType : uint8;
struct FDMMaterialLayerReference;
struct InPropertyChangedEvent;

namespace UE::DynamicMaterialEditor::Private
{
	inline const TCHAR* EditorDarkBackground = TEXT("Brushes.Title");
	inline const TCHAR* EditorLightBackground = TEXT("Brushes.Header");
}

class SDMMaterialEditor : public SCompoundWidget, public FSelfRegisteringEditorUndoClient, public FGCObject
{
	SLATE_DECLARE_WIDGET(SDMMaterialEditor, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialEditor)
		: _MaterialModelBase(nullptr)
		, _MaterialProperty(TOptional<FDMObjectMaterialProperty>())
		, _PreviewMaterialModelBase(nullptr)
		{}
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, MaterialModelBase)
		SLATE_ARGUMENT(TOptional<FDMObjectMaterialProperty>, MaterialProperty)
		SLATE_ARGUMENT(UDynamicMaterialModelBase*, PreviewMaterialModelBase)
	SLATE_END_ARGS()

public:
	static UPackage* GetSaveablePackage(UObject* InObject);

	SDMMaterialEditor();

	virtual ~SDMMaterialEditor() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget);

	TSharedPtr<SDMMaterialDesigner> GetDesignerWidget() const;

	/** Material Model / Actor */
	UDynamicMaterialModelBase* GetOriginalMaterialModelBase() const;

	UDynamicMaterialModelBase* GetPreviewMaterialModelBase() const;

	UDynamicMaterialModel* GetPreviewMaterialModel() const;

	bool IsDynamicModel() const;

	const FDMObjectMaterialProperty* GetMaterialObjectProperty() const;

	AActor* GetMaterialActor() const;

	UDMMaterialComponent* GetOriginalComponent(UDMMaterialComponent* InPreviewComponent) const;

	/** Widget Components */
	EDMMaterialEditorMode GetEditMode() const;

	EDMMaterialPropertyType GetSelectedPropertyType() const;

	const TSharedRef<FUICommandList>& GetCommandList() const;

	TSharedRef<FDMPreviewMaterialManager> GetPreviewMaterialManager() const;

	TSharedPtr<SDMMaterialSlotEditor> GetSlotEditorWidget() const;

	TSharedPtr<SDMMaterialComponentEditor> GetComponentEditorWidget() const;

	UDMMaterialSlot* SlotSelectedSlot() const;

	UDMMaterialComponent* GetSelectedComponent() const;

	/** Actions */
	void SelectProperty(EDMMaterialPropertyType InProperty, bool bInForceRefresh = false);

	void EditSlot(UDMMaterialSlot* InSlot, bool bInForceRefresh = false);

	void EditComponent(UDMMaterialComponent* InComponent, bool bInForceRefresh = false);

	void EditGlobalSettings(bool bInForceRefresh = false);

	void EditProperties(bool bInForceRefresh = false);

	void OnLayerSelected(const TSharedRef<SDMMaterialSlotLayerView>& InSlotView, const TSharedPtr<FDMMaterialLayerReference>& InLayerView);

	void OnStageSelected(const TSharedRef<SDMMaterialSlotLayerItem>& InLayerItem, UDMMaterialStage* InStage);

	void OnEffectSelected(const TSharedRef<SDMMaterialSlotLayerEffectView>& InEffectView, UDMMaterialEffect* InEffect);

	void OpenMaterialPreviewTab();

	void CloseMaterialPreviewTab();

	TSharedPtr<IToolTip> GetMaterialPreviewToolTip();

	void DestroyMaterialPreviewToolTip();

	void Validate();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEditedSlotChanged, const TSharedRef<SDMMaterialSlotEditor>&, UDMMaterialSlot*);
	FOnEditedSlotChanged::RegistrationType& GetOnEditedSlotChanged();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEditedComponentChanged, const TSharedRef<SDMMaterialComponentEditor>&, UDMMaterialComponent*);
	FOnEditedComponentChanged::RegistrationType& GetOnEditedComponentChanged();

	bool PageHistoryBack();

	bool PageHistoryForward();

	bool SetActivePage(const FDMMaterialEditorPage& InPage);

	void SaveOriginal();

	void ApplyToOriginal();

	void Compile();

	void HandleDrop_CreateTextureSet(const TArray<FAssetData>& InTextureAssets);

	void HandleDrop_TextureSet(UDMTextureSet* InTextureSet);

	//~ Begin SWidget
	virtual bool SupportsKeyboardFocus() const override;
	virtual FReply OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	//~ End SWidget

	//~ Begin FUndoClient
	virtual void PostUndo(bool bInSuccess) override;
	virtual void PostRedo(bool bInSuccess) override;
	//~ End FUndoClient

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

protected:
	TWeakPtr<SDMMaterialDesigner> DesignerWidgetWeak;

	TDMWidgetSlot<SWidget> ContentSlot;
	TDMWidgetSlot<SDMToolBar> ToolBarSlot;
	TDMWidgetSlot<SWidget> MainSlot;
	TDMWidgetSlot<SWidget> MaterialPreviewSlot;
	TDMWidgetSlot<SDMMaterialPropertySelector> PropertySelectorSlot;
	TDMWidgetSlot<SDMMaterialGlobalSettingsEditor> GlobalSettingsEditorSlot;
	TDMWidgetSlot<SDMMaterialProperties> MaterialPropertiesSlot;
	FSlotBase* SplitterSlot;
	TDMWidgetSlot<SDMMaterialSlotEditor> SlotEditorSlot;
	TDMWidgetSlot<SDMMaterialComponentEditor> ComponentEditorSlot;
	TDMWidgetSlot<SDMStatusBar> StatusBarSlot;

	TWeakObjectPtr<UDynamicMaterialModelBase> OriginalMaterialModelBaseWeak;
	TObjectPtr<UDynamicMaterialInstance> PreviewMaterialInstance;
	TObjectPtr<UDynamicMaterialModelBase> PreviewMaterialModelBase;
	TOptional<FDMObjectMaterialProperty> ObjectMaterialPropertyOpt;

	TSharedRef<FUICommandList> CommandList;
	TSharedPtr<FDMKeyTracker> KeyTracker_V;
	TSharedRef<FDMPreviewMaterialManager> PreviewMaterialManager;
	TSharedPtr<SDockTab> MaterialPreviewTab;
	TDMWidgetSlot<SWidget> MaterialPreviewTabSlot;
	TSharedPtr<IToolTip> MaterialPreviewToolTip;
	TDMWidgetSlot<SWidget> MaterialPreviewToolTipSlot;

	FDMEditorSelectionContext SelectionContext;

	FOnEditedSlotChanged OnEditedSlotChanged;
	FOnEditedComponentChanged OnEditedComponentChanged;

	TWeakObjectPtr<UDynamicMaterialModelEditorOnlyData> EditorOnlyDataUpdateObject;

	bool bSkipApplyOnCompile;

	/** Operations */
	void SetMaterialModelBase(UDynamicMaterialModelBase* InMaterialModelBase, UDynamicMaterialModelBase* InPreviewMaterialModelBase);

	void SetObjectMaterialProperty(const FDMObjectMaterialProperty& InObjectProperty, UDynamicMaterialModelBase* InPreviewMaterialModelBase);

	void SetMaterialActor(AActor* InActor);

	void BindCommands(SDMMaterialSlotEditor* InSlotEditor);

	bool IsPropertyValidForModel(EDMMaterialPropertyType InProperty) const;

	void Close();

	void ValidateSlots();

	virtual void ValidateSlots_Main() = 0;

	void ClearSlots();

	virtual void ClearSlots_Main() = 0;

	void PageHistoryAdd(const FDMMaterialEditorPage& InPage);

	/** Slots */
	void CreateLayout();

	TSharedRef<SWidget> CreateSlot_Container();

	TSharedRef<SDMToolBar> CreateSlot_ToolBar();

	virtual TSharedRef<SWidget> CreateSlot_Main() = 0;

	TSharedRef<SDMMaterialGlobalSettingsEditor> CreateSlot_GlobalSettingsEditor();

	TSharedRef<SDMMaterialProperties> CreateSlot_MaterialProperties();

	TSharedRef<SWidget> CreateSlot_Preview();

	TSharedRef<SDMMaterialPropertySelector> CreateSlot_PropertySelector();

	virtual TSharedRef<SDMMaterialPropertySelector> CreateSlot_PropertySelector_Impl() = 0;

	TSharedRef<SDMMaterialSlotEditor> CreateSlot_SlotEditor();

	TSharedRef<SDMMaterialComponentEditor> CreateSlot_ComponentEditor();

	TSharedRef<SDMStatusBar> CreateSlot_StatusBar();

	/** Events */
	void OnUndo();

	void ApplySelectionContext();

	/** The material preview window is not cleaned up properly on uobject shutdown, so do it here. */
	void OnEnginePreExit();

	void OnEditorSplitterResized();

	void BindEditorOnlyDataUpdate(UDynamicMaterialModelBase* InMaterialModelBase);

	void UnbindEditorOnlyDataUpdate();

	void OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModelBase);

	void OnPropertyUpdate(UDynamicMaterialModelBase* InMaterialModelBase);

	void OnSlotListUpdate(UDynamicMaterialModelBase* InMaterialModelBase);

	void OnSettingsChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	void NavigateForward_Execute();
	bool NavigateForward_CanExecute();

	void NavigateBack_Execute();
	bool NavigateBack_CanExecute();

	bool CheckOpacityInput(const FKeyEvent& InKeyEvent);

	/** Actions */
	void SelectProperty_Impl(EDMMaterialPropertyType InProperty);

	virtual void EditSlot_Impl(UDMMaterialSlot* InSlot);

	virtual void EditComponent_Impl(UDMMaterialComponent* InComponent);

	virtual void EditGlobalSettings_Impl();

	virtual void EditProperties_Impl();
};
