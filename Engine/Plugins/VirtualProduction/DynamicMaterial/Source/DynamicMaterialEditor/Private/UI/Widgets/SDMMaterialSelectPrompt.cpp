// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/SDMMaterialSelectPrompt.h"

#include "DynamicMaterialEditorStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialSelectPrompt"

void SDMMaterialSelectPrompt::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

void SDMMaterialSelectPrompt::Construct(const FArguments& InArgs)
{
	SetCanTick(false);

	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Center)
		.Padding(5.0f, 5.0f, 5.0f, 5.0f)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
			.Text(LOCTEXT("NoActiveMaterial", "No active Material Designer Material."))
		]
	];
}

#undef LOCTEXT_NAMESPACE
