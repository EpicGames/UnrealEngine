// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SDMMaterialSelectPrompt : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SDMMaterialSelectPrompt, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialSelectPrompt) {}
	SLATE_END_ARGS()

public:
	virtual ~SDMMaterialSelectPrompt() override = default;

	void Construct(const FArguments& InArgs);
};
