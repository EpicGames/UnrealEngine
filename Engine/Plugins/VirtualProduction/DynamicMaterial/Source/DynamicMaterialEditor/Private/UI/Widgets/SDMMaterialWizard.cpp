// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/SDMMaterialWizard.h"

#include "AssetRegistry/AssetData.h"
#include "AssetTextFilter.h"
#include "AssetThumbnail.h"
#include "DMDefs.h"
#include "DMTextureSetBlueprintFunctionLibrary.h"
#include "DynamicMaterialEditorSettings.h"
#include "DynamicMaterialEditorStyle.h"
#include "DynamicMaterialModule.h"
#include "Engine/EngineTypes.h"
#include "IContentBrowserSingleton.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "SAssetSearchBox.h"
#include "SAssetView.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "UI/Widgets/SDMMaterialDesigner.h"
#include "Utils/DMMaterialInstanceFunctionLibrary.h"
#include "Utils/DMMaterialModelFunctionLibrary.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMTextureSetFunctionLibrary.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMMaterialWizard"

namespace UE::DynamicMaterialDesigner::Private
{
	constexpr float SeparationDistance = 20.f;
	constexpr float TitleContentDistance = 5.f;
	static const FMargin ButtonPadding = FMargin(10.f, 5.f);
	static const FMargin TextPadding = FMargin(5.f, 2.f);
	static const FVector2D WrapBoxSlotPadding = FVector2D(5, 5);
}

void SDMMaterialWizard::PrivateRegisterAttributes(FSlateAttributeDescriptor::FInitializer&)
{
}

SDMMaterialWizard::~SDMMaterialWizard()
{
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDynamicMaterialModel* MaterialModel = MaterialModelWeak.Get())
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
		{
			EditorOnlyData->GetOnMaterialBuiltDelegate().RemoveAll(this);
		}
	}
}

void SDMMaterialWizard::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget)
{
	DesignerWidgetWeak = InDesignerWidget;
	MaterialModelWeak = InArgs._MaterialModel;
	MaterialObjectProperty = InArgs._MaterialProperty;

	SetCanTick(false);

	FCoreDelegates::OnEnginePreExit.AddSP(this, &SDMMaterialWizard::OnEnginePreExit);

	if (MaterialObjectProperty.IsSet())
	{
		if (UDynamicMaterialModelBase* MaterialModelBase = MaterialObjectProperty.GetValue().GetMaterialModelBase())
		{
			if (UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(MaterialModelBase))
			{
				// Override any parameter given.
				MaterialModelWeak = MaterialModel;
			}
		}
	}

	if (const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>())
	{
		if (!Settings->MaterialChannelPresets.IsEmpty())
		{
			CurrentPreset = Settings->MaterialChannelPresets[0].Name;
		}
	}

	if (UDynamicMaterialModel* MaterialModel = MaterialModelWeak.Get())
	{
		if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
		{
			// Subscribe to this in case the wizard completes externally and this widget is no longer needed.
			EditorOnlyData->GetOnMaterialBuiltDelegate().AddSP(this, &SDMMaterialWizard::OnMaterialBuilt);
		}
	}

	ChildSlot
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Fill)
		[
			CreateLayout()
		]
	];
}

TSharedPtr<SDMMaterialDesigner> SDMMaterialWizard::GetDesignerWidget() const
{
	return DesignerWidgetWeak.Pin();
}

UDynamicMaterialModel* SDMMaterialWizard::GetMaterialModel() const
{
	return MaterialModelWeak.Get();
}

void SDMMaterialWizard::HandleDrop_CreateTextureSet(const TArray<FAssetData>& InTextureAssets)
{
	if (InTextureAssets.Num() < 2)
	{
		return;
	}

	UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssetsInteractive(
		InTextureAssets,
		FDMTextureSetBuilderOnComplete::CreateSPLambda(
			this,
			[this](UDMTextureSet* InTextureSet, bool bInWasAccepted)
			{
				if (bInWasAccepted)
				{
					HandleDrop_TextureSet(InTextureSet);
				}
			}
		)
	);
}

void SDMMaterialWizard::HandleDrop_TextureSet(UDMTextureSet* InTextureSet)
{
	if (!InTextureSet)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = GetMaterialModel();

	if (!MaterialModel)
	{
		if (!MaterialObjectProperty.IsSet())
		{
			return;
		}

		if (!MaterialObjectProperty->IsValid())
		{
			return;
		}

		UDynamicMaterialInstance* Material = MaterialObjectProperty->GetMaterial();

		if (!Material)
		{
			MaterialModel = UDMMaterialInstanceFunctionLibrary::CreateMaterialInObject(*MaterialObjectProperty);
		}
		else
		{
			UDynamicMaterialModelBase* MaterialModelBase = Material->GetMaterialModelBase();

			if (!MaterialModelBase)
			{
				return;
			}

			MaterialModel = Cast<UDynamicMaterialModel>(MaterialModelBase);
		}

		if (!MaterialModel)
		{
			return;
		}
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	FDMScopedUITransaction Transaction(LOCTEXT("DropTextureSet", "Drop Texture Set"));

	EditorOnlyData->Modify();
	UDMTextureSetFunctionLibrary::AddTextureSetToModel(EditorOnlyData, InTextureSet, /* Replace */ true);

	OpenMaterialInEditor();
}

TSharedRef<SWidget> SDMMaterialWizard::CreateLayout()
{
	using namespace UE::DynamicMaterialDesigner::Private;

	return SNew(SBox)
		.Padding(SeparationDistance)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, SeparationDistance, 0.0f, 0.f)
			[
				CreateModeSelector()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, SeparationDistance, 0.0f, 0.f)
			[
				SAssignNew(Switcher, SWidgetSwitcher)
				+ SWidgetSwitcher::Slot()
				[
					CreateNewTemplateLayout()
				]
				+ SWidgetSwitcher::Slot()
				[
					CreateNewInstanceLayout()
				]
			]
		];
}

TSharedRef<SWidget> SDMMaterialWizard::CreateModeSelector()
{
	TSharedRef<SHorizontalBox> Container = SNew(SHorizontalBox);

	for (int32 SwitcherIndex = 0; SwitcherIndex < EDMMaterialWizardMode::Count; ++SwitcherIndex)
	{
		switch (SwitcherIndex)
		{
			case EDMMaterialWizardMode::Template:
			{
				Container->AddSlot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.Padding(0.f, 0.f, 5.f, 0.f)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "DetailsView.SectionButton")
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.Padding(FMargin(10.f, 6.f))
						.IsChecked(this, &SDMMaterialWizard::IsModeSelected, EDMMaterialWizardMode::Template)
						.OnCheckStateChanged(this, &SDMMaterialWizard::SetMode, EDMMaterialWizardMode::Template)
						.ToolTipText(LOCTEXT("PresetModeToolTip", "Set up a new Material based on simple channel presets."))
						.Content()
						[
							SNew(STextBlock)
							.TextStyle(FDynamicMaterialEditorStyle::Get(), "BoldFont")
							.Text(LOCTEXT("PresetMode", "New Material"))
						]
					];

				break;
			}

			case EDMMaterialWizardMode::Instance:
			{
				Container->AddSlot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.Padding(0.f, 0.f, 5.f, 0.f)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "DetailsView.SectionButton")
						.HAlign(EHorizontalAlignment::HAlign_Center)
						.Padding(FMargin(10.f, 6.f))
						.IsChecked(this, &SDMMaterialWizard::IsModeSelected, EDMMaterialWizardMode::Instance)
						.OnCheckStateChanged(this, &SDMMaterialWizard::SetMode, EDMMaterialWizardMode::Instance)
						.ToolTipText(LOCTEXT("TemplateModeToolTip", "Create a new Material Designer Instance based on a template."))
						.Content()
						[
							SNew(STextBlock)
							.TextStyle(FDynamicMaterialEditorStyle::Get(), "BoldFont")
							.Text(LOCTEXT("TemplateMode", "New Material Instance"))
						]
					];

				break;
			}
		}
	}

	return Container;
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewTemplateLayout()
{
	using namespace UE::DynamicMaterialDesigner::Private;

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(0.0f, SeparationDistance, 0.0f, TitleContentDistance)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "BoldFont")
			.Text(LOCTEXT("MaterialType", "Material Type"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.Padding(0.0f, 0.f, 0.0f, TitleContentDistance)
		[
			CreateNewTemplate_ChannelPresets()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(0.0f, SeparationDistance, 0.0f, TitleContentDistance)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "BoldFont")
			.Text(LOCTEXT("AvailableChannels", "Available Channels"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SAssignNew(PresetChannelContainer, SBox)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				CreateNewTemplate_ChannelList()
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(0.0f, SeparationDistance, 0.0f, 0.f)
		[
			CreateNewTemplate_AcceptButton()
		];
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewTemplate_ChannelPresets()
{
	using namespace UE::DynamicMaterialDesigner::Private;

	TSharedRef<SWrapBox> ChannelPresets = SNew(SWrapBox)
		.UseAllottedSize(true)
		.InnerSlotPadding(WrapBoxSlotPadding)
		.Orientation(EOrientation::Orient_Horizontal);

	for (const FDMMaterialChannelListPreset& Preset : GetDefault<UDynamicMaterialEditorSettings>()->MaterialChannelPresets)
	{
		ChannelPresets->AddSlot()
			[
				SNew(SCheckBox)
				.Style(FDynamicMaterialEditorStyle::Get(), "DulledSectionButton")
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.Padding(ButtonPadding)
				.IsChecked(this, &SDMMaterialWizard::Preset_GetState, Preset.Name)
				.OnCheckStateChanged(this, &SDMMaterialWizard::Preset_OnChange, Preset.Name)
				[
					SNew(STextBlock)
					.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
					.Text(FText::FromName(Preset.Name))
				]
			];
	}

	return ChannelPresets;
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewTemplate_ChannelList()
{
	using namespace UE::DynamicMaterialDesigner::Private;

	TSharedRef<SWrapBox> ChannelPresets = SNew(SWrapBox)
		.UseAllottedSize(true)
		.InnerSlotPadding(WrapBoxSlotPadding)
		.Orientation(EOrientation::Orient_Horizontal);

	if (const FDMMaterialChannelListPreset* Preset = GetDefault<UDynamicMaterialEditorSettings>()->GetPresetByName(CurrentPreset))
	{
		TArray<EDMMaterialPropertyType> ChannelList;

		for (const TPair<EDMMaterialPropertyType, UDMMaterialProperty*>& Property : GetDefault<UDynamicMaterialModelEditorOnlyData>()->GetMaterialProperties())
		{
			if (Property.Key == EDMMaterialPropertyType::OpacityMask)
			{
				continue;
			}

			if (!Preset->IsPropertyEnabled(Property.Key))
			{
				continue;
			}

			ChannelList.Add(Property.Key);
		}

		for (int32 Index = 0; Index < ChannelList.Num(); ++Index)
		{
			const FText PropertyName = UE::DynamicMaterialEditor::Private::GetMaterialPropertyLongDisplayName(ChannelList[Index]);

			const FText Name = Index == (ChannelList.Num() - 1)
				? FText::Format(LOCTEXT("ListEnd", "{0}."), PropertyName)
				: FText::Format(LOCTEXT("ListEntry", "{0},"), PropertyName);

			ChannelPresets->AddSlot()
				.Padding(TextPadding)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "TinyText")
					.Text(Name)
				];
		}
	}

	return ChannelPresets;
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewInstanceLayout()
{
	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[		
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
			.Padding(3.f)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 0.f, 0.f, 3.f)
				[
					CreateNewInstance_SearchBox()
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					CreateNewInstance_Picker()
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(0.f, 10.f, 0.f, 0.f)
		[
			CreateNewInstance_AcceptButton()
		];
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewInstance_SearchBox()
{
	return SAssignNew(AssetSearchBox, SAssetSearchBox)
		.HintText(NSLOCTEXT("ContentBrowser", "SearchBoxHint", "Search Assets"))
		.OnTextChanged(this, &SDMMaterialWizard::OnSearchBoxChanged)
		.OnTextCommitted(this, &SDMMaterialWizard::OnSearchBoxCommitted)
		.DelayChangeNotificationsWhileTyping(true);
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewInstance_Picker()
{
	FARFilter Filter;
	Filter.ClassPaths.Add(FTopLevelAssetPath(UDynamicMaterialInstance::StaticClass()));
	Filter.bIncludeOnlyOnDiskAssets = true;
	Filter.PackagePaths.Reset();

	TextFilter = MakeShared<FAssetTextFilter>();
	TextFilter->SetIncludeClassName(true);
	TextFilter->SetIncludeAssetPath(false);
	TextFilter->SetIncludeCollectionNames(false);

	AssetView = SNew(SAssetView)
		.InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAssets)
		.SelectionMode(ESelectionMode::Single)
		.OnShouldFilterAsset(this, &SDMMaterialWizard::ShouldFilterOutAsset)
		.OnItemsActivated(this, &SDMMaterialWizard::OnAssetsActivated)
		.InitialBackendFilter(Filter)
		.InitialViewType(EAssetViewType::Tile)
		.InitialAssetSelection({})
		.ShowBottomToolbar(false)
		.AllowDragging(false)
		.CanShowClasses(false)
		.CanShowFolders(true)
		.CanShowReadOnlyFolders(true)
		.ShowViewOptions(false)
		.bShowPathViewFilters(false)
		.FilterRecursivelyWithBackendFilter(false)
		.CanShowRealTimeThumbnails(true)
		.CanShowDevelopersFolder(true)
		.ForceShowEngineContent(true)
		.ForceShowPluginContent(true)
		.HighlightedText(TAttribute<FText>(this, &SDMMaterialWizard::GetSearchText))
		.ThumbnailLabel(EThumbnailLabel::AssetName)
		.AllowFocusOnSync(false)
		.InitialThumbnailSize(EThumbnailSize::Small)
		.ShowTypeInTileView(false)
		.TextFilter(TextFilter);

	AssetView->OverrideShowEngineContent();
	AssetView->OverrideShowPluginContent();
	AssetView->OverrideShowDeveloperContent();
	AssetView->RequestSlowFullListRefresh();

	return AssetView.ToSharedRef();
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewInstance_AcceptButton()
{
	using namespace UE::DynamicMaterialDesigner::Private;

	return SNew(SButton)
		.IsEnabled(this, &SDMMaterialWizard::NewInstance_CanAccept)
		.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
		.ContentPadding(ButtonPadding)
		.OnClicked(this, &SDMMaterialWizard::NewInstanceAccept_OnClick)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
			.Text(LOCTEXT("Continue", "Continue"))
		];
}

TSharedRef<SWidget> SDMMaterialWizard::CreateNewTemplate_AcceptButton()
{
	using namespace UE::DynamicMaterialDesigner::Private;

	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
		.ContentPadding(ButtonPadding)
		.OnClicked(this, &SDMMaterialWizard::NewTemplateAccept_OnClick)
		[
			SNew(STextBlock)
			.TextStyle(FDynamicMaterialEditorStyle::Get(), "RegularFont")
			.Text(LOCTEXT("Continue", "Continue"))
		];
}

ECheckBoxState SDMMaterialWizard::Preset_GetState(FName InPresetName) const
{
	return CurrentPreset == InPresetName
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SDMMaterialWizard::Preset_OnChange(ECheckBoxState InState, FName InPresetName)
{
	if (InState == ECheckBoxState::Checked)
	{
		CurrentPreset = InPresetName;

		if (PresetChannelContainer.IsValid())
		{
			PresetChannelContainer->SetContent(CreateNewTemplate_ChannelList());
		}
	}
}

FReply SDMMaterialWizard::NewTemplateAccept_OnClick()
{
	if (CurrentPreset.IsNone())
	{
		return FReply::Handled();
	}

	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return FReply::Handled();
	}

	if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		SetChannelListInModel(CurrentPreset, MaterialModel);
	}
	else if (MaterialObjectProperty.IsSet())
	{
		if (MaterialObjectProperty->IsValid())
		{
			const FDMScopedUITransaction Transaction(LOCTEXT("CreateMaterialDesignerMaterialInActor", "Create Material Designer Material in Actor"));

			if (UObject* Outer = MaterialObjectProperty->GetOuter())
			{
				Outer->Modify();
			}

			CreateTemplateMaterialInActor(CurrentPreset, MaterialObjectProperty.GetValue());
		}
		else
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid actor property to create new template in."));

			DesignerWidget->ShowSelectPrompt();
		}
	}
	else
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Missing material information for new template."));

		DesignerWidget->ShowSelectPrompt();
	}

	return FReply::Handled();
}

void SDMMaterialWizard::OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModel)
{
	UDynamicMaterialModel* MaterialModel = GetMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	if (InMaterialModel != MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	if (!EditorOnlyData->NeedsWizard())
	{
		EditorOnlyData->GetOnMaterialBuiltDelegate().RemoveAll(this);
		OpenMaterialInEditor();
	}
}

void SDMMaterialWizard::OpenMaterialInEditor()
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	if (MaterialObjectProperty.IsSet())
	{
		if (DesignerWidget->OpenObjectMaterialProperty(*MaterialObjectProperty))
		{
			return;
		}
	}
	else if (UDynamicMaterialModel* MaterialModel = MaterialModelWeak.Get())
	{
		if (DesignerWidget->OpenMaterialModelBase(MaterialModel))
		{
			return;
		}
	}

	DesignerWidget->Empty();
}

ECheckBoxState SDMMaterialWizard::IsModeSelected(EDMMaterialWizardMode InMode) const
{
	if (Switcher.IsValid() && Switcher->GetActiveWidgetIndex() == InMode)
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void SDMMaterialWizard::SetMode(ECheckBoxState InState, EDMMaterialWizardMode InMode)
{
	if (InState == ECheckBoxState::Checked && Switcher.IsValid())
	{
		Switcher->SetActiveWidgetIndex(InMode);
	}
}

void SDMMaterialWizard::OnSearchBoxChanged(const FText& InSearchText)
{
	SetSearchText(InSearchText);
}

void SDMMaterialWizard::OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type InCommitInfo)
{
	SetSearchText(InSearchText);
}

FText SDMMaterialWizard::GetSearchText() const
{
	if (TextFilter.IsValid())
	{
		return TextFilter->GetRawFilterText();
	}

	return FText::GetEmpty();
}

void SDMMaterialWizard::SetSearchText(const FText& InSearchText)
{
	if (InSearchText.ToString().Equals(TextFilter->GetRawFilterText().ToString(), ESearchCase::CaseSensitive))
	{
		return;
	}

	TextFilter->SetRawFilterText(InSearchText);
	AssetView->SetUserSearching(!InSearchText.IsEmpty());
}

bool SDMMaterialWizard::ShouldFilterOutAsset(const FAssetData& InAsset) const
{
	UDynamicMaterialInstance* MaterialInstance = Cast<UDynamicMaterialInstance>(InAsset.GetAsset());

	if (!MaterialInstance)
	{
		return true;
	}

	UDynamicMaterialModelBase* AssetMaterialModelBase = MaterialInstance->GetMaterialModelBase();

	if (!AssetMaterialModelBase)
	{
		return true;
	}

	// Only non-dynamic models can be used as a basis.
	if (!AssetMaterialModelBase->IsA<UDynamicMaterialModel>())
	{
		return true;
	}

	if (UDynamicMaterialModel* MaterialModel = MaterialModelWeak.Get())
	{
		// Can't use it off ourselves
		if (AssetMaterialModelBase == MaterialModel)
		{
			return true;
		}
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(AssetMaterialModelBase);

	if (!EditorOnlyData)
	{
		return true;
	}

	// Can't base it off things which also need a wizard.
	if (EditorOnlyData->NeedsWizard())
	{
		return true;
	}

	return false;
}

bool SDMMaterialWizard::NewInstance_CanAccept() const
{
	return AssetView.IsValid() && !AssetView->GetSelectedAssets().IsEmpty();
}

void SDMMaterialWizard::OnAssetsActivated(TArrayView<const FContentBrowserItem> InSelectedItems, EAssetTypeActivationMethod::Type InActivationMethod)
{
	if (InSelectedItems.IsEmpty() || InActivationMethod == EAssetTypeActivationMethod::Previewed)
	{
		return;
	}

	FAssetData AssetData;

	if (!InSelectedItems[0].Legacy_TryGetAssetData(AssetData))
	{
		return;
	}

	UDynamicMaterialInstance* Instance = Cast<UDynamicMaterialInstance>(AssetData.GetAsset());

	if (!Instance)
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = Cast<UDynamicMaterialModel>(Instance->GetMaterialModelBase());

	if (!MaterialModel)
	{
		return;
	}

	SelectTemplate(MaterialModel);
}

FReply SDMMaterialWizard::NewInstanceAccept_OnClick()
{
	if (!AssetView.IsValid())
	{
		return FReply::Handled();
	}

	TArray<FContentBrowserItem> SelectedItems = AssetView->GetSelectedItems();
	OnAssetsActivated(SelectedItems, EAssetTypeActivationMethod::DoubleClicked);

	return FReply::Handled();
}

void SDMMaterialWizard::OnEnginePreExit()
{
	TextFilter.Reset();
	AssetSearchBox.Reset();
	AssetView.Reset();
	Switcher.Reset();

	ChildSlot.DetachWidget();
}

void SDMMaterialWizard::SelectTemplate(UDynamicMaterialModel* InTemplateModel)
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	if (MaterialObjectProperty.IsSet())
	{
		if (MaterialObjectProperty->IsValid())
		{
			const FDMScopedUITransaction Transaction(LOCTEXT("CreateMaterialDesignerInstanceInActor", "Create Material Designer Instance in Actor"));

			if (UObject* Outer = MaterialObjectProperty->GetOuter())
			{
				Outer->Modify();
			}

			CreateNewDynamicInstanceInActor(InTemplateModel, MaterialObjectProperty.GetValue());
		}
		else
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid actor property to create new dynamic material in."));

			DesignerWidget->ShowSelectPrompt();
		}
	}
	else if (UDynamicMaterialModel* MaterialModel = GetMaterialModel())
	{
		if (UDynamicMaterialInstance* Instance = MaterialModel->GetDynamicMaterialInstance())
		{
			const FDMScopedUITransaction Transaction(LOCTEXT("ReplaceMaterialDesignerModelInMaterial", "Replace Material Designer Model in Material"));
			Instance->Modify();

			CreateDynamicMaterialInInstance(InTemplateModel, Instance);
		}
		else
		{
			UE::DynamicMaterialEditor::Private::LogError(TEXT("Unable to find material instance to create new dynamic material in."));

			DesignerWidget->ShowSelectPrompt();
		}
	}
	else
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Missing material information for dynamic material."));

		DesignerWidget->ShowSelectPrompt();
	}
}

void SDMMaterialWizard::CreateDynamicMaterialInInstance(UDynamicMaterialModel* InTemplateModel, UDynamicMaterialInstance* InToInstance)
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	if (!UDMMaterialModelFunctionLibrary::CreateModelInstanceInMaterial(InTemplateModel, InToInstance))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to create new dynamic model in existing instance."));
		return;
	}

	DesignerWidget->OpenMaterialModelBase(InToInstance->GetMaterialModelBase());
}

void SDMMaterialWizard::CreateNewDynamicInstanceInActor(UDynamicMaterialModel* InFromModel, FDMObjectMaterialProperty& InMaterialObjectProperty)
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	UObject* Outer = InMaterialObjectProperty.GetOuter();

	if (!Outer)
	{
		return;
	}

	UDynamicMaterialInstanceFactory* Factory = NewObject<UDynamicMaterialInstanceFactory>(GetTransientPackage(), TEXT("MaterialDesigner"));

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(Factory->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(),
		Outer,
		NAME_None,
		RF_Transactional | RF_Public,
		nullptr,
		GWarn
	));

	if (!NewInstance)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to create new material instance."));
		return;
	}

	if (!UDMMaterialModelFunctionLibrary::CreateModelInstanceInMaterial(InFromModel, NewInstance))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to create new dynamic model in new instance."));
		return;
	}

	if (!UDMMaterialInstanceFunctionLibrary::SetMaterialInObject(InMaterialObjectProperty, NewInstance))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to set material instance on object."));
		return;
	}

	DesignerWidget->OpenObjectMaterialProperty(InMaterialObjectProperty);
}

void SDMMaterialWizard::SetChannelListInModel(FName InChannelList, UDynamicMaterialModel* InMaterialModel)
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	UDynamicMaterialModel* MaterialModel = GetMaterialModel();

	if (!MaterialModel)
	{
		return;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel);

	if (!EditorOnlyData)
	{
		return;
	}

	EditorOnlyData->GetOnMaterialBuiltDelegate().RemoveAll(this);
	EditorOnlyData->SetChannelListPreset(InChannelList);
	EditorOnlyData->OnWizardComplete();

	if (MaterialObjectProperty.IsSet())
	{
		DesignerWidget->OpenObjectMaterialProperty(*MaterialObjectProperty);
	}
	else
	{
		DesignerWidget->OpenMaterialModelBase(MaterialModel);
	}
}

void SDMMaterialWizard::CreateTemplateMaterialInActor(FName InChannelList, FDMObjectMaterialProperty& InMaterialObjectProperty)
{
	TSharedPtr<SDMMaterialDesigner> DesignerWidget = GetDesignerWidget();

	if (!DesignerWidget.IsValid())
	{
		return;
	}

	UObject* Outer = InMaterialObjectProperty.GetOuter();

	if (!Outer)
	{
		return;
	}

	UDynamicMaterialInstanceFactory* Factory = NewObject<UDynamicMaterialInstanceFactory>(GetTransientPackage(), TEXT("MaterialDesigner"));

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(Factory->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(),
		Outer,
		NAME_None,
		RF_Transactional | RF_Public,
		nullptr,
		GWarn
	));

	if (!NewInstance)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to create new material instance."));
		return;
	}

	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(NewInstance))
	{
		EditorOnlyData->SetChannelListPreset(InChannelList);
		EditorOnlyData->OnWizardComplete();
		EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Immediate);
	}

	if (!UDMMaterialInstanceFunctionLibrary::SetMaterialInObject(InMaterialObjectProperty, NewInstance))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to set material template on object."));
		return;
	}

	DesignerWidget->OpenMaterialModelBase(NewInstance->GetMaterialModelBase());
}

#undef LOCTEXT_NAMESPACE
