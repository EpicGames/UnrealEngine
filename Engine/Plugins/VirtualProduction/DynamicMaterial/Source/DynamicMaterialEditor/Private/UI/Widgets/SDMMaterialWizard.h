// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "DMObjectMaterialProperty.h"
#include "IAssetTypeActions.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class FAssetTextFilter;
class FAssetThumbnail;
class SAssetSearchBox;
class SAssetView;
class SBox;
class SDMMaterialDesigner;
class SToolInputAssetPicker;
class SWidget;
class SWidgetSwitcher;
class UDMTextureSet;
class UDynamicMaterialModel;
enum class ECheckBoxState : uint8;
struct FAssetData;
struct FContentBrowserItem;

class SDMMaterialWizard : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET(SDMMaterialWizard, SCompoundWidget)

	SLATE_BEGIN_ARGS(SDMMaterialWizard)
		: _MaterialModel(nullptr)
		, _MaterialProperty(TOptional<FDMObjectMaterialProperty>())
		{}
		SLATE_ARGUMENT(UDynamicMaterialModel*, MaterialModel)
		SLATE_ARGUMENT(TOptional<FDMObjectMaterialProperty>, MaterialProperty)
	SLATE_END_ARGS()

	virtual ~SDMMaterialWizard() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialDesigner>& InDesignerWidget);

	TSharedPtr<SDMMaterialDesigner> GetDesignerWidget() const;

	UDynamicMaterialModel* GetMaterialModel() const;

	void HandleDrop_CreateTextureSet(const TArray<FAssetData>& InTextureAssets);

	void HandleDrop_TextureSet(UDMTextureSet* InTextureSet);

protected:
	enum EDMMaterialWizardMode : uint8
	{
		Template = 0,
		Instance = 1,

		Count = 2
	};

	TWeakPtr<SDMMaterialDesigner> DesignerWidgetWeak;
	FName CurrentPreset;
	TSharedPtr<SBox> PresetChannelContainer;
	TWeakObjectPtr<UDynamicMaterialModel> MaterialModelWeak;
	TOptional<FDMObjectMaterialProperty> MaterialObjectProperty;
	TSharedPtr<SWidgetSwitcher> Switcher;
	TSharedPtr<SAssetView> AssetView;
	TSharedPtr<SAssetSearchBox> AssetSearchBox;
	TSharedPtr<FAssetTextFilter> TextFilter;

	/** Creation of widgets. */
	TSharedRef<SWidget> CreateLayout();
	TSharedRef<SWidget> CreateModeSelector();
	TSharedRef<SWidget> CreateNewTemplateLayout();
	TSharedRef<SWidget> CreateNewTemplate_ChannelPresets();
	TSharedRef<SWidget> CreateNewTemplate_ChannelList();
	TSharedRef<SWidget> CreateNewTemplate_AcceptButton();
	TSharedRef<SWidget> CreateNewInstanceLayout();
	TSharedRef<SWidget> CreateNewInstance_SearchBox();
	TSharedRef<SWidget> CreateNewInstance_Picker();
	TSharedRef<SWidget> CreateNewInstance_AcceptButton();

	/** Attributes and Events */
	ECheckBoxState Preset_GetState(FName InPresetName) const;
	void Preset_OnChange(ECheckBoxState InState, FName InPresetName);

	FReply NewTemplateAccept_OnClick();

	void OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModel);

	void OpenMaterialInEditor();

	ECheckBoxState IsModeSelected(EDMMaterialWizardMode InMode) const;

	void SetMode(ECheckBoxState InState, EDMMaterialWizardMode InMode);

	void OnSearchBoxChanged(const FText& InSearchText);
	void OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type InCommitInfo);

	FText GetSearchText() const;
	void SetSearchText(const FText& InSearchText);

	bool ShouldFilterOutAsset(const FAssetData& InAsset) const;

	bool NewInstance_CanAccept() const;

	void OnAssetsActivated(TArrayView<const FContentBrowserItem> InSelectedItems, EAssetTypeActivationMethod::Type InActivationMethod);

	FReply NewInstanceAccept_OnClick();

	void OnEnginePreExit();

	/** Operations */
	void SelectTemplate(UDynamicMaterialModel* InTemplateModel);

	void CreateDynamicMaterialInInstance(UDynamicMaterialModel* InTemplateModel, UDynamicMaterialInstance* InToInstance);

	void CreateNewDynamicInstanceInActor(UDynamicMaterialModel* InFromModel, FDMObjectMaterialProperty& InMaterialObjectProperty);

	void SetChannelListInModel(FName InChannelList, UDynamicMaterialModel* InMaterialModel);

	void CreateTemplateMaterialInActor(FName InChannelList, FDMObjectMaterialProperty& InMaterialObjectProperty);
};
