// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Visualizers/SDMMaterialComponentPreview.h"

#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialValue.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "DynamicMaterialModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "UI/Utils/DMPreviewMaterialManager.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "Widgets/Images/SImage.h"

SDMMaterialComponentPreview::SDMMaterialComponentPreview()
	: Brush(FSlateMaterialBrush(FVector2D(1.f, 1.f)))
	, PreviewSize(FVector2D(48.f))
{
	Brush.SetUVRegion(FBox2f(FVector2f::ZeroVector, FVector2f::UnitVector));
}

SDMMaterialComponentPreview::~SDMMaterialComponentPreview()
{
	if (EndOfFrameDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(EndOfFrameDelegateHandle);
		EndOfFrameDelegateHandle.Reset();
	}

	if (!FDynamicMaterialModule::AreUObjectsSafe())
	{
		return;
	}

	if (UDMMaterialComponent* Component = ComponentWeak.Get())
	{
		Component->GetOnUpdate().RemoveAll(this);

		if (TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin())
		{
			EditorWidget->GetPreviewMaterialManager()->CreatePreviewMaterial(Component);
		}
	}
}

void SDMMaterialComponentPreview::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialComponent* InComponent)
{
	EditorWidgetWeak = InEditorWidget;
	ComponentWeak = InComponent;
	PreviewSize = InArgs._PreviewSize;

	PreviewMaterialBaseWeak = InEditorWidget->GetPreviewMaterialManager()->CreatePreviewMaterial(InComponent);
	PreviewMaterialDynamicWeak = InEditorWidget->GetPreviewMaterialManager()->CreatePreviewMaterialDynamic(PreviewMaterialBaseWeak.Get());
	MaterialModelBaseWeak = InEditorWidget->GetPreviewMaterialModelBase();

	SetCanTick(true);

	if (ensure(IsValid(InComponent)))
	{
		InComponent->GetOnUpdate().AddSP(this, &SDMMaterialComponentPreview::OnComponentUpdated);
		OnComponentUpdated(InComponent, InComponent, EDMUpdateType::Structure);
	}

	ChildSlot
	[
		SAssignNew(PreviewImage, SImage)
		.Image(&Brush)
		.DesiredSizeOverride(PreviewSize)
	];
}

FSlateMaterialBrush& SDMMaterialComponentPreview::GetBrush()
{
	return Brush;
}

const FVector2D& SDMMaterialComponentPreview::GetPreviewSize() const
{
	return PreviewSize;
}

void SDMMaterialComponentPreview::SetPreviewSize(const FVector2D& InSize)
{
	if (PreviewSize.Equals(InSize))
	{
		return;
	}

	PreviewSize = InSize;

	if (PreviewImage.IsValid())
	{
		PreviewImage->SetDesiredSizeOverride(PreviewSize);
	}
}

UDMMaterialComponent* SDMMaterialComponentPreview::GetComponent() const
{
	return ComponentWeak.Get();
}

UMaterial* SDMMaterialComponentPreview::GetPreviewMaterial() const
{
	return PreviewMaterialBaseWeak.Get();
}

UMaterialInstanceDynamic* SDMMaterialComponentPreview::GetPreviewMaterialDynamic() const
{
	return PreviewMaterialDynamicWeak.Get();
}

void SDMMaterialComponentPreview::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (!PreviewMaterialBaseWeak.IsValid() || !PreviewMaterialDynamicWeak.IsValid())
	{
		Brush.SetMaterial(nullptr);
	}	
}

void SDMMaterialComponentPreview::OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	// We're going to recreate the material anyway so this doesn't need to run.
	if (EndOfFrameDelegateHandle.IsValid())
	{
		return;
	}

	if (InComponent != ComponentWeak.Get() || !IsValid(InComponent) || !InComponent->IsComponentValid())
	{
		return;
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UMaterialInstanceDynamic* MID = PreviewMaterialDynamicWeak.Get();

	if (MID && !EnumHasAnyFlags(InUpdateType, EDMUpdateType::Structure))
	{
		if (UDMMaterialValue* Value = Cast<UDMMaterialValue>(InSource))
		{
			Value->SetMIDParameter(MID);
		}
		else if (UDMMaterialValueDynamic* ValueDynamic = Cast<UDMMaterialValueDynamic>(InSource))
		{
			ValueDynamic->SetMIDParameter(MID);
		}
		else if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(InSource))
		{
			TextureUV->SetMIDParameters(MID);
		}
		else if (UDMTextureUVDynamic* TextureUVDynamic = Cast<UDMTextureUVDynamic>(InSource))
		{
			TextureUVDynamic->SetMIDParameters(MID);
		}
	}
	else if (!EndOfFrameDelegateHandle.IsValid())
	{
		EndOfFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddSP(this, &SDMMaterialComponentPreview::OnEndOfFrame);
	}
}

void SDMMaterialComponentPreview::OnEndOfFrame()
{
	if (EndOfFrameDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(EndOfFrameDelegateHandle);
		EndOfFrameDelegateHandle.Reset();
	}

	RecreateMaterial();
}

void SDMMaterialComponentPreview::RecreateMaterial()
{
	TSharedPtr<SDMMaterialEditor> EditorWidget = EditorWidgetWeak.Pin();

	if (!EditorWidget.IsValid())
	{
		return;
	}

	UDMMaterialComponent* Component = ComponentWeak.Get();

	if (!Component)
	{
		return;
	}

	UMaterial* PreviewMaterialBase = PreviewMaterialBaseWeak.Get();

	if (!PreviewMaterialBase)
	{
		PreviewMaterialBase = EditorWidget->GetPreviewMaterialManager()->CreatePreviewMaterial(Component);
	}

	if (UDMMaterialStage* Stage = Cast<UDMMaterialStage>(Component))
	{
		Stage->GeneratePreviewMaterial(PreviewMaterialBase);

		EditorWidget->GetPreviewMaterialManager()->FreePreviewMaterialDynamic(PreviewMaterialBase);
		PreviewMaterialDynamicWeak = EditorWidget->GetPreviewMaterialManager()->CreatePreviewMaterialDynamic(PreviewMaterialBase);

		UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

		if (UDynamicMaterialModel* PreviewMaterialModel = Cast<UDynamicMaterialModel>(PreviewMaterialModelBase))
		{
			PreviewMaterialModel->ApplyComponents(PreviewMaterialDynamicWeak.Get());
		}
		else if (UDynamicMaterialModelDynamic* PreviewMaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(PreviewMaterialModelBase))
		{
			PreviewMaterialModelDynamic->ApplyComponents(PreviewMaterialDynamicWeak.Get());
		}

		Brush.SetMaterial(PreviewMaterialDynamicWeak.Get());
	}
	else if (UDMMaterialProperty* Property = Cast<UDMMaterialProperty>(Component))
	{
		Property->GeneratePreviewMaterial(PreviewMaterialBase);

		EditorWidget->GetPreviewMaterialManager()->FreePreviewMaterialDynamic(PreviewMaterialBase);
		PreviewMaterialDynamicWeak = EditorWidget->GetPreviewMaterialManager()->CreatePreviewMaterialDynamic(PreviewMaterialBase);

		UDynamicMaterialModelBase* PreviewMaterialModelBase = EditorWidget->GetPreviewMaterialModelBase();

		if (UDynamicMaterialModel* PreviewMaterialModel = Cast<UDynamicMaterialModel>(PreviewMaterialModelBase))
		{
			PreviewMaterialModel->ApplyComponents(PreviewMaterialDynamicWeak.Get());
		}
		else if (UDynamicMaterialModelDynamic* PreviewMaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(PreviewMaterialModelBase))
		{
			PreviewMaterialModelDynamic->ApplyComponents(PreviewMaterialDynamicWeak.Get());
		}

		Brush.SetMaterial(PreviewMaterialDynamicWeak.Get());
	}
}
