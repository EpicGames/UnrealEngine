// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Delegates/IDelegateInstance.h"
#include "DMEDefs.h"
#include "SlateMaterialBrush.h"
#include "UI/Utils/DMWidgetSlot.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SDMMaterialEditor;
class UDMMaterialComponent;
class UDMMaterialValueDynamic;
class UDMTextureUV;
class UDMTextureUVDynamic;
class UDynamicMaterialModelBase;
class UDynamicMaterialModelDynamic;

class SDMMaterialComponentPreview : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMMaterialComponentPreview)
		: _PreviewSize(FVector2D(48.f, 48.f))
		{}
		SLATE_ARGUMENT(FVector2D, PreviewSize)
	SLATE_END_ARGS()

	SDMMaterialComponentPreview();

	virtual ~SDMMaterialComponentPreview() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialComponent* InComponent);

	FSlateMaterialBrush& GetBrush();

	const FVector2D& GetPreviewSize() const;

	void SetPreviewSize(const FVector2D& InSize);

	UDMMaterialComponent* GetComponent() const;

	UMaterial* GetPreviewMaterial() const;

	UMaterialInstanceDynamic* GetPreviewMaterialDynamic() const;

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<UDMMaterialComponent> ComponentWeak;
	TWeakObjectPtr<UDynamicMaterialModelBase> MaterialModelBaseWeak;
	TWeakObjectPtr<UMaterial> PreviewMaterialBaseWeak;
	TWeakObjectPtr<UMaterialInstanceDynamic> PreviewMaterialDynamicWeak;
	FSlateMaterialBrush Brush;
	FDelegateHandle EndOfFrameDelegateHandle;
	TSharedPtr<SImage> PreviewImage;
	FVector2D PreviewSize;

	void OnComponentUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	void OnEndOfFrame();

	void RecreateMaterial();
};
