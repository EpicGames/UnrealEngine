// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Visualizers/SDMTextureUVVisualizer.h"

#include "Brushes/SlateColorBrush.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "DynamicMaterialEditorModule.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/Visualizers/SDMMaterialComponentPreview.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SDMTextureUVVisualizer"

namespace UE::DynamicMaterialEditor::Private
{
	/** In details panel visualizer is small and square. */
	const FVector2D TextureUVVisualizerImageSize = FVector2D(128.f, 128.f);

	/** Popout visualizer is large and aspected differently. */
	const FVector2D TextureUVVisualizerPopoutImageSize = FVector2D(1024.f, 768.f);

	/** Outer size of the position handle. */
	constexpr float TextureUVVisualizerLargeRadius = 25.f;

	/** Inner size of the position handle. */
	constexpr float TextureUVVisualizerSmallRadius = 10.f;

	/** "Radius" circle handle is for mouse interaction. */
	constexpr float TextureUVVisualizerCircleHandleRadius = 5.f;

	/** Based distance the circle handle is from the center compared to the width of the image. */
	constexpr float TextureUVVisualizerCircleHandleBaseRadiusMultiplier = 0.4f;

	/** Scale for the clamped version of the base cirle base radius. */
	constexpr float TextureUVVisualizerCircleHandleBaseRadiusClampedMultiplier = 4.f;

	/** Size of the border around the center square */
	constexpr float TextureUVVisualizerBorder = 8.f;
}

SDMTextureUVVisualizer::SDMTextureUVVisualizer()
	: bIsPopout(false)
	, bPivotEditMode(false)
	, CurrentAbsoluteSize(FVector2f::ZeroVector)
	, CurrentAbsoluteCenter(FVector2f::ZeroVector)
	, ScrubbingMode(EScrubbingMode::None)
	, ScrubbingStartAbsoluteCenter(FVector2f::ZeroVector)
	, ScrubbingStartAbsoluteMouse(FVector2f::ZeroVector)
	, HandleAxis(EHandleAxis::None)
	, ValueStart(FVector2D::ZeroVector)
	, bInvertTiling(false)
{
}

void SDMTextureUVVisualizer::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage)
{
	check(InMaterialStage);
	check(InArgs._TextureUV || InArgs._TextureUVDynamic);

	EditorWidgetWeak = InEditorWidget;
	StageWeak = InMaterialStage;

	TextureUVComponentWeak = InArgs._TextureUV
		? static_cast<UDMMaterialComponent*>(InArgs._TextureUV)
		: static_cast<UDMMaterialComponent*>(InArgs._TextureUVDynamic);

	bIsPopout = InArgs._IsPopout;

	SetCanTick(true);

	using namespace UE::DynamicMaterialEditor::Private;

	ChildSlot
	[
		SAssignNew(StagePreview, SDMMaterialComponentPreview, InEditorWidget, InMaterialStage)
		.PreviewSize(bIsPopout ? TextureUVVisualizerPopoutImageSize : TextureUVVisualizerImageSize)
	];
}

SDMTextureUVVisualizer::EScrubbingMode SDMTextureUVVisualizer::GetScrubbingMode() const
{
	return ScrubbingMode;
}

bool SDMTextureUVVisualizer::IsInPivotEditMode() const
{
	return bPivotEditMode;
}

void SDMTextureUVVisualizer::SetInPivotEditMode(bool bInEditingPivot)
{
	bPivotEditMode = bInEditingPivot;
}

void SDMTextureUVVisualizer::TogglePivotEditMode()
{
	SetInPivotEditMode(!IsInPivotEditMode());
}

UDMMaterialStage* SDMTextureUVVisualizer::GetStage() const
{
	return StageWeak.Get();
}

UDMMaterialComponent* SDMTextureUVVisualizer::GetTextureUVComponent() const
{
	return TextureUVComponentWeak.Get();
}

UDMTextureUV* SDMTextureUVVisualizer::GetTextureUV() const
{
	return Cast<UDMTextureUV>(GetTextureUVComponent());
}

UDMTextureUVDynamic* SDMTextureUVVisualizer::GetTextureUVDynamic() const
{
	return Cast<UDMTextureUVDynamic>(GetTextureUVComponent());
}

const FVector2D& SDMTextureUVVisualizer::GetOffset() const
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		return TextureUVDynamic->GetOffset();
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		return TextureUV->GetOffset();
	}

	return GetDefault<UDMTextureUV>()->GetOffset();
}

bool SDMTextureUVVisualizer::SetOffset(const FVector2D& InOffset)
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		TextureUVDynamic->SetOffset(InOffset);
		return true;
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		TextureUV->SetOffset(InOffset);
		return true;
	}

	return false;
}

float SDMTextureUVVisualizer::GetRotation() const
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		return TextureUVDynamic->GetRotation();
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		return TextureUV->GetRotation();
	}

	return GetDefault<UDMTextureUV>()->GetRotation();
}

bool SDMTextureUVVisualizer::SetRotation(float InRotation)
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		TextureUVDynamic->SetRotation(InRotation);
		return true;
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		TextureUV->SetRotation(InRotation);
		return true;
	}

	return false;
}

const FVector2D& SDMTextureUVVisualizer::GetTiling() const
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		return TextureUVDynamic->GetTiling();
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		return TextureUV->GetTiling();
	}

	return GetDefault<UDMTextureUV>()->GetTiling();
}

bool SDMTextureUVVisualizer::SetTiling(const FVector2D& InTiling)
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		TextureUVDynamic->SetTiling(InTiling);
		return true;
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		TextureUV->SetTiling(InTiling);
		return true;
	}

	return false;
}

const FVector2D& SDMTextureUVVisualizer::GetPivot() const
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		return TextureUVDynamic->GetPivot();
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		return TextureUV->GetPivot();
	}

	return GetDefault<UDMTextureUV>()->GetPivot();
}

bool SDMTextureUVVisualizer::SetPivot(const FVector2D& InPivot)
{
	if (UDMTextureUVDynamic* TextureUVDynamic = GetTextureUVDynamic())
	{
		TextureUVDynamic->SetPivot(InPivot);
		return true;
	}

	if (UDMTextureUV* TextureUV = GetTextureUV())
	{
		TextureUV->SetPivot(InPivot);
		return true;
	}

	return false;
}

void SDMTextureUVVisualizer::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	CurrentAbsoluteSize = StagePreview->GetTickSpaceGeometry().GetAbsoluteSize();
	CurrentAbsoluteCenter = StagePreview->GetTickSpaceGeometry().GetAbsolutePosition() + (CurrentAbsoluteSize * 0.5f);

	if (ScrubbingMode != EScrubbingMode::None)
	{
		if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
		{
			SetScrubbingMode(EScrubbingMode::None, EHandleAxis::None);
		}
		else
		{
			UpdateScrub();
		}
	}
	else
	{
		ScrubbingTransaction.Reset();
	}

	if (bIsPopout)
	{
		UpdatePopoutUVs();
	}
}

FReply SDMTextureUVVisualizer::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	const bool bIsLeftMouse = InMouseEvent.GetEffectingButton() != EKeys::RightMouseButton;

	if (!bIsLeftMouse)
	{
		return SCompoundWidget::OnMouseButtonDown(InGeometry, InMouseEvent);
	}

	// Could use the mouse event, but I want a consistent value source.
	const FVector2f MousePosition = FSlateApplication::Get().GetCursorPos();
	const bool bResetToDefault = InMouseEvent.GetModifierKeys().IsControlDown();

	if (TryClickCenterHandle(MousePosition, bResetToDefault))
	{
		return FReply::Handled();
	}

	if (bPivotEditMode && TryClickCircleHandle(MousePosition, bResetToDefault))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FCursorReply SDMTextureUVVisualizer::OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InCursorEvent) const
{
	if (HandleAxis != EHandleAxis::None)
	{
		return FCursorReply::Cursor(EMouseCursor::Crosshairs);
	}

	const FVector2f MousePosition = FSlateApplication::Get().GetCursorPos();

	// If the UV image isn't under the mouse, we don't need to do anything.
	if (!StagePreview->GetTickSpaceGeometry().IsUnderLocation(MousePosition))
	{
		return SCompoundWidget::OnCursorQuery(InGeometry, InCursorEvent);
	}

	// Work out if the mouse is over a part of the center handle
	EHandleAxis Axis = GetCenterHandleAxis(MousePosition);

	switch (Axis)
	{
		case EHandleAxis::X:
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);

		case EHandleAxis::Y:
			return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);

		case EHandleAxis::XY:
			return FCursorReply::Cursor(EMouseCursor::CardinalCross);
	}

	if (bPivotEditMode)
	{
		// Work out if the mouse is over a part of the circle handle
		Axis = GetCircleHandleAxis(MousePosition);

		switch (Axis)
		{
			case EHandleAxis::X:
				return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);

			case EHandleAxis::Y:
				return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);

			case EHandleAxis::XY:
				return FCursorReply::Cursor(EMouseCursor::GrabHand);
		}
	}

	return SCompoundWidget::OnCursorQuery(InGeometry, InCursorEvent);
}

int32 SDMTextureUVVisualizer::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, 
	FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	if (!GetTextureUVComponent())
	{
		return InLayerId;
	}

	++InLayerId;

	using namespace UE::DynamicMaterialEditor::Private;

	static const FSlateColorBrush WhiteBrush = FSlateColorBrush(FLinearColor(1.f, 1.f, 1.f, 1.f));
	static const FLinearColor BorderColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
	static const FLinearColor NormalColor = FLinearColor(1.f, 1.f, 1.f, 0.25f);
	static const FLinearColor HighlightColor = FStyleColors::Primary.GetSpecifiedColor();

	const float Rotation = GetRotation();
	const float RotationRadians = FMath::DegreesToRadians(Rotation);
	const FVector2f LocalCenterOffset = GetOffsetLocation(InAllottedGeometry.GetLocalSize());
	const FVector2f LocalPivotOffset = GetPivotLocation(InAllottedGeometry.GetLocalSize());

	auto DrawRotatedBox = [this, &OutDrawElements, &InLayerId, &InAllottedGeometry, &Rotation, &LocalCenterOffset, &LocalPivotOffset]
		(const FVector2f& InLocation, const FVector2f& InSize, const FLinearColor& InColor, float InRotationRadians)
		{
			const FVector2f& BaseLocation = bPivotEditMode ? LocalPivotOffset : LocalCenterOffset;
			const FVector2f Location = BaseLocation + InLocation - InSize * 0.5f;

			const FVector2f RotationOffset = BaseLocation - Location;

			FSlateDrawElement::MakeRotatedBox(
				OutDrawElements,
				InLayerId,
				InAllottedGeometry.ToPaintGeometry(
					InSize,
					FSlateLayoutTransform(Location)
				),
				&WhiteBrush,
				ESlateDrawEffect::NoPixelSnapping,
				InRotationRadians,
				RotationOffset,
				FSlateDrawElement::ERotationSpace::RelativeToElement,
				InColor
			);

			return Location;
		};

	auto DrawRotatedBorderBox = [&DrawRotatedBox](const FVector2f& InLocation, const FVector2f& InSize, const FLinearColor& InInnerColor, float InRotationRadians)
		{
			DrawRotatedBox(InLocation, InSize, BorderColor, InRotationRadians);
			return DrawRotatedBox(InLocation, InSize - FVector2f(TextureUVVisualizerBorder, TextureUVVisualizerBorder), InInnerColor, InRotationRadians);
		};

	/** Center Handle */
	DrawRotatedBorderBox(
		FVector2f::ZeroVector,
		FVector2f(TextureUVVisualizerLargeRadius, TextureUVVisualizerLargeRadius) * 2.f,
		NormalColor,
		bPivotEditMode ? 0 : RotationRadians
	);

	if (ScrubbingMode == EScrubbingMode::Offset || ScrubbingMode == EScrubbingMode::Pivot)
	{
		if (HandleAxis != EHandleAxis::Y)
		{
			DrawRotatedBox(
				FVector2f(TextureUVVisualizerLargeRadius - TextureUVVisualizerSmallRadius * 0.5f - 2.f, 0.f),
				FVector2f(TextureUVVisualizerSmallRadius, TextureUVVisualizerLargeRadius * 2.f - TextureUVVisualizerBorder * 0.5f),
				HighlightColor,
				bPivotEditMode ? 0 : RotationRadians
			);

			DrawRotatedBox(
				FVector2f(-TextureUVVisualizerLargeRadius + TextureUVVisualizerSmallRadius * 0.5f + 2.f, 0.f),
				FVector2f(TextureUVVisualizerSmallRadius, TextureUVVisualizerLargeRadius * 2.f - TextureUVVisualizerBorder * 0.5f),
				HighlightColor,
				bPivotEditMode ? 0 : RotationRadians
			);
		}

		if (HandleAxis != EHandleAxis::X)
		{
			DrawRotatedBox(
				FVector2f(0.f, TextureUVVisualizerLargeRadius - TextureUVVisualizerSmallRadius * 0.5f - 2.f),
				FVector2f(TextureUVVisualizerLargeRadius * 2.f - TextureUVVisualizerBorder * 0.5f, TextureUVVisualizerSmallRadius),
				HighlightColor,
				bPivotEditMode ? 0 : RotationRadians
			);

			DrawRotatedBox(
				FVector2f(0.f, -TextureUVVisualizerLargeRadius + TextureUVVisualizerSmallRadius * 0.5f + 2.f),
				FVector2f(TextureUVVisualizerLargeRadius * 2.f - TextureUVVisualizerBorder * 0.5f, TextureUVVisualizerSmallRadius),
				HighlightColor,
				bPivotEditMode ? 0 : RotationRadians
			);
		}
	}

	if (bPivotEditMode)
	{
		/** Tiling Handles */
		float RadiusAtAngle[16];

		for (int32 Direction = 0; Direction < 16; ++Direction)
		{
			RadiusAtAngle[Direction] = GetCircleHandleRadiusAtAngle(static_cast<float>(Direction) * 22.5f);
		}

		DrawRotatedBorderBox(
			FVector2f(0.f, RadiusAtAngle[0]),
			FVector2f(TextureUVVisualizerLargeRadius, TextureUVVisualizerLargeRadius),
			(ScrubbingMode == EScrubbingMode::Tiling && HandleAxis == EHandleAxis::Y) ? HighlightColor : NormalColor,
			RotationRadians
		);

		DrawRotatedBorderBox(
			FVector2f(RadiusAtAngle[4], 0.f),
			FVector2f(TextureUVVisualizerLargeRadius, TextureUVVisualizerLargeRadius),
			(ScrubbingMode == EScrubbingMode::Tiling && HandleAxis == EHandleAxis::X) ? HighlightColor : NormalColor,
			RotationRadians
		);

		DrawRotatedBorderBox(
			FVector2f(0.f, -RadiusAtAngle[8]),
			FVector2f(TextureUVVisualizerLargeRadius, TextureUVVisualizerLargeRadius),
			(ScrubbingMode == EScrubbingMode::Tiling && HandleAxis == EHandleAxis::Y) ? HighlightColor : NormalColor,
			RotationRadians
		);

		DrawRotatedBorderBox(
			FVector2f(-RadiusAtAngle[12], 0.f),
			FVector2f(TextureUVVisualizerLargeRadius, TextureUVVisualizerLargeRadius),
			(ScrubbingMode == EScrubbingMode::Tiling && HandleAxis == EHandleAxis::X) ? HighlightColor : NormalColor,
			RotationRadians
		);

		/** Rotation Handles */
		auto AngleRadiusToLocation = [&LocalPivotOffset, &Rotation, &RadiusAtAngle](int32 InAngleIndex)
			{
				const float Angle = FMath::DegreesToRadians(static_cast<float>(InAngleIndex) * 22.5f - Rotation);
				return LocalPivotOffset + (FVector2f(FMath::Sin(Angle), FMath::Cos(Angle)) * RadiusAtAngle[InAngleIndex]);
			};

		auto DrawPartialCicle = [this, &OutDrawElements, &InLayerId, &InAllottedGeometry, &AngleRadiusToLocation](int32 InStartIndex)
			{
				const TArray<FVector2f> Points = {
					AngleRadiusToLocation(InStartIndex),
					AngleRadiusToLocation(InStartIndex + 1),
					AngleRadiusToLocation(InStartIndex + 2)
				};

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					InLayerId,
					InAllottedGeometry.ToPaintGeometry(),
					Points,
					ESlateDrawEffect::NoPixelSnapping,
					BorderColor,
					true,
					TextureUVVisualizerSmallRadius
				);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					InLayerId,
					InAllottedGeometry.ToPaintGeometry(),
					Points,
					ESlateDrawEffect::NoPixelSnapping,
					ScrubbingMode == EScrubbingMode::Rotation ? HighlightColor : NormalColor,
					true,
					TextureUVVisualizerSmallRadius - 2.f
				);
			};

		DrawPartialCicle(1);
		DrawPartialCicle(5);
		DrawPartialCicle(9);
		DrawPartialCicle(13);
	}

	return InLayerId;
}

bool SDMTextureUVVisualizer::HasValidGeometry() const
{
	// The chances of the center being at anywhere near 0,0 is remote...
	return !FMath::IsNearlyZero(CurrentAbsoluteSize.X) && !FMath::IsNearlyZero(CurrentAbsoluteSize.Y)
		&& !FMath::IsNearlyZero(CurrentAbsoluteCenter.X) && !FMath::IsNearlyZero(CurrentAbsoluteCenter.Y);
}

float SDMTextureUVVisualizer::GetCircleHandleBaseRadius() const
{
	using namespace UE::DynamicMaterialEditor::Private;

	const FVector2f ImageSize = StagePreview->GetBrush().GetImageSize();
	const float CircleHandleRadius = FMath::Min(ImageSize.X, ImageSize.Y) * TextureUVVisualizerCircleHandleBaseRadiusMultiplier;

	return FMath::Clamp(CircleHandleRadius, 10, 50.f) * TextureUVVisualizerCircleHandleBaseRadiusClampedMultiplier;
}

FVector2f SDMTextureUVVisualizer::ApplyTextureUVTransform(const FVector2f& InUV) const
{
	FVector2f Offset = static_cast<FVector2f>(GetOffset());
	Offset.Y *= -1.f;

	const float Rotation = GetRotation();
	const FVector2f& Tiling = static_cast<FVector2f>(GetTiling());
	const FVector2f& Pivot = static_cast<FVector2f>(GetPivot());

	FVector2f TransformedUV = InUV;
	TransformedUV -= Pivot;

	if (!FMath::IsNearlyZero(Rotation))
	{
		TransformedUV = TransformedUV.GetRotated(Rotation);
	}

	TransformedUV /= Tiling;
	TransformedUV += Pivot;

	if (!FMath::IsNearlyZero(Rotation))
	{
		TransformedUV += (Offset / Tiling).GetRotated(Rotation);
	}
	else
	{
		TransformedUV += Offset / Tiling;
	}

	return TransformedUV;
}

FVector2f SDMTextureUVVisualizer::GetOffsetLocation(const FVector2f& InSize) const
{
	static const FVector2f Center = {0.5f, 0.5f};

	if (bIsPopout)
	{
		return ToPopoutLocation(FVector2f::UnitVector, ApplyTextureUVTransform(Center)) * InSize;
	}

	return ApplyTextureUVTransform(Center) * InSize;
}

FVector2f SDMTextureUVVisualizer::GetPivotLocation(const FVector2f& InSize) const
{
	if (bIsPopout)
	{
		return ToPopoutLocation(FVector2f::UnitVector, static_cast<FVector2f>(GetPivot())) * InSize;
	}

	return static_cast<FVector2f>(GetPivot()) * InSize;
}

FVector2f SDMTextureUVVisualizer::GetAbsoluteOffsetLocation() const
{
	return CurrentAbsoluteCenter - (CurrentAbsoluteSize * 0.5f) + GetOffsetLocation(CurrentAbsoluteSize);
}

FVector2f SDMTextureUVVisualizer::GetAbsolutePivotLocation() const
{
	return CurrentAbsoluteCenter - (CurrentAbsoluteSize * 0.5f) + GetPivotLocation(CurrentAbsoluteSize);
}

float SDMTextureUVVisualizer::GetCircleHandleRadiusAtAngle(float InAngle) const
{
	const float BaseDistance = GetCircleHandleBaseRadius();

	if (!IsValid(GetTextureUVComponent()))
	{
		return BaseDistance;
	}

	const FVector2D& Tiling = GetTiling();

	if (FMath::IsNearlyEqual(Tiling.X, Tiling.Y))
	{
		return BaseDistance / Tiling.X;
	}

	const float RadianAngle = FMath::DegreesToRadians(InAngle);

	const FVector2f TilingFloat = static_cast<FVector2f>(Tiling);

	const FVector2f Offset = {
		FMath::Sin(RadianAngle) / TilingFloat.X,
		FMath::Cos(RadianAngle) / TilingFloat.Y
	};

	return BaseDistance * Offset.Size();
}

SDMTextureUVVisualizer::EHandleAxis SDMTextureUVVisualizer::GetCenterHandleAxis(const FVector2f& InAbsolutePosition) const
{
	if (!HasValidGeometry())
	{
		return EHandleAxis::None;
	}

	if (!IsValid(GetTextureUVComponent()))
	{
		return EHandleAxis::None;
	}

	using namespace UE::DynamicMaterialEditor::Private;

	const FVector2f HandleLocation = bPivotEditMode ? GetAbsolutePivotLocation() : GetAbsoluteOffsetLocation();
	FVector2f HandleOffset = InAbsolutePosition - HandleLocation;
		
	if (!bPivotEditMode)
	{
		const float Rotation = GetRotation();

		if (!FMath::IsNearlyZero(Rotation))
		{
			HandleOffset = HandleOffset.GetRotated(-Rotation);
		}
	}

	HandleOffset.X = FMath::Abs(HandleOffset.X);
	HandleOffset.Y = FMath::Abs(HandleOffset.Y);

	const float UIScale = GetTickSpaceGeometry().Scale;

	if (UIScale > 0)
	{
		HandleOffset /= UIScale;
	}

	static constexpr float LargeMinusSmallRadius = TextureUVVisualizerLargeRadius - TextureUVVisualizerSmallRadius;

	if (HandleOffset.X <= LargeMinusSmallRadius && HandleOffset.Y <= LargeMinusSmallRadius)
	{
		return EHandleAxis::XY;
	}

	if (HandleOffset.X <= LargeMinusSmallRadius && HandleOffset.Y <= TextureUVVisualizerLargeRadius)
	{
		return EHandleAxis::Y;
	}

	if (HandleOffset.X <= TextureUVVisualizerLargeRadius && HandleOffset.Y <= LargeMinusSmallRadius)
	{
		return EHandleAxis::X;
	}

	if (HandleOffset.X <= TextureUVVisualizerLargeRadius && HandleOffset.Y <= TextureUVVisualizerLargeRadius)
	{
		return EHandleAxis::XY;
	}

	return EHandleAxis::None;
}

SDMTextureUVVisualizer::EHandleAxis SDMTextureUVVisualizer::GetCircleHandleAxis(const FVector2f& InAbsolutePosition) const
{
	if (!IsValid(GetTextureUVComponent()))
	{
		return EHandleAxis::None;
	}

	using namespace UE::DynamicMaterialEditor::Private;

	float UIScale = GetTickSpaceGeometry().Scale;

	if (FMath::IsNearlyZero(UIScale))
	{
		UIScale = 1.f;
	}

	const FVector2f HandleOffset = (InAbsolutePosition - GetAbsolutePivotLocation()) / UIScale;

	const float DistanceFromHandle = HandleOffset.Size();
	const FVector2D& Tiling2D = GetTiling();
	// We manage angle clockwise from +Y axis. Atan2 handles it anti-clockwise from +X axis.
	float Angle = 90.f - FMath::RadiansToDegrees(FMath::Atan2(HandleOffset.Y, HandleOffset.X)) + GetRotation();
	float DistanceFromCircleHandle;

	if (FMath::IsNearlyEqual(Tiling2D.X, Tiling2D.Y))
	{
		DistanceFromCircleHandle = FMath::Abs(DistanceFromHandle - GetCircleHandleBaseRadius()) / UIScale;
	}
	else
	{
		DistanceFromCircleHandle = FMath::Abs(DistanceFromHandle - GetCircleHandleRadiusAtAngle(Angle)) / UIScale;
	}

	if (DistanceFromCircleHandle > TextureUVVisualizerCircleHandleRadius)
	{
		return EHandleAxis::None;
	}

	Angle += 22.5f; // Rotate slightly so it's easier to follow the below code.
	Angle = UE::Math::TRotator<float>::ClampAxis(Angle);

	constexpr float AnglePerQuadrant = 360.f / 8.f; // 45
	constexpr float TopEnd = AnglePerQuadrant; // 45
	constexpr float TopRightEnd = TopEnd + AnglePerQuadrant; // 90
	constexpr float RightEnd = TopRightEnd + AnglePerQuadrant; // 135
	constexpr float BottomRightEnd = RightEnd + AnglePerQuadrant; // 180
	constexpr float BottomEnd = BottomRightEnd + AnglePerQuadrant; // 225
	constexpr float BottomLeftEnd = BottomEnd + AnglePerQuadrant; // 270
	constexpr float LeftEnd = BottomLeftEnd + AnglePerQuadrant; // 313
	constexpr float TopLeftEnd = LeftEnd + AnglePerQuadrant; // 360

	if (Angle < TopEnd)
	{
		return EHandleAxis::Y;
	}

	if (Angle < TopRightEnd)
	{
		return EHandleAxis::XY;
	}

	if (Angle < RightEnd)
	{
		return EHandleAxis::X;
	}

	if (Angle < BottomRightEnd)
	{
		return EHandleAxis::XY;
	}

	if (Angle < BottomEnd)
	{
		return EHandleAxis::Y;
	}

	if (Angle < BottomLeftEnd)
	{
		return EHandleAxis::XY;
	}

	if (Angle < LeftEnd)
	{
		return EHandleAxis::X;
	}

	// Top left
	return EHandleAxis::XY;
}

bool SDMTextureUVVisualizer::TryClickCenterHandle(const FVector2f& InMousePosition, bool bInResetToDefault)
{
	if (!IsValid(GetTextureUVComponent()))
	{
		return false;
	}

	// Work out if the mouse is over a part of the center handle
	EHandleAxis Axis = GetCenterHandleAxis(InMousePosition);

	if (Axis == EHandleAxis::None)
	{
		if (!bInResetToDefault)
		{
			return false;
		}

		if (!bPivotEditMode)
		{
			FScopedTransaction Transaction(LOCTEXT("ResetOffset", "Reset Offset to Default."));
			SetOffset(FVector2D::ZeroVector);
		}
		else
		{
			FScopedTransaction Transaction(LOCTEXT("ResetPivot", "Reset Pivot to Default."));
			SetPivot(FVector2D::ZeroVector);
		}

		return true;
	}

	// Offset handle
	if (!bPivotEditMode)
	{
		if (!bInResetToDefault)
		{
			SetScrubbingMode(EScrubbingMode::Offset, Axis);
		}
		else
		{
			FScopedTransaction Transaction(LOCTEXT("ResetOffset", "Reset Offset to Default."));
			ModifyTextureUVComponent();
			FVector2D NewOffset = GetOffset();

			if (Axis == EHandleAxis::X)
			{
				NewOffset.X = 0;
			}
			else if (Axis == EHandleAxis::Y)
			{
				NewOffset.Y = 0;
			}
			else
			{
				NewOffset = FVector2D::ZeroVector;
			}

			SetOffset(NewOffset);
		}
	}
	// Pivot handle
	else
	{
		if (!bInResetToDefault)
		{
			SetScrubbingMode(EScrubbingMode::Pivot, Axis);
		}
		else
		{
			FScopedTransaction Transaction(LOCTEXT("ResetPivot", "Reset Pivot to Default."));
			ModifyTextureUVComponent();
			FVector2D NewPivot = GetOffset();

			if (Axis == EHandleAxis::X)
			{
				NewPivot.X = 0.5;
			}
			else if (Axis == EHandleAxis::Y)
			{
				NewPivot.Y = 0.5;
			}
			else
			{
				NewPivot = FVector2D(0.5, 0.5);
			}

			SetPivot(NewPivot);
		}
	}

	return true;
}

bool SDMTextureUVVisualizer::TryClickCircleHandle(const FVector2f& InMousePosition, bool bInResetToDefault)
{
	if (!IsValid(GetTextureUVComponent()))
	{
		return false;
	}

	const float Rotation = GetRotation();
	const FVector2f AbsolutePivotLocation = GetAbsolutePivotLocation();

	// When the uv is rotated, do the opposite action
	const bool bRegularInvertTiling = Rotation <= 90.f || Rotation > 270.f;

	// Work out if the mouse is over part of the circle handle.
	EHandleAxis Axis = GetCircleHandleAxis(InMousePosition);

	switch (Axis)
	{
		default:
			if (!bInResetToDefault)
			{
				return false;
			}

			{
				FScopedTransaction Transaction(LOCTEXT("ResetRotationAndTiling", "Reset Rotation and Tiling to Default."));
				ModifyTextureUVComponent();
				SetRotation(0);
				SetTiling(FVector2D::UnitVector);
			}

			return true;

		// XY on the circle handle represents rotation
		case EHandleAxis::XY:
		{
			if (!bInResetToDefault)
			{
				SetScrubbingMode(EScrubbingMode::Rotation, Axis);
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("ResetRotation", "Reset Rotation to Default."));
				ModifyTextureUVComponent();
				SetRotation(0.f);
			}

			break;
		}

		case EHandleAxis::X:
		{
			if (!bInResetToDefault)
			{
				SetScrubbingMode(EScrubbingMode::Tiling, Axis);
				bInvertTiling = (InMousePosition.X <= AbsolutePivotLocation.X) == bRegularInvertTiling;
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("ResetTilingeX", "Reset Tiling X to Default."));
				ModifyTextureUVComponent();
				FVector2D NewTiling = GetTiling();
				NewTiling.X = 1.0;
				SetTiling(NewTiling);
			}

			break;
		}

		case EHandleAxis::Y:
		{
			if (!bInResetToDefault)
			{
				SetScrubbingMode(EScrubbingMode::Tiling, Axis);
				bInvertTiling = (InMousePosition.Y <= AbsolutePivotLocation.Y) == bRegularInvertTiling;
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("ResetTilingY", "Reset Tiling Y to Default."));
				ModifyTextureUVComponent();
				FVector2D NewTiling = GetTiling();
				NewTiling.Y = 1.0;
				SetTiling(NewTiling);
			}

			break;
		}
	}

	return true;
}

void SDMTextureUVVisualizer::UpdatePopoutUVs()
{
	FBox2d UVRegion = FBox2d(FVector2D(-1, -1), FVector2D(2, 2));

	if (!FMath::IsNearlyZero(CurrentAbsoluteSize.X) && !FMath::IsNearlyZero(CurrentAbsoluteSize.Y))
	{
		if (CurrentAbsoluteSize.Y <= CurrentAbsoluteSize.X)
		{
			const float Multiplier = CurrentAbsoluteSize.X / CurrentAbsoluteSize.Y;
			UVRegion.Min.X = 0.5f - (Multiplier * 1.5f);
			UVRegion.Max.X = 0.5f + (Multiplier * 1.5f);
		}
		else
		{
			const float Multiplier = CurrentAbsoluteSize.Y / CurrentAbsoluteSize.X;
			UVRegion.Min.Y = 0.5f - (Multiplier * 1.5f);
			UVRegion.Max.Y = 0.5f + (Multiplier * 1.5f);
		}
	}

	StagePreview->GetBrush().SetUVRegion(UVRegion);
}

void SDMTextureUVVisualizer::SetScrubbingMode(EScrubbingMode InMode, EHandleAxis InAxis)
{
	UDMMaterialComponent* TextureUVComponent = GetTextureUVComponent();

	if (InMode != EScrubbingMode::None && (InAxis == EHandleAxis::None || !IsValid(TextureUVComponent) || !HasValidGeometry()))
	{
		InMode = EScrubbingMode::None;
	}

	ScrubbingMode = InMode;
	HandleAxis = ScrubbingMode != EScrubbingMode::None ? InAxis : EHandleAxis::None;

	if (InMode == EScrubbingMode::None)
	{
		ScrubbingTransaction.Reset();
		return;
	}

	ScrubbingStartAbsoluteCenter = CurrentAbsoluteCenter;
	ScrubbingStartAbsoluteMouse = FSlateApplication::Get().GetCursorPos();
	FProperty* TextureProperty = nullptr;

	switch (ScrubbingMode)
	{
		case EScrubbingMode::Offset:
			ValueStart = GetOffset();
			TextureProperty = UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Offset);
			UE_LOGF(LogDynamicMaterialEditor, Verbose, "Started Offset mode");
			break;

		case EScrubbingMode::Rotation:
		{
			ValueStart.X = GetRotation();

			/** Store the original mouse angle */
			const FVector2f MouseOffset = ScrubbingStartAbsoluteMouse - GetAbsolutePivotLocation();
			ValueStart.Y = FMath::RadiansToDegrees(FMath::Atan2(MouseOffset.Y, MouseOffset.X));

			TextureProperty = UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Rotation);

			UE_LOGF(LogDynamicMaterialEditor, Verbose, "Started Rotation mode");
			break;
		}

		case EScrubbingMode::Tiling:
			ValueStart = GetTiling();
			TextureProperty = UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Tiling);
			UE_LOGF(LogDynamicMaterialEditor, Verbose, "Started Tiling mode");
			break;

		case EScrubbingMode::Pivot:
			ValueStart = GetPivot();
			TextureProperty = UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Pivot);
			UE_LOGF(LogDynamicMaterialEditor, Verbose, "Started Pivot mode");
			break;

		default:
			return;
	}

	ScrubbingTransaction = MakeShared<FScopedTransaction>(LOCTEXT("VisualizerUVScrubbingTransaction", "UV Visualizer Scrub"));
	ModifyTextureUVComponent();

	TextureUVComponent->PreEditChange(TextureProperty);
}

FVector2f SDMTextureUVVisualizer::ToPopoutLocation(const FVector2f& InSize, FVector2f&& InLocation) const
{
	const FBox2d BrushUV = StagePreview->GetBrush().GetUVRegion();

	InLocation.X = FMath::GetMappedRangeValueUnclamped<float, float>(
		FVector2f(BrushUV.Min.X, BrushUV.Max.X),
		{0, 1},
		InLocation.X / InSize.X
	) * InSize.X;

	InLocation.Y = FMath::GetMappedRangeValueUnclamped<float, float>(
		FVector2f(BrushUV.Min.Y, BrushUV.Max.Y),
		{0, 1},
		InLocation.Y / InSize.Y
	) * InSize.Y;

	return InLocation;
}

FVector2f SDMTextureUVVisualizer::FromPopoutLocation(const FVector2f& InSize, FVector2f&& InLocation) const
{
	const FBox2d BrushUV = StagePreview->GetBrush().GetUVRegion();

	InLocation.X = FMath::GetMappedRangeValueUnclamped<float, float>(
		{0, 1},
		FVector2f(BrushUV.Min.X, BrushUV.Max.X),
		InLocation.X / InSize.X
	) * InSize.X;

	InLocation.Y = FMath::GetMappedRangeValueUnclamped<float, float>(
		{0, 1},
		FVector2f(BrushUV.Min.Y, BrushUV.Max.Y),
		InLocation.Y / InSize.Y
	) * InSize.Y;

	return InLocation;
}

void SDMTextureUVVisualizer::ModifyTextureUVComponent()
{
	if (UDMMaterialComponent* TextureUVComponent = GetTextureUVComponent())
	{
		TextureUVComponent->Modify();
	}
}

void SDMTextureUVVisualizer::UpdateScrub()
{
	switch (ScrubbingMode)
	{
		case EScrubbingMode::Offset:
			UpdateScrub_Offset();
			break;

		case EScrubbingMode::Rotation:
			UpdateScrub_Rotation();
			break;

		case EScrubbingMode::Tiling:
			UpdateScrub_Tiling();
			break;

		case EScrubbingMode::Pivot:
			UpdateScrub_Pivot();
			break;

		default:
			return;
	}

	if (GEditor)
	{
		GEditor->RedrawLevelEditingViewports();
	}
}

void SDMTextureUVVisualizer::UpdateScrub_Offset()
{
	UDMMaterialComponent* TextureUVComponent = GetTextureUVComponent();

	if (!IsValid(TextureUVComponent))
	{
		return;
	}

	const FVector2f MouseOffset = FSlateApplication::Get().GetCursorPos() - ScrubbingStartAbsoluteMouse;
	const FBox2d BrushUV = StagePreview->GetBrush().GetUVRegion();

	FVector2D OffsetChange = static_cast<FVector2D>(MouseOffset / CurrentAbsoluteSize);
	OffsetChange.X *= (BrushUV.Max.X - BrushUV.Min.X);
	OffsetChange.Y *= (BrushUV.Max.Y - BrushUV.Min.Y);

	const float Rotation = GetRotation();

	if (!FMath::IsNearlyZero(Rotation))
	{
		OffsetChange = OffsetChange.GetRotated(-Rotation);
	}

	OffsetChange *= GetTiling();

	if (HandleAxis == EHandleAxis::X)
	{
		OffsetChange.Y = 0;
	}
	else
	{
		OffsetChange.Y *= -1.0;
	}

	if (HandleAxis == EHandleAxis::Y)
	{
		OffsetChange.X = 0;
	}

	SetOffset(ValueStart + OffsetChange);

	FPropertyChangedEvent ChangedEvent(
		UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Offset),
		EPropertyChangeType::Interactive,
		{TextureUVComponent}
	);

	TextureUVComponent->PostEditChangeProperty(ChangedEvent);
}

void SDMTextureUVVisualizer::UpdateScrub_Rotation()
{
	UDMMaterialComponent* TextureUVComponent = GetTextureUVComponent();

	if (!IsValid(TextureUVComponent))
	{
		return;
	}

	const FVector2f AbsoluteMousePosition = FSlateApplication::Get().GetCursorPos();
	const FVector2f CurrentMouseOffset = AbsoluteMousePosition - GetAbsolutePivotLocation();
	const float Angle = FMath::RadiansToDegrees(FMath::Atan2(CurrentMouseOffset.Y, CurrentMouseOffset.X));
	const float NewValue = UE::Math::TRotator<float>::ClampAxis(ValueStart.X + Angle - ValueStart.Y);

	SetRotation(NewValue);

	FPropertyChangedEvent ChangedEvent(
		UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Rotation),
		EPropertyChangeType::Interactive,
		{TextureUVComponent}
	);

	TextureUVComponent->PostEditChangeProperty(ChangedEvent);
}

void SDMTextureUVVisualizer::UpdateScrub_Tiling()
{
	UDMMaterialComponent* TextureUVComponent = GetTextureUVComponent();

	if (!IsValid(TextureUVComponent))
	{
		return;
	}

	const FVector2f MouseOffset = FSlateApplication::Get().GetCursorPos() - ScrubbingStartAbsoluteMouse;
	const FBox2d BrushUV = StagePreview->GetBrush().GetUVRegion();

	FVector2D TilingChange = static_cast<FVector2D>(MouseOffset / CurrentAbsoluteSize);
	TilingChange.X *= (BrushUV.Max.X - BrushUV.Min.X);
	TilingChange.Y *= (BrushUV.Max.Y - BrushUV.Min.Y);
	TilingChange /= GetTiling();

	if (!bInvertTiling)
	{
		TilingChange *= -1.f;
	}

	const float Rotation = GetRotation();

	if (!FMath::IsNearlyZero(Rotation))
	{
		TilingChange = TilingChange.GetRotated(-Rotation);
	}

	if (HandleAxis == EHandleAxis::X)
	{
		TilingChange.Y = 0;
	}
	else
	{
		TilingChange.Y *= -1.0;
	}

	if (HandleAxis == EHandleAxis::Y)
	{
		TilingChange.X = 0;
	}

	FVector2D NewTiling = ValueStart;

	if (HandleAxis == EHandleAxis::X)
	{
		if (TilingChange.X > 0)
		{
			NewTiling.X = FMath::Max(0.001, NewTiling.X * (1.f + TilingChange.X));
		}
		else
		{
			NewTiling.X = FMath::Max(0.001, NewTiling.X / (1.f - TilingChange.X));
		}
	}
	else
	{
		if (TilingChange.Y > 0)
		{
			NewTiling.Y = FMath::Max(0.001, NewTiling.Y / (1.f + TilingChange.Y));
		}
		else
		{
			NewTiling.Y = FMath::Max(0.001, NewTiling.Y * (1.f - TilingChange.Y));
		}
	}

	SetTiling(NewTiling);

	FPropertyChangedEvent ChangedEvent(
		UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Tiling),
		EPropertyChangeType::Interactive,
		{TextureUVComponent}
	);

	TextureUVComponent->PostEditChangeProperty(ChangedEvent);
}

void SDMTextureUVVisualizer::UpdateScrub_Pivot()
{
	UDMMaterialComponent* TextureUVComponent = GetTextureUVComponent();

	if (!IsValid(TextureUVComponent))
	{
		return;
	}

	const FVector2f MouseOffset = FSlateApplication::Get().GetCursorPos() - ScrubbingStartAbsoluteMouse;
	const FBox2d BrushUV = StagePreview->GetBrush().GetUVRegion();

	FVector2D PivotChange = static_cast<FVector2D>(MouseOffset / CurrentAbsoluteSize);
	PivotChange.X *= (BrushUV.Max.X - BrushUV.Min.X);
	PivotChange.Y *= (BrushUV.Max.Y - BrushUV.Min.Y);

	if (HandleAxis == EHandleAxis::X)
	{
		PivotChange.Y = 0;
	}
	else if (HandleAxis == EHandleAxis::Y)
	{
		PivotChange.X = 0;
	}

	SetPivot(ValueStart + PivotChange);

	FPropertyChangedEvent ChangedEvent(
		UDMTextureUV::StaticClass()->FindPropertyByName(UDMTextureUV::NAME_Pivot),
		EPropertyChangeType::Interactive,
		{TextureUVComponent}
	);

	TextureUVComponent->PostEditChangeProperty(ChangedEvent);
}

#undef LOCTEXT_NAMESPACE
