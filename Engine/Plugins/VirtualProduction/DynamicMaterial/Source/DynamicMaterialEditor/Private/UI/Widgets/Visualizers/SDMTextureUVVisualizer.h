// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FScopedTransaction;
class SDMMaterialComponentPreview;
class SDMMaterialEditor;
class SWidget;
class UDMMaterialComponent;
class UDMMaterialStage;
class UDMTextureUV;
class UDMTextureUVDynamic;
class UMaterial;
enum class EDMUpdateType : uint8;

/**
 * Material Designer Texture UV Visualizer 
 *
 * Ability to edit Texture UV settings in a visual manner.
 */
class SDMTextureUVVisualizer : public SCompoundWidget
{
public:
	enum class EScrubbingMode : uint8
	{
		None,
		Offset,
		Rotation,
		Tiling,
		Pivot
	};

	enum class EHandleAxis : uint8
	{
		None,
		X,
		Y,
		XY
	};

	SLATE_BEGIN_ARGS(SDMTextureUVVisualizer)
		: _TextureUV(nullptr)
		, _TextureUVDynamic(nullptr)
		, _IsPopout(false)
		{}
		SLATE_ARGUMENT(UDMTextureUV*, TextureUV)
		SLATE_ARGUMENT(UDMTextureUVDynamic*, TextureUVDynamic)
		SLATE_ARGUMENT(bool, IsPopout)
	SLATE_END_ARGS()

	SDMTextureUVVisualizer();

	/** The TextureUV should be a sub-property of the stage */
	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage);

	TSharedPtr<SDMMaterialEditor> GetEditorWidget() const { return EditorWidgetWeak.Pin(); }

	EScrubbingMode GetScrubbingMode() const;

	bool IsInPivotEditMode() const;

	void SetInPivotEditMode(bool bInEditingPivot);

	void TogglePivotEditMode();

	UDMMaterialStage* GetStage() const;

	UDMMaterialComponent* GetTextureUVComponent() const;

	UDMTextureUV* GetTextureUV() const;

	UDMTextureUVDynamic* GetTextureUVDynamic() const;

	const FVector2D& GetOffset() const;
	bool SetOffset(const FVector2D& InOffset);

	float GetRotation() const;
	bool SetRotation(float InRotation);

	const FVector2D& GetTiling() const;
	bool SetTiling(const FVector2D& InTiling);

	const FVector2D& GetPivot() const;
	bool SetPivot(const FVector2D& InPivot);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InCursorEvent) const override;
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, 
		FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakObjectPtr<UDMMaterialStage> StageWeak;
	TWeakObjectPtr<UDMMaterialComponent> TextureUVComponentWeak;
	TSharedPtr<SDMMaterialComponentPreview> StagePreview;
	bool bIsPopout;
	bool bPivotEditMode;
	FVector2f CurrentAbsoluteSize;
	FVector2f CurrentAbsoluteCenter;
	EScrubbingMode ScrubbingMode;
	FVector2f ScrubbingStartAbsoluteCenter;
	FVector2f ScrubbingStartAbsoluteMouse;
	EHandleAxis HandleAxis;
	FVector2D ValueStart;
	bool bInvertTiling;
	TSharedPtr<FScopedTransaction> ScrubbingTransaction;

	bool HasValidGeometry() const;

	float GetCircleHandleBaseRadius() const;

	FVector2f ApplyTextureUVTransform(const FVector2f& InUV) const;

	FVector2f GetOffsetLocation(const FVector2f& InSize) const;

	FVector2f GetPivotLocation(const FVector2f& InSize) const;

	FVector2f GetAbsoluteOffsetLocation() const;

	FVector2f GetAbsolutePivotLocation() const;

	/** Degrees clockwise from +Y axis */
	float GetCircleHandleRadiusAtAngle(float InAngle) const;

	EHandleAxis GetCenterHandleAxis(const FVector2f& InAbsolutePosition) const;

	EHandleAxis GetCircleHandleAxis(const FVector2f& InAbsolutePosition) const;

	bool TryClickCenterHandle(const FVector2f& InMousePosition, bool bInResetToDefault);

	bool TryClickCircleHandle(const FVector2f& InMousePosition, bool bInResetToDefault);

	void UpdatePopoutUVs();

	void SetScrubbingMode(EScrubbingMode InMode, EHandleAxis InAxis);

	FVector2f ToPopoutLocation(const FVector2f& InSize, FVector2f&& InLocation) const;

	FVector2f FromPopoutLocation(const FVector2f& InSize, FVector2f&& InLocation) const;

	void ModifyTextureUVComponent();

	void UpdateScrub();
	void UpdateScrub_Offset();
	void UpdateScrub_Rotation();
	void UpdateScrub_Tiling();
	void UpdateScrub_Pivot();
};
