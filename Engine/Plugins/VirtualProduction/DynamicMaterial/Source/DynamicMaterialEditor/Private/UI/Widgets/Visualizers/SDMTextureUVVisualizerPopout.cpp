// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Visualizers/SDMTextureUVVisualizerPopout.h"

#include "Components/DMMaterialStage.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"
#include "DetailLayoutBuilder.h"
#include "DMWorldSubsystem.h"
#include "Engine/World.h"
#include "ICustomDetailsView.h"
#include "IDynamicMaterialEditorModule.h"
#include "Items/ICustomDetailsViewCustomItem.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModelBase.h"
#include "Styling/SlateIconFinder.h"
#include "UI/PropertyGenerators/DMTextureUVDynamicPropertyRowGenerator.h"
#include "UI/PropertyGenerators/DMTextureUVPropertyRowGenerator.h"
#include "UI/Utils/DMWidgetLibrary.h"
#include "UI/Widgets/SDMMaterialEditor.h"
#include "UI/Widgets/Visualizers/SDMTextureUVVisualizer.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMTextureUVVisualizerPopout"

const FName SDMTextureUVVisualizerPopout::TabId = TEXT("SDMTextureUVVisualizerPopout");

namespace UE::DynamicMaterialEditor::Private
{
	TSharedPtr<SDockTab> GetVisualizerTab(FName InTabId)
	{
		if (!FGlobalTabmanager::Get()->HasTabSpawner(InTabId))
		{
			FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
				InTabId,
				FOnSpawnTab::CreateLambda(
					[InTabId](const FSpawnTabArgs& InArgs)
					{
						TSharedRef<SDockTab> DockTab = SNew(SDockTab)
							.Label(FText::FromName(InTabId))
							.LabelSuffix(LOCTEXT("TabSuffix", "[UV Vis]"));

						DockTab->SetTabIcon(FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()).GetIcon());

						return DockTab;
					}
				)
			);
		}

		return FGlobalTabmanager::Get()->TryInvokeTab(InTabId);
	}
}

void SDMTextureUVVisualizerPopout::CreatePopout(const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage, UDMTextureUV* InTextureUV)
{
	if (!IsValid(InMaterialStage) || !IsValid(InTextureUV))
	{
		return;
	}

	TSharedPtr<SDockTab> Tab = UE::DynamicMaterialEditor::Private::GetVisualizerTab(TabId);

	if (!Tab.IsValid())
	{
		return;
	}

	Tab->ActivateInParent(ETabActivationCause::SetDirectly);
	Tab->SetLabel(FText::FromString(InTextureUV->GetPathName()));
	Tab->SetContent(SNew(SDMTextureUVVisualizerPopout, InEditorWidget, InMaterialStage).TextureUV(InTextureUV));
}

void SDMTextureUVVisualizerPopout::CreatePopout(const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage, UDMTextureUVDynamic* InTextureUVDynamic)
{
	if (!IsValid(InMaterialStage) || !IsValid(InTextureUVDynamic))
	{
		return;
	}

	TSharedPtr<SDockTab> Tab = UE::DynamicMaterialEditor::Private::GetVisualizerTab(TabId);

	if (!Tab.IsValid())
	{
		return;
	}

	Tab->ActivateInParent(ETabActivationCause::SetDirectly);
	Tab->SetLabel(FText::FromString(InTextureUVDynamic->GetPathName()));
	Tab->SetContent(SNew(SDMTextureUVVisualizerPopout, InEditorWidget, InMaterialStage).TextureUVDynamic(InTextureUVDynamic));
}

void SDMTextureUVVisualizerPopout::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage)
{
	check(InMaterialStage);
	check(InArgs._TextureUV || InArgs._TextureUVDynamic);

	SetCanTick(false);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.Padding(3.f, 0.f, 0.f, 0.f)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(Visualizer, SDMTextureUVVisualizer, InEditorWidget, InMaterialStage)
				.TextureUV(InArgs._TextureUV)
				.TextureUVDynamic(InArgs._TextureUVDynamic)
				.IsPopout(true)
			]
			+ SOverlay::Slot()
			[
				SNew(SBox)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Top)
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(SColorBlock)
					.Color(FLinearColor(0, 0, 0, 0.5))
					.Size(this, &SDMTextureUVVisualizerPopout::GetHorizontalBarSize)
				]
			]
			+ SOverlay::Slot()
			[
				SNew(SBox)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Bottom)
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(SColorBlock)
					.Color(FLinearColor(0, 0, 0, 0.5))
					.Size(this, &SDMTextureUVVisualizerPopout::GetHorizontalBarSize)
				]
			]
			+ SOverlay::Slot()
			[
				SNew(SBox)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(SColorBlock)
					.Color(FLinearColor(0, 0, 0, 0.5))
					.Size(this, &SDMTextureUVVisualizerPopout::GetSideBlockSize)
				]
			]
			+ SOverlay::Slot()
			[
				SNew(SBox)
				.HAlign(EHorizontalAlignment::HAlign_Right)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Visibility(EVisibility::HitTestInvisible)
				[
					SNew(SColorBlock)
					.Color(FLinearColor(0, 0, 0, 0.5))
					.Size(this, &SDMTextureUVVisualizerPopout::GetSideBlockSize)
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Top)
		[
			SNew(SBox)
			.WidthOverride(300.f)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.VAlign(EVerticalAlignment::VAlign_Top)
			[
				CreatePropertyWidget(
					InEditorWidget,
					InArgs._TextureUV 
						? static_cast<UDMMaterialComponent*>(InArgs._TextureUV)
						: static_cast<UDMMaterialComponent*>(InArgs._TextureUVDynamic)
				)
			]
		]
	];
}

void SDMTextureUVVisualizerPopout::NotifyPreChange(FProperty* InPropertyAboutToChange)
{	
}

void SDMTextureUVVisualizerPopout::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
}

FText SDMTextureUVVisualizerPopout::GetModeButtonText() const
{
	if (Visualizer.IsValid() && Visualizer->IsInPivotEditMode())
	{
		return LOCTEXT("VisualizerPivot", "Pivot");
	}

	return LOCTEXT("VisualizerOffset", "Offset");
}

FVector2D SDMTextureUVVisualizerPopout::GetHorizontalBarSize() const
{
	if (Visualizer.IsValid())
	{
		const FVector2f LocalSize = Visualizer->GetTickSpaceGeometry().GetLocalSize();

		if (!FMath::IsNearlyZero(LocalSize.X) && !FMath::IsNearlyZero(LocalSize.Y))
		{
			if (LocalSize.Y <= LocalSize.X)
			{
				return FVector2D(1, LocalSize.Y / 3.f);
			}

			return FVector2D(1, LocalSize.X / 3.f + (LocalSize.Y - LocalSize.X) * 0.5f);
		}
	}

	return FVector2D::UnitVector;
}

FVector2D SDMTextureUVVisualizerPopout::GetSideBlockSize() const
{
	if (Visualizer.IsValid())
	{
		const FVector2f LocalSize = Visualizer->GetTickSpaceGeometry().GetLocalSize();

		if (!FMath::IsNearlyZero(LocalSize.X) && !FMath::IsNearlyZero(LocalSize.Y))
		{
			if (LocalSize.X <= LocalSize.Y)
			{
				return FVector2D(LocalSize.X / 3.f, LocalSize.X / 3.f);
			}

			return FVector2D(LocalSize.Y / 3.f + (LocalSize.X - LocalSize.Y) * 0.5f, LocalSize.Y / 3.f);
		}
	}

	return FVector2D::UnitVector;
}

TSharedRef<SWidget> SDMTextureUVVisualizerPopout::CreatePropertyWidget(const TSharedRef<SDMMaterialEditor>& InEditorWidget, 
	UDMMaterialComponent* InComponent)
{
	FDMWidgetLibrary::Get().ClearPropertyHandles(this);

	if (!InComponent)
	{
		return SNullWidget::NullWidget;
	}

	const bool bIsDynamic = InComponent && InComponent->IsA<UDMTextureUVDynamic>();

	FCustomDetailsViewArgs Args;
	Args.KeyframeHandler = nullptr;
	Args.bAllowGlobalExtensions = true;
	Args.bAllowResetToDefault = true;
	Args.bShowCategories = false;

	UWorld* World = InComponent->GetWorld();

	if (!World)
	{
		if (UDynamicMaterialModelBase* OriginalMaterialModelBase = InEditorWidget->GetOriginalMaterialModelBase())
		{
			World = OriginalMaterialModelBase->GetWorld();
		}
	}

	if (World)
	{
		if (UDMWorldSubsystem* WorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>())
		{
			Args.KeyframeHandler = WorldSubsystem->GetKeyframeHandler();
		}
	}

	TSharedRef<ICustomDetailsView> DetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(Args);
	const FCustomDetailsViewItemId RootId = DetailsView->GetRootItem()->GetItemId();

	TArray<FDMPropertyHandle> TextureUVPropertyRows;
	TSet<UObject*> ProcessedObjects;

	FDMComponentPropertyRowGeneratorParams Params(TextureUVPropertyRows, ProcessedObjects);
	Params.Owner = this;
	Params.NotifyHook = this;
	Params.Object = InComponent;
	Params.PreviewMaterialModelBase = InEditorWidget->GetPreviewMaterialModelBase();
	Params.OriginalMaterialModelBase = InEditorWidget->GetOriginalMaterialModelBase();

	if (bIsDynamic)
	{
		FDMTextureUVDynamicPropertyRowGenerator::AddPopoutComponentProperties(Params);
	}
	else
	{
		FDMTextureUVPropertyRowGenerator::AddPopoutComponentProperties(Params);
	}

	FDMPropertyHandle EditModeButtonRow;
	EditModeButtonRow.ValueName = TEXT("EditMode");
	EditModeButtonRow.NameOverride = LOCTEXT("EditMode", "Edit Mode");
	EditModeButtonRow.ValueWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillContentWidth(1.f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(FVector2D(10.f, 3.f))
			.IsChecked(this, &SDMTextureUVVisualizerPopout::GetModeCheckBoxState, /* Is Pivot */ false)
			.OnCheckStateChanged(this, &SDMTextureUVVisualizerPopout::OnModeCheckBoxStateChanged, /* Is Pivot */ false)
			.ToolTipText(LOCTEXT("VisualizerOffsetToolTip", "Allows changing of the UV offset."))
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("VisualizerOffset", "Offset"))
			]
		]
		+ SHorizontalBox::Slot()
		.FillContentWidth(1.f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(FVector2D(10.f, 3.f))
			.IsChecked(this, &SDMTextureUVVisualizerPopout::GetModeCheckBoxState, /* Is Pivot */ true)
			.OnCheckStateChanged(this, &SDMTextureUVVisualizerPopout::OnModeCheckBoxStateChanged, /* Is Pivot */ true)
			.ToolTipText(LOCTEXT("VisualizerPivotToolTip", "Allows changing of the UV pivot, rotation and tiling."))
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("VisualizerPivot", "Pivot"))
			]
		];

	TextureUVPropertyRows.Add(EditModeButtonRow);

	for (const FDMPropertyHandle& EditRow : TextureUVPropertyRows)
	{
		const bool bHasValidCustomWidget = EditRow.ValueWidget.IsValid() && !EditRow.ValueName.IsNone() && EditRow.NameOverride.IsSet();

		if (!EditRow.PreviewHandle.DetailTreeNode && !bHasValidCustomWidget)
		{
			continue;
		}

		ECustomDetailsTreeInsertPosition Position = ECustomDetailsTreeInsertPosition::Child;

		if (EditRow.PreviewHandle.PropertyHandle.IsValid())
		{
			if (EditRow.PreviewHandle.PropertyHandle->HasMetaData("HighPriority"))
			{
				Position = ECustomDetailsTreeInsertPosition::FirstChild;
			}
			else if (EditRow.PreviewHandle.PropertyHandle->HasMetaData("LowPriority"))
			{
				Position = ECustomDetailsTreeInsertPosition::LastChild;
			}
		}

		if (bHasValidCustomWidget)
		{
			TSharedPtr<ICustomDetailsViewCustomItem> Item = DetailsView->CreateCustomItem(
				DetailsView->GetRootItem(),
				EditRow.ValueName, 
				EditRow.NameOverride.GetValue(), 
				EditRow.NameToolTipOverride.Get(FText::GetEmpty())
			);

			if (!Item.IsValid())
			{
				continue;
			}

			if (!EditRow.bEnabled)
			{
				Item->AsItem()->SetEnabledOverride(false);
			}

			Item->SetValueWidget(EditRow.ValueWidget.ToSharedRef());
			DetailsView->ExtendTree(RootId, Position, Item->AsItem());
			continue;
		}

		if (!EditRow.PreviewHandle.DetailTreeNode)
		{
			continue;
		}

		TSharedRef<ICustomDetailsViewItem> Item = DetailsView->CreateDetailTreeItem(DetailsView->GetRootItem(), EditRow.PreviewHandle.DetailTreeNode.ToSharedRef());

		if (EditRow.NameOverride.IsSet())
		{
			Item->SetOverrideWidget(
				ECustomDetailsViewWidgetType::Name,
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(EditRow.NameOverride.GetValue())
					.ToolTipText(EditRow.NameToolTipOverride.Get(FText::GetEmpty()))
			);
		}

		if (!EditRow.bEnabled)
		{
			Item->SetEnabledOverride(false);
		}

		if (EditRow.PreviewHandle.PropertyHandle->HasMetaData("NotKeyframeable"))
		{
			Item->SetKeyframeEnabled(false);
		}

		if (EditRow.ResetToDefaultOverride.IsSet())
		{
			Item->SetResetToDefaultOverride(EditRow.ResetToDefaultOverride.GetValue());
		}

		DetailsView->ExtendTree(RootId, Position, Item);
	}

	DetailsView->RebuildTree(ECustomDetailsViewBuildType::InstantBuild);

	return DetailsView;
}

ECheckBoxState SDMTextureUVVisualizerPopout::GetModeCheckBoxState(bool bInIsPivot) const
{
	if (!Visualizer.IsValid())
	{
		return ECheckBoxState::Undetermined;
	}

	return Visualizer->IsInPivotEditMode() == bInIsPivot
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SDMTextureUVVisualizerPopout::OnModeCheckBoxStateChanged(ECheckBoxState InState, bool bInIsPivot)
{
	if (InState != ECheckBoxState::Checked || !Visualizer.IsValid())
	{
		return;
	}

	Visualizer->SetInPivotEditMode(bInIsPivot);
}

#undef LOCTEXT_NAMESPACE
