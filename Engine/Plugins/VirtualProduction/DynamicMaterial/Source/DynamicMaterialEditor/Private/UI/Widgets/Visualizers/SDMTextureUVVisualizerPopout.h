// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointerFwd.h"

class ICustomDetailsView;
class SDMMaterialEditor;
class SDMTextureUVVisualizer;
class SDockTab;
class UDMMaterialComponent;
class UDMMaterialStage;
class UDMTextureUV;
class UDMTextureUVDynamic;
enum class ECheckBoxState : uint8;

/**
 * Material Designer Texture UV Visualizer Popout
 *
 * Houses a Texture UV editor and a few buttons to control it.
 *
 * The popout specifically expands the visible area of the preview to 3x the normal size
 * on the smallest axis. The other axis is expanded to match the aspect ratio.
 */
class SDMTextureUVVisualizerPopout : public SCompoundWidget, public FNotifyHook
{
public:
	static const FName TabId;

	static void CreatePopout(const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage, UDMTextureUV* InTextureUV);

	static void CreatePopout(const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage, UDMTextureUVDynamic* InTextureUVDynamic);

	SLATE_BEGIN_ARGS(SDMTextureUVVisualizerPopout)
		: _TextureUV(nullptr)
		, _TextureUVDynamic(nullptr)
		{}
		SLATE_ARGUMENT(UDMTextureUV*, TextureUV)
		SLATE_ARGUMENT(UDMTextureUVDynamic*, TextureUVDynamic)
	SLATE_END_ARGS()

	/** The TextureUV should be a sub-property of the stage */
	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage);

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	TSharedPtr<SDMTextureUVVisualizer> Visualizer;

	FText GetModeButtonText() const;

	FVector2D GetHorizontalBarSize() const;

	FVector2D GetSideBlockSize() const;

	TSharedRef<SWidget> CreatePropertyWidget(const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialComponent* InComponent);

	ECheckBoxState GetModeCheckBoxState(bool bInIsPivot) const;

	void OnModeCheckBoxStateChanged(ECheckBoxState InState, bool bInIsPivot);
};
