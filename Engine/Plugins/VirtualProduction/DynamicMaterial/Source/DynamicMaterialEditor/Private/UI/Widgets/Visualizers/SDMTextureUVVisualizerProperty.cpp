// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Widgets/Visualizers/SDMTextureUVVisualizerProperty.h"

#include "Components/DMMaterialStage.h"
#include "Components/DMTextureUV.h"
#include "Components/DMTextureUVDynamic.h"
#include "DetailLayoutBuilder.h"
#include "DynamicMaterialEditorSettings.h"
#include "UI/Widgets/Visualizers/SDMTextureUVVisualizer.h"
#include "UI/Widgets/Visualizers/SDMTextureUVVisualizerPopout.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMTextureUVVisualizerProperty"

void SDMTextureUVVisualizerProperty::Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage)
{
	check(InMaterialStage);
	check(InArgs._TextureUV || InArgs._TextureUVDynamic);

	SetCanTick(false);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(0.f, 3.f, 0.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.OnClicked(this, &SDMTextureUVVisualizerProperty::OnToggleVisualizerClicked)
				.Content()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("ToggleVisualizer", "Toggle"))
				]
			]
			+ SHorizontalBox::Slot()
			.FillContentWidth(1.f)
			[
				SNew(SButton)
				.OnClicked(this, &SDMTextureUVVisualizerProperty::OnOpenPopoutClicked)
				.Content()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("PopoutVisualizer", "Popout"))
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Top)
		.Padding(0.f, 3.f, 0.f, 0.f)
		[
			SNew(SBox)
			.Visibility(this, &SDMTextureUVVisualizerProperty::GetVisualizerVisibility)
			.MinAspectRatio(1)
			.MaxAspectRatio(1)
			[
				SAssignNew(Visualizer, SDMTextureUVVisualizer, InEditorWidget, InMaterialStage)
				.TextureUV(InArgs._TextureUV)
				.TextureUVDynamic(InArgs._TextureUVDynamic)
				.IsPopout(false)
			]			
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SDMTextureUVVisualizerProperty::GetVisualizerVisibility)
			+ SHorizontalBox::Slot()
			.FillContentWidth(1.f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.Padding(FVector2D(5.f, 3.f))
				.IsChecked(this, &SDMTextureUVVisualizerProperty::GetModeCheckBoxState, /* Is Pivot */ false)
				.OnCheckStateChanged(this, &SDMTextureUVVisualizerProperty::OnModeCheckBoxStateChanged, /* Is Pivot */ false)
				.ToolTipText(LOCTEXT("VisualizerOffsetToolTip", "Allows changing of the UV offset."))
				.Content()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("VisualizerOffset", "Offset"))
				]
			]
			+ SHorizontalBox::Slot()
			.FillContentWidth(1.f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.Padding(FVector2D(5.f, 3.f))
				.IsChecked(this, &SDMTextureUVVisualizerProperty::GetModeCheckBoxState, /* Is Pivot */ true)
				.OnCheckStateChanged(this, &SDMTextureUVVisualizerProperty::OnModeCheckBoxStateChanged, /* Is Pivot */ true)
				.ToolTipText(LOCTEXT("VisualizerPivotToolTip", "Allows changing of the UV pivot, rotation and tiling."))
				.Content()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("VisualizerPivot", "Pivot"))
				]
			]
		]
	];
}

FReply SDMTextureUVVisualizerProperty::OnToggleVisualizerClicked()
{
	if (UDynamicMaterialEditorSettings* Settings = GetMutableDefault<UDynamicMaterialEditorSettings>())
	{
		Settings->bUVVisualizerVisible = !Settings->bUVVisualizerVisible;
		Settings->SaveConfig();
	}

	return FReply::Handled();
}

FReply SDMTextureUVVisualizerProperty::OnToggleModeClicked()
{
	if (Visualizer.IsValid())
	{
		Visualizer->TogglePivotEditMode();
	}

	return FReply::Handled();
}

FReply SDMTextureUVVisualizerProperty::OnOpenPopoutClicked()
{
	if (!Visualizer.IsValid())
	{
		return FReply::Handled();
	}

	UDMMaterialStage* Stage = Visualizer->GetStage();
	UDMMaterialComponent* TextureUVComponent = Visualizer->GetTextureUVComponent();

	if (!IsValid(Stage) || !IsValid(TextureUVComponent))
	{
		return FReply::Handled();
	}

	TSharedPtr<SDMMaterialEditor> EditorWidget = Visualizer->GetEditorWidget();

	if (!EditorWidget.IsValid())
	{
		return FReply::Handled();
	}

	if (UDMTextureUV* TextureUV = Cast<UDMTextureUV>(TextureUVComponent))
	{
		SDMTextureUVVisualizerPopout::CreatePopout(EditorWidget.ToSharedRef(), Stage, TextureUV);
	}
	else if (UDMTextureUVDynamic* TextureUVDynamic = Cast<UDMTextureUVDynamic>(TextureUVComponent))
	{
		SDMTextureUVVisualizerPopout::CreatePopout(EditorWidget.ToSharedRef(), Stage, TextureUVDynamic);
	}

	return FReply::Handled();
}

EVisibility SDMTextureUVVisualizerProperty::GetVisualizerVisibility() const
{
	if (const UDynamicMaterialEditorSettings* Settings = GetDefault<UDynamicMaterialEditorSettings>())
	{
		return Settings->bUVVisualizerVisible
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

ECheckBoxState SDMTextureUVVisualizerProperty::GetModeCheckBoxState(bool bInIsPivot) const
{
	if (!Visualizer.IsValid())
	{
		return ECheckBoxState::Undetermined;
	}

	return Visualizer->IsInPivotEditMode() == bInIsPivot
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SDMTextureUVVisualizerProperty::OnModeCheckBoxStateChanged(ECheckBoxState InState, bool bInIsPivot)
{
	if (InState != ECheckBoxState::Checked || !Visualizer.IsValid())
	{
		return;
	}

	Visualizer->SetInPivotEditMode(bInIsPivot);
}

#undef LOCTEXT_NAMESPACE
