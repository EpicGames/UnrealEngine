// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SDMMaterialEditor;
class SDMTextureUVVisualizer;
class SDMTextureUVVisualizerPopout;
class UDMMaterialStage;
class UDMTextureUV;
class UDMTextureUVDynamic;
enum class ECheckBoxState : uint8;

/**
 * Material Designer Texture UV Visualizer Property
 *
 * Houses a Texture UV editor and a few buttons to control it.
 */
class SDMTextureUVVisualizerProperty : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMTextureUVVisualizerProperty)
		: _TextureUV(nullptr)
		, _TextureUVDynamic(nullptr)
		{}
		SLATE_ARGUMENT(UDMTextureUV*, TextureUV)
		SLATE_ARGUMENT(UDMTextureUVDynamic*, TextureUVDynamic)
	SLATE_END_ARGS()

	/** The TextureUV should be a sub-property of the stage */
	void Construct(const FArguments& InArgs, const TSharedRef<SDMMaterialEditor>& InEditorWidget, UDMMaterialStage* InMaterialStage);

protected:
	TSharedPtr<SDMTextureUVVisualizer> Visualizer;

	FReply OnToggleVisualizerClicked();

	FReply OnToggleModeClicked();

	FReply OnOpenPopoutClicked();

	EVisibility GetVisualizerVisibility() const;

	ECheckBoxState GetModeCheckBoxState(bool bInIsPivot) const;

	void OnModeCheckBoxStateChanged(ECheckBoxState InState, bool bInIsPivot);
};
