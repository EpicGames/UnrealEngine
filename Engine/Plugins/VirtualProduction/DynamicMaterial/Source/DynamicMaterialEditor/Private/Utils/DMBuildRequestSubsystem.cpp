// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMBuildRequestSubsystem.h"

#include "Components/DMMaterialComponent.h"
#include "DMEDefs.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMBuildRequestSubsystem)

UDMBuildRequestSubsystem* UDMBuildRequestSubsystem::Get()
{
	if (!GEditor)
	{
		return nullptr;
	}

	return GEditor->GetEditorSubsystem<UDMBuildRequestSubsystem>();
}

void UDMBuildRequestSubsystem::Deinitialize()
{
	Super::Deinitialize();

	BuildRequestList.Empty();
}

void UDMBuildRequestSubsystem::Tick(float InDeltaTime)
{
	ProcessBuildRequestList();
}

TStatId UDMBuildRequestSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UDMBuildRequestSubsystem, STATGROUP_Tickables);
}

void UDMBuildRequestSubsystem::AddBuildRequest(UObject* InToBuild, bool bInDirtyAssets)
{
	if (!IsValid(InToBuild))
	{
		return;
	}

	BuildRequestList.Add({InToBuild->GetPathName(), bInDirtyAssets});

	// Make sure we don't spam updates on a single tick.
	UDMMaterialComponent::PreventClean(UE_KINDA_SMALL_NUMBER);
}

void UDMBuildRequestSubsystem::RemoveBuildRequest(UObject* InToNotBuild)
{
	if (!InToNotBuild)
	{
		return;
	}

	BuildRequestList.Remove({InToNotBuild->GetPathName(), /* Dirty Assets */ false});
}

void UDMBuildRequestSubsystem::RemoveBuildRequestForOuter(UObject* InOuter)
{
	if (!InOuter)
	{
		return;
	}

	const FString ObjectPath = InOuter->GetPathName();
	const int32 ObjectPathLength = ObjectPath.Len();

	for (TSet<FDMBuildRequestEntry>::TIterator Iter(BuildRequestList); Iter; ++Iter)
	{
		if (Iter->AssetPath.Len() > ObjectPathLength && Iter->AssetPath.StartsWith(ObjectPath))
		{
			// Make sure it's a path separator after the parent path.
			switch (Iter->AssetPath[ObjectPathLength])
			{
				case '.':
				case '/':
				case ':':
					Iter.RemoveCurrent();
					break;
			}
		}
	}
}

void UDMBuildRequestSubsystem::ProcessBuildRequestList()
{
	if (UDMMaterialComponent::CanClean() == false)
	{
		return;
	}

	TArray<FDMBuildRequestEntry> BuildRequestListLocal;
	BuildRequestListLocal.Append(BuildRequestList.Array());
	BuildRequestList.Empty();

	for (const FDMBuildRequestEntry& ToBuild : BuildRequestListLocal)
	{
		if (UObject* Object = FindObject<UObject>(nullptr, *ToBuild.AssetPath, EFindObjectFlags::None))
		{
			ProcessBuildRequest(Object, ToBuild.bDirtyAssets);
		}
	}
}

void UDMBuildRequestSubsystem::ProcessBuildRequest(UObject* InToBuild, bool bInDirtyAssets)
{
	if (!IsValid(InToBuild))
	{
		return;
	}

	if (InToBuild->GetClass()->ImplementsInterface(UDMBuildable::StaticClass()) == false)
	{
		return;
	}

	IDMBuildable::Execute_DoBuild(InToBuild, bInDirtyAssets);
}
