// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "TickableEditorObject.h"

#include "Containers/Set.h"

#include "DMBuildRequestSubsystem.generated.h"

struct FDMBuildRequestEntry
{
	FString AssetPath;
	bool bDirtyAssets;

	bool operator==(const FDMBuildRequestEntry& InOther) const
	{
		return AssetPath == InOther.AssetPath;
	}

	friend uint32 GetTypeHash(const FDMBuildRequestEntry& InEntry)
	{
		return GetTypeHash(InEntry.AssetPath);
	}
};

UCLASS()
class UDMBuildRequestSubsystem : public UEditorSubsystem, public FTickableEditorObject
{
	GENERATED_BODY()

public:
	static UDMBuildRequestSubsystem* Get();

	void AddBuildRequest(UObject* InToBuild, bool bInDirtyAssets);

	void RemoveBuildRequest(UObject* InToNotBuild);

	void RemoveBuildRequestForOuter(UObject* InOuter);

	//~ Begin USubsystem
	virtual void Deinitialize() override;
	//~ End USubsystem

	//~ Begin FTickableEditorObject
	virtual void Tick(float InDeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject

protected:
	TSet<FDMBuildRequestEntry> BuildRequestList;

	void ProcessBuildRequestList();

	void ProcessBuildRequest(UObject* InToBuild, bool bInDirtyAssets);
};
