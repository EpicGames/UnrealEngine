// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMInputNodeBuilder.h"
#include "DMEDefs.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Model/DMMaterialBuildState.h"
#include "Model/DMMaterialBuildUtils.h"
#include "Utils/DMPrivate.h"
#include "Utils/DMUtils.h"

namespace UE::DynamicMaterialEditor::Private
{
	void BuildExpressionInputs(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap, const TArray<FDMInputInputs>& Inputs)
	{
		switch (Inputs.Num())
		{
			case 0:
				break;

			case 1:
				BuildExpressionOneInput(InBuildState, InputConnectionMap, Inputs[0]);
				break;

			case 2:
				BuildExpressionTwoInputs(InBuildState, InputConnectionMap, Inputs[0], Inputs[1]);
				break;

			default:
				BuildExpressionThreeInputs(InBuildState, InputConnectionMap, Inputs[0], Inputs[1], Inputs[2]);
				break;
		}
	}

	void BuildExpressionOneInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const FDMInputInputs& InputOne)
	{
		int32 OutputIndex;
		int32 OutputChannel;

		const TArray<UMaterialExpression*> InputOneExpression = InBuildState->GetBuildUtils().CreateExpressionInputs(
			InputConnectionMap,
			InputOne.InputIndex,
			InputOne.ChannelInputs,
			OutputIndex,
			OutputChannel
		);

		InBuildState->GetBuildUtils().UpdatePreviewMaterial(
			InputOneExpression.Last(),
			OutputIndex,
			OutputChannel,
			32
		);
	}

	void BuildExpressionTwoInputs(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const FDMInputInputs& InputOne, const FDMInputInputs& InputTwo)
	{
		UMaterialExpressionTextureCoordinate* TexCoord = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionTextureCoordinate>(UE_DM_NodeComment_Default);
		UMaterialExpressionComponentMask* Mask = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionComponentMask>(UE_DM_NodeComment_Default);
		UMaterialExpressionIf* IfExpr = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionIf>(UE_DM_NodeComment_Default);

		TexCoord->ConnectExpression(&Mask->Input, 0);

		Mask->R = 1;
		Mask->G = 0;
		Mask->B = 0;
		Mask->A = 0;
		Mask->ConnectExpression(&IfExpr->A, 0);

		IfExpr->ConstB = 0.5;

		InBuildState->AddOtherExpressions({TexCoord, Mask, IfExpr});

		int32 OutputIndexOne;
		int32 OutputChannelOne;
		int32 OutputIndexTwo;
		int32 OutputChannelTwo;

		const TArray<UMaterialExpression*> InputOneExpressions = InBuildState->GetBuildUtils().CreateExpressionInputs(
			InputConnectionMap,
			InputOne.InputIndex,
			InputOne.ChannelInputs,
			OutputIndexOne,
			OutputChannelOne
		);

		const TArray<UMaterialExpression*> InputTwoExpressions = InBuildState->GetBuildUtils().CreateExpressionInputs(
			InputConnectionMap,
			InputTwo.InputIndex,
			InputTwo.ChannelInputs,
			OutputIndexTwo,
			OutputChannelTwo
		);

		InputOneExpressions.Last()->ConnectExpression(&IfExpr->ALessThanB, OutputIndexOne);
		InputOneExpressions.Last()->ConnectExpression(&IfExpr->AEqualsB, OutputIndexOne);
		InputTwoExpressions.Last()->ConnectExpression(&IfExpr->AGreaterThanB, OutputIndexTwo);

		const TArray<FExpressionOutput>& OutputsOne = InputOneExpressions.Last()->GetOutputs();
		const TArray<FExpressionOutput>& OutputsTwo = InputTwoExpressions.Last()->GetOutputs();

		UE::DynamicMaterialEditor::Private::SetMask(IfExpr->ALessThanB, OutputsOne[OutputIndexOne], OutputChannelOne);
		UE::DynamicMaterialEditor::Private::SetMask(IfExpr->AEqualsB, OutputsOne[OutputIndexOne], OutputChannelOne);
		UE::DynamicMaterialEditor::Private::SetMask(IfExpr->AGreaterThanB, OutputsTwo[OutputIndexTwo], OutputChannelTwo);

		InBuildState->GetBuildUtils().UpdatePreviewMaterial(IfExpr, 0, 0, 32);
	}

	void BuildExpressionThreeInputs(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const FDMInputInputs& InputOne, const FDMInputInputs& InputTwo, const FDMInputInputs& InputThree)
	{
		UMaterialExpressionTextureCoordinate* TexCoord = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionTextureCoordinate>(UE_DM_NodeComment_Default);
		UMaterialExpressionComponentMask* Mask = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionComponentMask>(UE_DM_NodeComment_Default);
		UMaterialExpressionIf* IfExpr1 = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionIf>(UE_DM_NodeComment_Default);
		UMaterialExpressionIf* IfExpr2 = InBuildState->GetBuildUtils().CreateExpression<UMaterialExpressionIf>(UE_DM_NodeComment_Default);

		TexCoord->ConnectExpression(&Mask->Input, 0);

		Mask->R = 1;
		Mask->G = 0;
		Mask->B = 0;
		Mask->A = 0;
		Mask->ConnectExpression(&IfExpr1->A, 0);
		Mask->ConnectExpression(&IfExpr2->A, 0);

		IfExpr1->ConstB = 0.33f;
		IfExpr1->ConnectExpression(&IfExpr2->ALessThanB, 0);
		IfExpr1->ConnectExpression(&IfExpr2->AEqualsB, 0);

		IfExpr2->ConstB = 0.66f;

		InBuildState->AddOtherExpressions({TexCoord, Mask, IfExpr1, IfExpr1});

		int32 OutputIndexOne;
		int32 OutputChannelOne;
		int32 OutputIndexTwo;
		int32 OutputChannelTwo;
		int32 OutputIndexThree;
		int32 OutputChannelThree;

		const TArray<UMaterialExpression*> InputOneExpressions = InBuildState->GetBuildUtils().CreateExpressionInputs(
			InputConnectionMap,
			InputOne.InputIndex,
			InputOne.ChannelInputs,
			OutputIndexOne,
			OutputChannelOne
		);

		const TArray<UMaterialExpression*> InputTwoExpressions = InBuildState->GetBuildUtils().CreateExpressionInputs(
			InputConnectionMap,
			InputTwo.InputIndex,
			InputTwo.ChannelInputs,
			OutputIndexTwo,
			OutputChannelTwo
		);

		const TArray<UMaterialExpression*> InputThreeExpressions = InBuildState->GetBuildUtils().CreateExpressionInputs(
			InputConnectionMap,
			InputThree.InputIndex,
			InputThree.ChannelInputs,
			OutputIndexThree,
			OutputChannelThree
		);

		InputOneExpressions.Last()->ConnectExpression(&IfExpr1->ALessThanB, OutputIndexOne);
		InputOneExpressions.Last()->ConnectExpression(&IfExpr1->AEqualsB, OutputIndexOne);
		InputTwoExpressions.Last()->ConnectExpression(&IfExpr1->AGreaterThanB, OutputIndexTwo);
		InputThreeExpressions.Last()->ConnectExpression(&IfExpr2->AGreaterThanB, OutputIndexThree);

		const TArray<FExpressionOutput>& OutputsOne = InputOneExpressions.Last()->GetOutputs();
		const TArray<FExpressionOutput>& OutputsTwo = InputTwoExpressions.Last()->GetOutputs();
		const TArray<FExpressionOutput>& OutputsThree = InputThreeExpressions.Last()->GetOutputs();

		UE::DynamicMaterialEditor::Private::SetMask(IfExpr1->ALessThanB, OutputsOne[OutputIndexOne], OutputChannelOne);
		UE::DynamicMaterialEditor::Private::SetMask(IfExpr1->AEqualsB, OutputsOne[OutputIndexOne], OutputChannelOne);
		UE::DynamicMaterialEditor::Private::SetMask(IfExpr1->AGreaterThanB, OutputsTwo[OutputIndexTwo], OutputChannelTwo);
		UE::DynamicMaterialEditor::Private::SetMask(IfExpr2->AGreaterThanB, OutputsThree[OutputIndexThree], OutputChannelThree);

		InBuildState->GetBuildUtils().UpdatePreviewMaterial(IfExpr2, 0, FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 32);
	}
}