// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UDMMaterialStageInput;
struct FDMMaterialBuildState;
struct FDMMaterialStageConnection;

namespace UE::DynamicMaterialEditor::Private
{
	struct FDMInputInputs;

	void BuildExpressionInputs(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const TArray<FDMInputInputs>& Inputs);

	void BuildExpressionOneInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const FDMInputInputs& InputOne);

	void BuildExpressionTwoInputs(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const FDMInputInputs& InputOne, const FDMInputInputs& InputTwo);

	void BuildExpressionThreeInputs(const TSharedRef<FDMMaterialBuildState>& InBuildState, const TArray<FDMMaterialStageConnection>& InputConnectionMap,
		const FDMInputInputs& InputOne, const FDMInputInputs& InputTwo, const FDMInputInputs& InputThree);
}
