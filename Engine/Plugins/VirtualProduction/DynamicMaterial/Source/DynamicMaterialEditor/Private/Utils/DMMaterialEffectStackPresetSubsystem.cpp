// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialEffectStackPresetSubsystem.h"

#include "Components/DMMaterialEffect.h"
#include "Components/DMMaterialEffectStack.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "DynamicMaterialEditorModule.h"
#include "Editor.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Utils/DMJsonUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialEffectStackPresetSubsystem)

#define LOCTEXT_NAMESPACE "DMMaterialEffectStackPresetSubsystem"

namespace UE::DynamicMaterialEditor::Private
{
	static const FString JsonPath = FString(TEXT("Config")) / FString(TEXT("MaterialEffectPresets"));
	static const FString EffectListFieldName = TEXT("Effects");
	static const FJsonObject::FStringType EffectClassFieldName(TEXT("Class"));
	static const FJsonObject::FStringType EffectDataFieldName(TEXT("Data"));

	FString GetEffectPresetBasePath(bool bInAllowCreate = true)
	{
		const FString ProjectPath = FPaths::ProjectDir() / JsonPath;

		if (FPaths::DirectoryExists(ProjectPath))
		{
			return ProjectPath;
		}

		if (bInAllowCreate)
		{
			IFileManager::Get().MakeDirectory(*ProjectPath, /* Create tree */ true);

			if (FPaths::DirectoryExists(ProjectPath))
			{
				return ProjectPath;
			}
		}

		return "";
	}

	FString GetEffectPresetPath(const FString& InPresetName)
	{
		return GetEffectPresetBasePath() / InPresetName + TEXT(".json");
	}

	bool LoadPreset(const FString& InPath, FDMMaterialEffectStackJson& OutPreset)
	{
		FString FileJsonString;

		if (!FFileHelper::LoadFileToString(FileJsonString, *InPath))
		{
			UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: LoadPreset() - Unable to load Json file: %ls", *InPath);
			return false;
		}

		const TSharedPtr<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(FileJsonString);
		TSharedPtr<FJsonObject> RootObject;

		if (!FJsonSerializer::Deserialize(JsonReader.ToSharedRef(), RootObject))
		{
			UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: LoadPreset() - Unable to parse file [%ls]. Json=[%ls]", *InPath, *FileJsonString);
			return false;
		}

		if (!RootObject->HasTypedField<EJson::Array>(EffectListFieldName))
		{
			UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: LoadPreset() - Malformed data in json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
			return false;
		}

		const TArray<TSharedPtr<FJsonValue>> EffectList = RootObject->GetArrayField(EffectListFieldName);

		FDMMaterialEffectStackJson MaterialEffectStackJson;
		MaterialEffectStackJson.Effects.Reserve(EffectList.Num());

		for (const TSharedPtr<FJsonValue>& JsonMaterialEffect : EffectList)
		{
			const TSharedPtr<FJsonObject>* JsonMaterialEffectObject = nullptr;

			if (JsonMaterialEffect->TryGetObject(JsonMaterialEffectObject))
			{
				if (!(*JsonMaterialEffectObject)->HasTypedField<EJson::Object>(EffectDataFieldName))
				{
					UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: LoadPreset() - Malformed data in json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
					return false;
				}

				FDMMaterialEffectJson MaterialEffectJson;

				if (!FDMJsonUtils::Deserialize<UDMMaterialEffect>((*JsonMaterialEffectObject)->Values[EffectClassFieldName], MaterialEffectJson.Class))
				{
					UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: LoadPreset() - Invalid class in json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
					return false;
				}

				MaterialEffectJson.Data = (*JsonMaterialEffectObject)->Values[EffectDataFieldName];

				MaterialEffectStackJson.Effects.Add(MaterialEffectJson);
			}
			else
			{
				UE_LOGF(LogDynamicMaterialEditor, Warning, "Material Effect Preset Subsystem: LoadPreset() - Malformed data in json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
				return false;
			}
		}

		OutPreset = MoveTemp(MaterialEffectStackJson);

		return true;
	}

	bool SavePreset(const FString& InPath, const FDMMaterialEffectStackJson& InPreset)
	{
		TArray<TSharedPtr<FJsonValue>> EffectList;
		EffectList.Reserve(InPreset.Effects.Num());

		for (const FDMMaterialEffectJson& MaterialEffectJson : InPreset.Effects)
		{
			TSharedRef<FJsonObject> JsonMaterialEffectObject = MakeShared<FJsonObject>();
			JsonMaterialEffectObject->SetField(EffectClassFieldName, FDMJsonUtils::Serialize(MaterialEffectJson.Class));
			JsonMaterialEffectObject->SetField(EffectDataFieldName, MaterialEffectJson.Data);

			EffectList.Add(MakeShared<FJsonValueObject>(JsonMaterialEffectObject));
		}

		TSharedRef<FJsonObject> RootObject = MakeShared<FJsonObject>();
		RootObject->SetArrayField(EffectListFieldName, EffectList);

		FString FileJsonString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&FileJsonString);

		if (!FJsonSerializer::Serialize(RootObject, Writer))
		{
			UE_LOGF(LogDynamicMaterialEditor, Warning, "Material Effect Preset Subsystem: SavePreset() - Unable to serialize [%ls]. Json=[%ls]", *InPath, *FileJsonString);
			return false;
		}

		if (!FFileHelper::SaveStringToFile(FileJsonString, *InPath))
		{
			UE_LOGF(LogDynamicMaterialEditor, Warning, "Material Effect Preset Subsystem: SavePreset() - Unable to save Json file: %ls", *InPath);
			return false;
		}

		return true;
	}
}

UDMMaterialEffectStackPresetSubsystem* UDMMaterialEffectStackPresetSubsystem::Get()
{
	if (!GEditor)
	{
		return nullptr;
	}

	return GEditor->GetEditorSubsystem<UDMMaterialEffectStackPresetSubsystem>();
}

bool UDMMaterialEffectStackPresetSubsystem::SavePreset(const FString& InPresetName, const FDMMaterialEffectStackJson& InPreset) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	if (InPreset.Effects.Num() == 0)
	{
		return false;
	}

	const FString Path = GetEffectPresetBasePath(/* Allow create */ true);

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return false;
	}

	const FString PresetFile = GetEffectPresetPath(InPresetName);

	if (!UE::DynamicMaterialEditor::Private::SavePreset(PresetFile, InPreset))
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: SavePreset() - Failed [%ls]", *InPresetName);
		return false;
	}

	UE_LOGF(LogDynamicMaterialEditor, Log, "Material Effect Preset Subsystem: SavePreset() - Success [%ls]", *InPresetName);
	return true;
}

bool UDMMaterialEffectStackPresetSubsystem::LoadPreset(const FString& InPresetName, FDMMaterialEffectStackJson& OutPreset) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	const FString Path = GetEffectPresetBasePath(/* Allow create */ false);

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return false;
	}

	const FString PresetFile = GetEffectPresetPath(InPresetName);

	if (!UE::DynamicMaterialEditor::Private::LoadPreset(PresetFile, OutPreset))
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Material Effect Preset Subsystem: LoadPreset() - Failed [%ls]", *InPresetName);
		return false;
	}

	UE_LOGF(LogDynamicMaterialEditor, Log, "Material Effect Preset Subsystem: LoadPreset() - Success [%ls]", *InPresetName);
	return true;
}

bool UDMMaterialEffectStackPresetSubsystem::RemovePreset(const FString& InPresetName) const
{
	using namespace UE::DynamicMaterialEditor::Private;

	const FString Path = GetEffectPresetBasePath(/* Allow create */ false);

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return false;
	}

	const FString PresetFile = GetEffectPresetPath(InPresetName);

	if (!FPaths::FileExists(PresetFile))
	{
		return false;
	}

	return IFileManager::Get().Delete(*PresetFile);
}

TArray<FString> UDMMaterialEffectStackPresetSubsystem::GetPresetNames() const
{
	using namespace UE::DynamicMaterialEditor::Private;

	const FString Path = GetEffectPresetBasePath(/* Allow create */ false);

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return {};
	}

	TArray<FString> JsonFiles;
	IFileManager::Get().FindFiles(JsonFiles, *Path, TEXT(".json"));

	TArray<FString> PresetNames;

	for (const FString& JsonFile : JsonFiles)
	{
		const FString JsonFilePath = Path / JsonFile;
		const FString PresetName = FPaths::GetBaseFilename(JsonFilePath, /* Remove Path */ true);
		PresetNames.Add(PresetName);
	}

	return PresetNames;
}

#undef LOCTEXT_NAMESPACE
