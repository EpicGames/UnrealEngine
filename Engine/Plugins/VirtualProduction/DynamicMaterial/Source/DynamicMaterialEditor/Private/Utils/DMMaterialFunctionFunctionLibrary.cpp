// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialFunctionFunctionLibrary.h"

#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Components/MaterialValues/DMMaterialValueFloat2.h"
#include "Components/MaterialValues/DMMaterialValueFloat3RGB.h"
#include "Components/MaterialValues/DMMaterialValueFloat3XYZ.h"
#include "Components/MaterialValues/DMMaterialValueFloat4.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DMDefs.h"
#include "DynamicMaterialEditorSettings.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialFunctionFunctionLibrary)

#define LOCTEXT_NAMESPACE "DMMaterialFunctionFunctionLibrary"

void UDMMaterialFunctionFunctionLibrary::ApplyMetaData(const FFunctionExpressionInput& InFunctionInput,
	const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	const TArray<FString> MetaNames = {TEXT("UIMin"), TEXT("UIMax"), TEXT("ClampMin"), TEXT("ClampMax")};

	TArray<FString> MetaDatas;
	InFunctionInput.ExpressionInput->Desc.ParseIntoArray(MetaDatas, TEXT(","));

	for (FString& MetaData : MetaDatas)
	{
		const int32 EqualsPosition = MetaData.Find(TEXT("="));

		if (EqualsPosition == INDEX_NONE)
		{
			continue;
		}

		const FString MetaDataName = MetaData.Left(EqualsPosition).TrimStartAndEnd();

		bool bValidName = false;

		for (const FString& Name : MetaNames)
		{
			if (MetaDataName.Equals(Name, ESearchCase::IgnoreCase))
			{
				bValidName = true;
				break;
			}
		}

		if (!bValidName)
		{
			continue;
		}

		const FString MetaDataValue = MetaData.Mid(EqualsPosition + 1).TrimStartAndEnd();
		bool bValidValue = true;

		for (int32 Index = 0; Index < MetaDataValue.Len(); ++Index)
		{
			if (MetaDataValue[Index] != '.' && MetaDataValue[Index] != '-' && (MetaDataValue[Index] < '0' || MetaDataValue[Index] > '9'))
			{
				bValidValue = false;
				break;
			}
		}

		if (!bValidValue)
		{
			continue;
		}

		InPropertyHandle->SetInstanceMetaData(*MetaDataName, MetaDataValue);
	}
}

EDMValueType UDMMaterialFunctionFunctionLibrary::GetInputValueType(UMaterialExpressionFunctionInput* InFunctionInput)
{
	if (!InFunctionInput)
	{
		return EDMValueType::VT_None;
	}

	switch (InFunctionInput->InputType)
	{
		case EFunctionInputType::FunctionInput_Scalar:
			return EDMValueType::VT_Float1;

		case EFunctionInputType::FunctionInput_Vector2:
			return EDMValueType::VT_Float2;

		case EFunctionInputType::FunctionInput_Vector3:
			if (UDynamicMaterialEditorSettings::IsUseLinearColorForVectorsEnabled())
			{
				return EDMValueType::VT_Float3_RGB;
			}

			return EDMValueType::VT_Float3_XYZ;

		case EFunctionInputType::FunctionInput_Vector4:
			return EDMValueType::VT_Float4_RGBA;

		case EFunctionInputType::FunctionInput_Texture2D:
		case EFunctionInputType::FunctionInput_TextureCube:
		case EFunctionInputType::FunctionInput_VolumeTexture:
			return EDMValueType::VT_Texture;

		default:
			return EDMValueType::VT_None;
	}
}

void UDMMaterialFunctionFunctionLibrary::SetInputDefault(UMaterialExpressionFunctionInput* InFunctionInput, UDMMaterialValue* InValue)
{
	if (!InFunctionInput || !InFunctionInput->bUsePreviewValueAsDefault)
	{
		return;
	}

	const EDMValueType ValueType = GetInputValueType(InFunctionInput);

	switch (ValueType)
	{
		case EDMValueType::VT_Float1:
			if (UDMMaterialValueFloat1* Float1Value = Cast<UDMMaterialValueFloat1>(InValue))
			{
				Float1Value->SetDefaultValue(InFunctionInput->PreviewValue.X);
				Float1Value->ApplyDefaultValue();
			}
			break;

		case EDMValueType::VT_Float2:
			if (UDMMaterialValueFloat2* Float2Value = Cast<UDMMaterialValueFloat2>(InValue))
			{
				Float2Value->SetDefaultValue({InFunctionInput->PreviewValue.X, InFunctionInput->PreviewValue.Y});
				Float2Value->ApplyDefaultValue();
			}
			break;

		case EDMValueType::VT_Float3_XYZ:
			if (UDMMaterialValueFloat3XYZ* Float3XYZ = Cast<UDMMaterialValueFloat3XYZ>(InValue))
			{
				Float3XYZ->SetDefaultValue({InFunctionInput->PreviewValue.X, InFunctionInput->PreviewValue.Y, InFunctionInput->PreviewValue.Z});
				Float3XYZ->ApplyDefaultValue();
			}
			break;

		case EDMValueType::VT_Float3_RGB:
			if (UDMMaterialValueFloat3RGB* Float3RGB = Cast<UDMMaterialValueFloat3RGB>(InValue))
			{
				Float3RGB->SetDefaultValue({InFunctionInput->PreviewValue.X, InFunctionInput->PreviewValue.Y, InFunctionInput->PreviewValue.Z});
				Float3RGB->ApplyDefaultValue();
			}
			break;

		case EDMValueType::VT_Float4_RGBA:
			if (UDMMaterialValueFloat4* Float4Value = Cast<UDMMaterialValueFloat4>(InValue))
			{
				Float4Value->SetDefaultValue(InFunctionInput->PreviewValue);
				Float4Value->ApplyDefaultValue();
			}
			break;

		case EDMValueType::VT_Texture:
			if (UDMMaterialValueTexture* TextureValue = Cast<UDMMaterialValueTexture>(InValue))
			{
				SetInputDefault_Texture(InFunctionInput, TextureValue);
			}
			break;

		default:
			// Not possible
			break;
	}
}

void UDMMaterialFunctionFunctionLibrary::SetInputDefault_Texture(UMaterialExpressionFunctionInput* InFunctionInput, UDMMaterialValueTexture* InTextureValue)
{
	if (!InFunctionInput || !InTextureValue)
	{
		return;
	}

	UMaterialExpressionTextureObject* TextureObject = Cast<UMaterialExpressionTextureObject>(InFunctionInput->Preview.Expression);

	if (!TextureObject || !TextureObject->Texture)
	{
		return;
	}

	InTextureValue->SetDefaultValue(TextureObject->Texture);
	InTextureValue->ApplyDefaultValue();
}

#undef LOCTEXT_NAMESPACE
