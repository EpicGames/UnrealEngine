// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialFunctionLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

UMaterialFunctionInterface* FDMMaterialFunctionLibrary::LoadFunction(const FString& Path)
{
	return LoadObject<UMaterialFunctionInterface>(nullptr, *Path);
}

UMaterialFunctionInterface* FDMMaterialFunctionLibrary::GetFunction(const FName& Name, const FString& Path)
{
	TStrongObjectPtr<UMaterialFunctionInterface>* FunctionPtr = LoadedFunctions.Find(Name);

	if (FunctionPtr)
	{
		return (*FunctionPtr).Get();
	}

	UMaterialFunctionInterface* LoadedFunction = LoadFunction(Path);

	// Will emplace null to stop it uselessly looking it up in the future.
	LoadedFunctions.Emplace(Name, TStrongObjectPtr<UMaterialFunctionInterface>(LoadedFunction));

	return LoadedFunction;
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::MakeExpression(UMaterial* Parent, UMaterialFunctionInterface* Function, const FString& InComment)
{
	check(Parent);
	check(Function);

	UMaterialExpressionMaterialFunctionCall* MFC = NewObject<UMaterialExpressionMaterialFunctionCall>(Parent, NAME_None, RF_Transactional);
	MFC->SetMaterialFunction(Function);
	MFC->UpdateFromFunctionResource();
	MFC->Desc = InComment;

	Parent->GetEditorOnlyData()->ExpressionCollection.AddExpression(MFC);

	return MFC;
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::MakeExpression(UMaterial* Parent, const FName& Name, const FString& Path, const FString& InComment)
{
	check(Parent);

	UMaterialFunctionInterface* Function = GetFunction(Name, Path);

	if (!Function)
	{
		return nullptr;
	}

	return MakeExpression(Parent, Function, InComment);
}

FDMMaterialFunctionLibrary& FDMMaterialFunctionLibrary::Get()
{
	static FDMMaterialFunctionLibrary FunctionLibrary;
	return FunctionLibrary;
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetBreakOutFloat2Components(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"BreakOutFloat2Components",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat2Components.BreakOutFloat2Components"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetBreakOutFloat3Components(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"BreakOutFloat3Components",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat3Components.BreakOutFloat3Components"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetBreakOutFloat4Components(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"BreakOutFloat4Components",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/BreakOutFloat4Components.BreakOutFloat4Components"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetRGVtoHSV(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"RGVtoHSV",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/RGBtoHSV.RGBtoHSV"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetMakeFloat2(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"MakeFloat2",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat2.MakeFloat2"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetMakeFloat3(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"MakeFloat3",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat3.MakeFloat3"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetMakeFloat4(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"MakeFloat4",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Utility/MakeFloat4.MakeFloat4"),
		InComment
	);
}


UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetCustomRotator(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"CustomRotator",
		TEXT("/Engine/Functions/Engine_MaterialFunctions02/Texturing/CustomRotator.CustomRotator"),
		InComment
	);
}

UMaterialExpressionMaterialFunctionCall* FDMMaterialFunctionLibrary::GetRadialGradientExponential(UMaterial* Parent, const FString& InComment)
{
	return MakeExpression(
		Parent,
		"RadialGradientExponential",
		TEXT("/Engine/Functions/Engine_MaterialFunctions01/Gradient/RadialGradientExponential.RadialGradientExponential"),
		InComment
	);
}
