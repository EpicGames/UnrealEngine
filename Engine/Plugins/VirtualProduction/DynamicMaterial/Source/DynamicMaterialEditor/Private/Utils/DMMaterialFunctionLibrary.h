// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialFunctionInterface.h"
#include "UObject/StrongObjectPtr.h"

class UMaterial;
class UMaterialExpressionMaterialFunctionCall;

class FDMMaterialFunctionLibrary
{
public:
	static FDMMaterialFunctionLibrary& Get();

	UMaterialFunctionInterface* GetFunction(const FName& Name, const FString& Path);

	UMaterialFunctionInterface* LoadFunction(const FString& Path);

	UMaterialExpressionMaterialFunctionCall* MakeExpression(UMaterial* Parent, UMaterialFunctionInterface* Function, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* MakeExpression(UMaterial* Parent, const FName& Name, const FString& Path, const FString& InComment);

	UMaterialExpressionMaterialFunctionCall* GetBreakOutFloat2Components(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetBreakOutFloat3Components(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetBreakOutFloat4Components(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetRGVtoHSV(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetMakeFloat2(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetMakeFloat3(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetMakeFloat4(UMaterial* Parent, const FString& InComment);
	UMaterialExpressionMaterialFunctionCall* GetCustomRotator(UMaterial* Parent, const FString& InComment); /** Possibly promote this to a full expression? */
	UMaterialExpressionMaterialFunctionCall* GetRadialGradientExponential(UMaterial* Parent, const FString& InComment); /** Possibly promote this to a full expression? */

protected:
	TMap<FName, TStrongObjectPtr<UMaterialFunctionInterface>> LoadedFunctions;
};
