// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialInstanceFunctionLibrary.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/DMMaterialLayer.h"
#include "Components/PrimitiveComponent.h"
#include "ContentBrowserModule.h"
#include "DMObjectMaterialProperty.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorModule.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "IContentBrowserSingleton.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Model/IDynamicMaterialModelEditorOnlyDataInterface.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialInstanceFunctionLibrary)

#define LOCTEXT_NAMESPACE "DMMaterialInstanceFunctionLibrary"

TArray<FDMObjectMaterialProperty> UDMMaterialInstanceFunctionLibrary::GetActorMaterialProperties(AActor* InActor)
{
	TArray<FDMObjectMaterialProperty> ActorProperties;

	if (!IsValid(InActor))
	{
		return ActorProperties;
	}

	FDMGetObjectMaterialPropertiesDelegate PropertyGenerator = FDynamicMaterialEditorModule::GetCustomMaterialPropertyGenerator(InActor->GetClass());

	if (PropertyGenerator.IsBound())
	{
		ActorProperties = PropertyGenerator.Execute(InActor);

		if (!ActorProperties.IsEmpty())
		{
			return ActorProperties;
		}
	}

	InActor->ForEachComponent<UPrimitiveComponent>(/* Include Child Actors */ false, [&ActorProperties](UPrimitiveComponent* InComp)
		{
			for (int32 MaterialIdx = 0; MaterialIdx < InComp->GetNumMaterials(); ++MaterialIdx)
			{
				ActorProperties.Add({InComp, MaterialIdx});
			}
		});

	return ActorProperties;
}

bool UDMMaterialInstanceFunctionLibrary::SetMaterialInObject(FDMObjectMaterialProperty& InMaterialProperty,
	UDynamicMaterialInstance* InInstance)
{
	if (!InMaterialProperty.IsValid())
	{
		return false;
	}

	bool bSubsystemTakenOver = false;

	if (UObject* const Outer = InMaterialProperty.GetOuter())
	{
		if (const UWorld* const World = Outer->GetWorld())
		{
			if (IsValid(World))
			{
				if (UDMWorldSubsystem* const WorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>())
				{
					bSubsystemTakenOver = WorldSubsystem->ExecuteMaterialValueSetterDelegate(InMaterialProperty, InInstance);
				}
			}
		}
	}

	if (!bSubsystemTakenOver)
	{
		InMaterialProperty.SetMaterial(InInstance);
	}

	return InMaterialProperty.GetMaterial() == InInstance;
}

UDynamicMaterialModel* UDMMaterialInstanceFunctionLibrary::CreateMaterialInObject(FDMObjectMaterialProperty& InMaterialProperty)
{
	if (!InMaterialProperty.IsValid())
	{
		return nullptr;
	}

	UObject* const Outer = InMaterialProperty.GetOuter();

	if (!Outer)
	{
		return nullptr;
	}

	UDynamicMaterialInstanceFactory* const InstanceFactory = NewObject<UDynamicMaterialInstanceFactory>();
	check(InstanceFactory);

	UDynamicMaterialInstance* const NewInstance = Cast<UDynamicMaterialInstance>(InstanceFactory->FactoryCreateNew(UDynamicMaterialInstance::StaticClass(),
		Outer, NAME_None, RF_Transactional, nullptr, GWarn));
	check(NewInstance);

	UDynamicMaterialModel* MaterialModel = NewInstance->GetMaterialModel();

	if (!MaterialModel)
	{
		return nullptr;
	}

	SetMaterialInObject(InMaterialProperty, NewInstance);

	if (IDynamicMaterialModelEditorOnlyDataInterface* EditorOnlyData = MaterialModel->GetEditorOnlyData())
	{
		EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Async);
	}

	return MaterialModel;
}

#undef LOCTEXT_NAMESPACE
