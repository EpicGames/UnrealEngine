// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialModelFunctionLibrary.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Components/DMMaterialLayer.h"
#include "Components/PrimitiveComponent.h"
#include "ContentBrowserModule.h"
#include "DMWorldSubsystem.h"
#include "DynamicMaterialEditorModule.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "GameFramework/Actor.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelDynamic.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Model/IDynamicMaterialModelEditorOnlyDataInterface.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Utils/DMPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialModelFunctionLibrary)

#define LOCTEXT_NAMESPACE "DMMaterialModelFunctionLibrary"

UDynamicMaterialInstance* UDMMaterialModelFunctionLibrary::ExportMaterial(UDynamicMaterialModelBase* InMaterialModelBase)
{
	if (!IsValid(InMaterialModelBase))
	{
		return nullptr;
	}

	UDynamicMaterialInstance* MaterialInstance = InMaterialModelBase->GetDynamicMaterialInstance();

	if (!IsValid(MaterialInstance))
	{
		return nullptr;
	}

	const FString CurrentName = (InMaterialModelBase->IsA<UDynamicMaterialModel>() ? TEXT("MD_") : TEXT("MDI_")) + RemoveAssetPrefix(MaterialInstance->GetName());

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString PackageName, AssetName;
	AssetTools.CreateUniqueAssetName(CurrentName, TEXT(""), PackageName, AssetName);

	IContentBrowserSingleton& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowserModule.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;

	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return nullptr;
	}

	return ExportMaterial(MaterialInstance->GetMaterialModelBase(), SaveObjectPath);
}

UDynamicMaterialInstance* UDMMaterialModelFunctionLibrary::ExportMaterial(UDynamicMaterialModelBase* InMaterialModel, const FString& InSavePath)
{
	if (!IsValid(InMaterialModel))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid material to export."));
		return nullptr;
	}

	if (InSavePath.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid material save path to export."));
		return nullptr;
	}

	UDynamicMaterialInstance* Instance = InMaterialModel->GetDynamicMaterialInstance();
	if (!Instance)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to find a Material Designer Material to export."), true, InMaterialModel);
		return nullptr;
	}

	const FString PackagePath = FPaths::GetBaseFilename(*InSavePath, false);

	UPackage* Package = CreatePackage(*PackagePath);
	if (!Package)
	{
		UE::DynamicMaterialEditor::Private::LogError(FString::Printf(TEXT("Failed to create package for Material Designer Material (%s)."), *PackagePath));
		return nullptr;
	}

	const FString AssetName = FPaths::GetBaseFilename(*InSavePath, true);

	FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(Instance, Package, *AssetName,
		RF_Public | RF_Standalone, nullptr, EDuplicateMode::Normal, EInternalObjectFlags::None);

	UObject* NewAsset = StaticDuplicateObjectEx(Params);
	if (!NewAsset)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to create new Material Designer Material asset."));
		return nullptr;
	}

	// Not sure why these need to be set again!
	NewAsset->SetFlags(RF_Public | RF_Standalone);

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(NewAsset);

	if (NewInstance)
	{
		if (InMaterialModel->IsA<UDynamicMaterialModel>())
		{
			if (UDynamicMaterialModel* NewModel = NewInstance->GetMaterialModel())
			{
				if (IDynamicMaterialModelEditorOnlyDataInterface* ModelEditorOnlyData = NewModel->GetEditorOnlyData())
				{
					ModelEditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Immediate);
				}
			}
		}
		else if (InMaterialModel->IsA<UDynamicMaterialModelDynamic>())
		{
			NewInstance->InitializeMIDPublic();
		}
	}

	FAssetRegistryModule::AssetCreated(NewAsset);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("ExportedMaterial"));
	}

	return NewInstance;
}

UMaterial* UDMMaterialModelFunctionLibrary::ExportGeneratedMaterial(UDynamicMaterialModelBase* InMaterialModelBase)
{
	return nullptr;
}

UMaterial* UDMMaterialModelFunctionLibrary::ExportGeneratedMaterial(UDynamicMaterialModelBase* InMaterialModelBase, const FString& InSavePath)
{
	if (!IsValid(InMaterialModelBase))
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid material to export."));
		return nullptr;
	}

	if (InSavePath.IsEmpty())
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Invalid material save path to export."));
		return nullptr;
	}

	UMaterial* GeneratedMaterial = InMaterialModelBase->GetGeneratedMaterial();

	if (!GeneratedMaterial)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to find a generated material to export."));
		return nullptr;
	}

	const FString PackagePath = FPaths::GetBaseFilename(*InSavePath, false);

	UPackage* Package = CreatePackage(*PackagePath);
	if (!Package)
	{
		UE::DynamicMaterialEditor::Private::LogError(FString::Printf(TEXT("Failed to create package for exported material (%s)."), *PackagePath));
		return nullptr;
	}

	const FString AssetName = FPaths::GetBaseFilename(*InSavePath, true);

	FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(GeneratedMaterial, Package, *AssetName,
		RF_Public | RF_Standalone, nullptr, EDuplicateMode::Normal, EInternalObjectFlags::None);

	UObject* NewAsset = StaticDuplicateObjectEx(Params);

	if (!NewAsset)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to create new material asset."));
		return nullptr;
	}

	// Not sure why these need to be set again!
	NewAsset->SetFlags(RF_Public | RF_Standalone);

	FAssetRegistryModule::AssetCreated(NewAsset);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("ExportedGeneratedMaterial"));
	}

	return Cast<UMaterial>(NewAsset);
}

UDynamicMaterialModel* UDMMaterialModelFunctionLibrary::ExportToTemplateMaterialModel(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDynamicMaterialModel* ParentModel = InMaterialModelDynamic->GetParentModel();

	if (!ParentModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find parent model.");
		return nullptr;
	}

	// Where should we save it? (Always export to CB)
	const FString CurrentName = TEXT("MDM_") + RemoveAssetPrefix(InMaterialModelDynamic->GetName());

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString PackageName, AssetName;
	AssetTools.CreateUniqueAssetName(CurrentName, TEXT(""), PackageName, AssetName);

	IContentBrowserSingleton& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowserModule.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;

	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "No path was chosen for saving the new editable asset, cancelling.");
		return nullptr;
	}

	return ExportToTemplateMaterialModel(InMaterialModelDynamic, SaveObjectPath);
}

UDynamicMaterialModel* UDMMaterialModelFunctionLibrary::ExportToTemplateMaterialModel(UDynamicMaterialModelDynamic* InMaterialModelDynamic, const FString& InSavePath)
{
	UDynamicMaterialModel* ParentModel = InMaterialModelDynamic->GetParentModel();

	if (!ParentModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find parent model.");
		return nullptr;
	}

	// Create new model
	UDynamicMaterialModel* NewModel = InMaterialModelDynamic->ToEditable(GetTransientPackage());

	if (!NewModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to convert dynamic asset to editable.");
		return nullptr;
	}

	// Create a package for it
	const FString PackageName = FPaths::GetBaseFilename(*InSavePath, false);
	UPackage* Package = CreatePackage(*PackageName);

	if (!Package)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to create new package for editable asset.");
		return nullptr;
	}

	const FString AssetName = FPaths::GetBaseFilename(*InSavePath, true);

	NewModel->Rename(nullptr, Package, UE::DynamicMaterial::RenameFlags);
	NewModel->SetFlags(RF_Transactional | RF_Public | RF_Standalone);

	FAssetRegistryModule::AssetCreated(NewModel);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("ExportToTemplateMaterialModel"));
	}

	return NewModel;
}

UDynamicMaterialInstance* UDMMaterialModelFunctionLibrary::ExportToTemplateMaterial(UDynamicMaterialModelDynamic* InMaterialModelDynamic)
{
	UDynamicMaterialModel* ParentModel = InMaterialModelDynamic->GetParentModel();

	if (!ParentModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find parent model.");
		return nullptr;
	}

	UDynamicMaterialInstance* OldInstance = InMaterialModelDynamic->GetDynamicMaterialInstance();

	if (!OldInstance)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find material instance.");
		return nullptr;
	}

	// Where should we save it? (Always export to CB)
	const FString CurrentName = TEXT("MD_") + RemoveAssetPrefix(OldInstance ? OldInstance->GetName() : InMaterialModelDynamic->GetName());

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString PackageName, AssetName;
	AssetTools.CreateUniqueAssetName(CurrentName, TEXT(""), PackageName, AssetName);

	IContentBrowserSingleton& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowserModule.GetCurrentPath();
	const FString PathStr = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathStr;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.DefaultAssetName = AssetName;

	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "No path was chosen for saving the new editable asset, cancelling.");
		return nullptr;
	}

	return ExportToTemplateMaterial(InMaterialModelDynamic, SaveObjectPath);	
}

UDynamicMaterialInstance* UDMMaterialModelFunctionLibrary::ExportToTemplateMaterial(UDynamicMaterialModelDynamic* InMaterialModelDynamic, const FString& InSavePath)
{
	UDynamicMaterialModel* ParentModel = InMaterialModelDynamic->GetParentModel();

	if (!ParentModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find parent model.");
		return nullptr;
	}

	UDynamicMaterialInstance* OldInstance = InMaterialModelDynamic->GetDynamicMaterialInstance();

	if (!OldInstance)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to find material instance.");
		return nullptr;
	}

	// Create new model
	UDynamicMaterialModel* NewModel = InMaterialModelDynamic->ToEditable(GetTransientPackage());

	if (!NewModel)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to convert dynamic asset to editable.");
		return nullptr;
	}

	// Create a package for it
	const FString PackageName = FPaths::GetBaseFilename(*InSavePath, false);
	UPackage* Package = CreatePackage(*PackageName);

	if (!Package)
	{
		UE_LOGF(LogDynamicMaterialEditor, Error, "Failed to create new package for editable asset.");
		return nullptr;
	}

	const FString AssetName = FPaths::GetBaseFilename(*InSavePath, true);

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(GetMutableDefault<UDynamicMaterialInstanceFactory>()->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(),
		Package,
		*AssetName,
		RF_Transactional | RF_Public | RF_Standalone,
		NewModel,
		nullptr
	));

	FAssetRegistryModule::AssetCreated(NewInstance);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MaterialDesigner"), TEXT("Action"), TEXT("ExportToTemplateMaterial"));
	}

	return NewInstance;
}

bool UDMMaterialModelFunctionLibrary::IsModelValid(UDynamicMaterialModelBase* InMaterialModelBase)
{
	if (!IsValid(InMaterialModelBase))
	{
		return false;
	}

	if (UWorld* World = InMaterialModelBase->GetWorld())
	{
		if (UDMWorldSubsystem* WorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>())
		{
			if (!WorldSubsystem->ExecuteIsValidDelegate(InMaterialModelBase))
			{
				return false;
			}
		}
	}

	UActorComponent* ComponentOuter = InMaterialModelBase->GetTypedOuter<UActorComponent>();

	if (ComponentOuter && !IsValid(ComponentOuter))
	{
		return false;
	}

	AActor* ActorOuter = InMaterialModelBase->GetTypedOuter<AActor>();

	if (ActorOuter && !IsValid(ActorOuter))
	{
		return false;
	}

	UPackage* PackageOuter = InMaterialModelBase->GetPackage();

	if (PackageOuter && !IsValid(PackageOuter))
	{
		return false;
	}

	return true;
}

bool UDMMaterialModelFunctionLibrary::DuplicateModelBetweenMaterials(UDynamicMaterialModel* InFromModel, UDynamicMaterialInstance* InToInstance)
{
	if (!InFromModel || !InToInstance)
	{
		return false;
	}

	UDynamicMaterialModelBase* CurrentModel = InToInstance->GetMaterialModelBase();
	FString CurrentName;

	if (CurrentModel)
	{
		CurrentName = CurrentModel->GetName();

		const FString NewName = CurrentName + TEXT("_OLD");
		const FName NewNameUnique = MakeUniqueObjectName(GetTransientPackage(), CurrentModel->GetClass(), *NewName);
		CurrentModel->Rename(*NewNameUnique.ToString(), GetTransientPackage(), UE::DynamicMaterial::RenameFlags);

		InToInstance->SetMaterialModel(nullptr);
	}

	FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(InFromModel, InToInstance, InFromModel->GetFName(),
		InFromModel->GetFlags(), nullptr, EDuplicateMode::Normal, EInternalObjectFlags::None);

	UDynamicMaterialModel* NewModel = Cast<UDynamicMaterialModel>(StaticDuplicateObjectEx(Params));

	if (!NewModel)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to copy Material Model."));

		// Put back the original model
		CurrentModel->Rename(CurrentName.IsEmpty() ? nullptr : *CurrentName, InToInstance, UE::DynamicMaterial::RenameFlags);

		return false;
	}

	NewModel->Rename(CurrentName.IsEmpty() ? nullptr : *CurrentName, InToInstance, UE::DynamicMaterial::RenameFlags);

	InToInstance->SetMaterialModel(NewModel);
	NewModel->SetDynamicMaterialInstance(InToInstance);
	InToInstance->InitializeMIDPublic();

	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(NewModel))
	{
		EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Async);
	}

	return true;
}

bool UDMMaterialModelFunctionLibrary::CreateModelInstanceInMaterial(UDynamicMaterialModel* InFromModel, UDynamicMaterialInstance* InToInstance)
{
	if (!InFromModel || !InToInstance)
	{
		return false;
	}

	UDynamicMaterialModelBase* CurrentModel = InToInstance->GetMaterialModelBase();
	FString CurrentName;	

	if (CurrentModel)
	{
		CurrentName = CurrentModel->GetName();

		const FString NewName = CurrentName + TEXT("_OLD");
		const FName NewNameUnique = MakeUniqueObjectName(GetTransientPackage(), CurrentModel->GetClass(), *NewName);
		CurrentModel->Rename(*NewNameUnique.ToString(), GetTransientPackage(), UE::DynamicMaterial::RenameFlags);

		InToInstance->SetMaterialModel(nullptr);
	}

	UDynamicMaterialModelDynamic* NewModelDynamic = UDynamicMaterialModelDynamic::Create(InToInstance, InFromModel);

	if (!NewModelDynamic)
	{
		UE::DynamicMaterialEditor::Private::LogError(TEXT("Failed to make Material Designer Model Instance."));

		// Put back the original model
		CurrentModel->Rename(CurrentName.IsEmpty() ? nullptr : *CurrentName, InToInstance, UE::DynamicMaterial::RenameFlags);

		return false;
	}

	NewModelDynamic->Rename(CurrentName.IsEmpty() ? nullptr : *CurrentName, InToInstance, UE::DynamicMaterial::RenameFlags);

	InToInstance->SetMaterialModel(NewModelDynamic);
	NewModelDynamic->SetDynamicMaterialInstance(InToInstance);
	InToInstance->InitializeMIDPublic();

	return true;
}

FString UDMMaterialModelFunctionLibrary::RemoveAssetPrefix(const FString& InAssetName)
{
	if (InAssetName.StartsWith(TEXT("MD_"))) // Material Designer asset
	{
		return InAssetName.RightChop(3);
	}
	else if (InAssetName.StartsWith(TEXT("MDI_")) // Material Designer Instance
		|| InAssetName.StartsWith(TEXT("MDD_")) // Material Designer Dynamic (defunct)
		|| InAssetName.StartsWith(TEXT("MDM_"))) // Material Designer Model
	{
		return InAssetName.RightChop(4);
	}
	else if (InAssetName.StartsWith(TEXT("MDMI_"))) // Material Designer Model Instance
	{
		return InAssetName.RightChop(5);
	}

	return InAssetName;
}

UDynamicMaterialModelBase* UDMMaterialModelFunctionLibrary::CreatePreviewModel(UDynamicMaterialModelBase* InOriginalModelBase)
{
	UDynamicMaterialModelBase* PreviewMaterialModelBase = Cast<UDynamicMaterialModelBase>(StaticDuplicateObject(InOriginalModelBase, GetTransientPackage()));
	PreviewMaterialModelBase->SetDynamicMaterialInstance(nullptr);

	if (UDynamicMaterialModelDynamic* MaterialModelDynamic = Cast<UDynamicMaterialModelDynamic>(PreviewMaterialModelBase))
	{
		MaterialModelDynamic->EnsureComponents();
	}

	if (UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(PreviewMaterialModelBase))
	{
		EditorOnlyData->RequestMaterialBuild(EDMBuildRequestType::Async);
	}

	PreviewMaterialModelBase->MarkOriginalUpdated();

	return PreviewMaterialModelBase;
}

void UDMMaterialModelFunctionLibrary::MirrorMaterialModel(UDynamicMaterialModelBase* InSource, UDynamicMaterialModelBase*& InTarget)
{
	if (!InSource || !InTarget || InSource->GetClass() != InTarget->GetClass())
	{
		return;
	}

	TMap<UObject*, UObject*> ReplacedObjects;

	FObjectDuplicationParameters Params(InSource, InTarget->GetOuter());
	Params.CreatedObjects = &ReplacedObjects;
	Params.DestClass = InSource->GetClass();
	Params.DestName = InTarget->GetFName();

	if (UDynamicMaterialInstance* SourceInstance = InSource->GetDynamicMaterialInstance())
	{
		if (UDynamicMaterialInstance* TargetInstance = InTarget->GetDynamicMaterialInstance())
		{
			Params.DuplicationSeed.Add(SourceInstance, TargetInstance);
		}
	}

	InTarget = Cast<UDynamicMaterialModelBase>(StaticDuplicateObjectEx(Params));

	GEngine->NotifyToolsOfObjectReplacement(ReplacedObjects);
}

UObject* UDMMaterialModelFunctionLibrary::FindSubobject(UObject* InOuter, FStringView InPath)
{
	if (InPath.IsEmpty())
	{
		return InOuter;
	}

	int32 Index = INDEX_NONE;
	InPath.FindChar(TEXT('.'), Index);

	if (Index >= 0)
	{
		const FStringView FirstPath = InPath.Left(Index);
		InPath.RightChopInline(Index + 1);

		if (!FirstPath.IsEmpty())
		{
			const FString PathSection(FirstPath);

			if (UObject* Subobject = StaticFindObjectFast(UObject::StaticClass(), InOuter, *PathSection))
			{
				return FindSubobject(Subobject, InPath);
			}
		}
	}

	const FString Path(InPath);
	return StaticFindObjectFast(UObject::StaticClass(), InOuter, *Path);
}

#undef LOCTEXT_NAMESPACE
