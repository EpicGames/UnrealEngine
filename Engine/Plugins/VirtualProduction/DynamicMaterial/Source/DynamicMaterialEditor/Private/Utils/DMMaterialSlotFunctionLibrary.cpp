// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialSlotFunctionLibrary.h"

#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageExpression.h"
#include "Components/DMMaterialStageFunction.h"
#include "Components/DMMaterialStageGradient.h"
#include "Components/DMMaterialStageThroughputLayerBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialStageBlends/DMMSBNormal.h"
#include "Components/MaterialStageExpressions/DMMSESceneTexture.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIFunction.h"
#include "Components/MaterialStageInputs/DMMSIGradient.h"
#include "Components/MaterialStageInputs/DMMSISlot.h"
#include "Components/MaterialStageInputs/DMMSITextureUV.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DMPrivate.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMMaterialStageFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialSlotFunctionLibrary)

#define LOCTEXT_NAMESPACE "UDMMaterialSlotFunctionLibrary"

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer(UDMMaterialSlot* InSlot, UDMMaterialStage* InNewBaseStage, 
	UDMMaterialStage* InNewMaskStage)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	if (InNewBaseStage && !ensure(IsValid(InNewBaseStage)))
	{
		return nullptr;
	}

	if (InNewMaskStage && !ensure(IsValid(InNewMaskStage)))
	{
		return nullptr;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = InSlot->GetMaterialModelEditorOnlyData();

	if (!ensure(IsValid(ModelEditorOnlyData)))
	{
		return nullptr;
	}

	UDMMaterialLayerObject* Layer = nullptr;

	{
		const FDMUpdateGuard Guard;

		FDMScopedUITransaction Transaction(LOCTEXT("AddLayer", "Add Layer"));
		InSlot->Modify();

		EDMMaterialPropertyType MaterialProperty = EDMMaterialPropertyType::None;

		if (InSlot->GetLayers().IsEmpty())
		{
			TArray<EDMMaterialPropertyType> SlotProperties = ModelEditorOnlyData->GetMaterialPropertiesForSlot(InSlot);

			if (ensureMsgf(!SlotProperties.IsEmpty(), TEXT("Cannot find material property.")))
			{
				MaterialProperty = SlotProperties[0];
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			MaterialProperty = InSlot->GetLayers().Last()->GetMaterialProperty();
		}

		if (!ensureMsgf(MaterialProperty != EDMMaterialPropertyType::None, TEXT("Could not find material property.")))
		{
			return nullptr;
		}

		if (!InNewBaseStage && !InNewMaskStage)
		{
			Layer = InSlot->AddDefaultLayer(MaterialProperty);
		}
		else if (!InNewMaskStage)
		{
			Layer = InSlot->AddLayer(MaterialProperty, InNewBaseStage);
		}
		else
		{
			Layer = InSlot->AddLayerWithMask(MaterialProperty, InNewBaseStage, InNewMaskStage);
		}
	}

	const bool bValidLayer = IsValid(Layer);
	UDMMaterialStageSource* Source = nullptr;

	if (bValidLayer)
	{
		if (UDMMaterialStage* Stage = Layer->GetStage(EDMMaterialLayerStage::Base))
		{
			Source = Stage->GetSource();
		}
	}

	if (IsValid(Source))
	{
		Source->Update(Layer, EDMUpdateType::Structure);
	}
	else if (bValidLayer)
	{
		Layer->Update(Layer, EDMUpdateType::Structure);
	}

	return Layer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddTextureLayer(UDMMaterialSlot* InSlot, UTexture* InTexture, 
	EDMMaterialPropertyType InPropertyType, bool bInReplaceSlot)
{
	if (GUndo)
	{
		InSlot->Modify();
	}

	UDMMaterialLayerObject* Layer = nullptr;

	const FDMUpdateGuard Guard;

	Layer = InSlot->AddDefaultLayer(InPropertyType);

	if (!ensure(Layer))
	{
		return nullptr;
	}

	bool bMadeChange = true;

	UDMMaterialStage* Stage = Layer->GetStage(EDMMaterialLayerStage::Base);

	if (!ensure(Stage))
	{
		return nullptr;
	}

	UDMMaterialStageInputExpression* NewExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		Stage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (!ensure(NewExpression))
	{
		return nullptr;
	}

	UDMMaterialSubStage* SubStage = NewExpression->GetSubStage();

	if (!ensure(SubStage))
	{
		return nullptr;
	}

	UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		SubStage,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		EDMValueType::VT_Texture,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (ensure(InputValue))
	{
		UDMMaterialValueTexture* InputTexture = Cast<UDMMaterialValueTexture>(InputValue->GetValue());

		if (ensure(InputTexture))
		{
			InputTexture->SetValue(InTexture);
		}
	}

	if (bInReplaceSlot)
	{
		for (int32 Index = InSlot->GetLayers().Num() - 1; Index >= 0; --Index)
		{
			UDMMaterialLayerObject* LayerIter = InSlot->GetLayer(Index);

			if (!LayerIter || LayerIter->GetStage(EDMMaterialLayerStage::Base) == Stage)
			{
				continue;
			}

			InSlot->RemoveLayer(LayerIter);
		}
	}

	Layer->Update(Layer, EDMUpdateType::Structure);

	return Layer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_NewLocalValue(UDMMaterialSlot* InSlot, EDMValueType InValueType)
{
	return AddNewLayer_NewLocalValue(
		InSlot,
		UDMValueDefinitionLibrary::GetValueDefinition(InValueType).GetValueClass()
	);
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_NewLocalValue(UDMMaterialSlot* InSlot, 
	TSubclassOf<UDMMaterialValue> InValueClass)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	UDMMaterialStage* NewBase = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewBase);

	UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		NewBase, 
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
		InValueClass,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_GlobalValue(UDMMaterialSlot* InSlot, UDMMaterialValue* InValue)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	UDynamicMaterialModelEditorOnlyData* ModelEditorOnlyData = InSlot->GetMaterialModelEditorOnlyData();
	if (!ensure(ModelEditorOnlyData))
	{
		return nullptr;
	}

	if (!ensure(IsValid(InValue)) || !ensure(InValue->GetMaterialModel() == ModelEditorOnlyData->GetMaterialModel()))
	{
		return nullptr;
	}

	UDMMaterialStage* NewBase = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewBase);

	UDMMaterialStageInputValue::ChangeStageInput_Value(
		NewBase, 
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
		InValue,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	// Already existing global values should not have their value changed to the property default.

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_NewGlobalValue(UDMMaterialSlot* InSlot, EDMValueType InValueType)
{
	return AddNewLayer_NewGlobalValue(
		InSlot,
		UDMValueDefinitionLibrary::GetValueDefinition(InValueType).GetValueClass()
	);
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_NewGlobalValue(UDMMaterialSlot* InSlot, 
	TSubclassOf<UDMMaterialValue> InValueClass)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewStage);

	UDMMaterialStageInputValue::ChangeStageInput_NewValue(
		NewStage, 
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		InValueClass,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_Slot(UDMMaterialSlot* InTargetSlot, UDMMaterialSlot* InSourceSlot, 
	EDMMaterialPropertyType InMaterialProperty)
{
	if (!InTargetSlot || !IsValid(InTargetSlot))
	{
		return nullptr;
	}

	if (!ensure(IsValid(InSourceSlot)) || !ensure(InTargetSlot != InSourceSlot)
		|| !ensure(InSourceSlot->GetMaterialModelEditorOnlyData() == InTargetSlot->GetMaterialModelEditorOnlyData()))
	{
		return nullptr;
	}

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InTargetSlot, NewStage);

	UDMMaterialStageInputSlot::ChangeStageInput_Slot(
		NewStage, 
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
		InSourceSlot, 
		InMaterialProperty, 
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_Expression(UDMMaterialSlot* InSlot, 
	TSubclassOf<UDMMaterialStageExpression> InExpressionClass)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	if (!ensure(IsValid(InExpressionClass)) || !ensure(InExpressionClass != UDMMaterialStageExpression::StaticClass()))
	{
		return nullptr;
	}

	// Extra transaction here because there are changes to the slot afterwards.
	FDMScopedUITransaction Transaction(LOCTEXT("AddLayerExpression", "Add Layer (Expression)"));
	InSlot->Modify();

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewStage);

	UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		NewStage, 
		InExpressionClass,
		UDMMaterialStageBlend::InputB, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_Blend(UDMMaterialSlot* InSlot, 
	TSubclassOf<UDMMaterialStageBlend> InBlendClass)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	if (!ensure(IsValid(InBlendClass)) || !ensure(InBlendClass != UDMMaterialStageBlend::StaticClass()))
	{
		return nullptr;
	}

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(InBlendClass);
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewStage);

	UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		NewStage, 
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageBlend::InputB, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
		0, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_Gradient(UDMMaterialSlot* InSlot, 
	TSubclassOf<UDMMaterialStageGradient> InGradientClass)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	if (!ensure(IsValid(InGradientClass)) || !ensure(InGradientClass != UDMMaterialStageGradient::StaticClass()))
	{
		return nullptr;
	}

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewStage);

	UDMMaterialStageInputGradient::ChangeStageInput_Gradient(
		NewStage, 
		InGradientClass, 
		UDMMaterialStageBlend::InputB, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_UV(UDMMaterialSlot* InSlot)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = InSlot->GetMaterialModelEditorOnlyData();

	if (!ensure(EditorOnlyData))
	{
		return nullptr;
	}

	UDynamicMaterialModel* MaterialModel = EditorOnlyData->GetMaterialModel();

	if (!ensure(MaterialModel))
	{
		return nullptr;
	}

	UDMMaterialStage* NewStage = UDMMaterialStageInputTextureUV::CreateStage(MaterialModel);

	return AddNewLayer(InSlot, NewStage);
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_MaterialFunction(UDMMaterialSlot* InSlot,
	UMaterialFunctionInterface* InFunction)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	if (!InFunction)
	{
		InFunction = UDMMaterialStageFunction::GetNoOpFunction();
	}

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewStage);

	UDMMaterialStageInputFunction::ChangeStageInput_Function(
		NewStage, 
		InFunction,
		UDMMaterialStageBlend::InputB, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL, 
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_SceneTexture(UDMMaterialSlot* InSlot)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	UDMMaterialStage* NewStage = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewStage);

	UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		NewStage, 
		UDMMaterialStageExpressionSceneTexture::StaticClass(),
		UDMMaterialStageBlend::InputB, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0, 
		FDMMaterialStageConnectorChannel::THREE_CHANNELS
	);

	if (UDMMaterialStage* MaskStage = NewLayer->GetStage(EDMMaterialLayerStage::Mask))
	{
		UDMMaterialStageInputExpression::ChangeStageInput_Expression(
			MaskStage, 
			UDMMaterialStageExpressionSceneTexture::StaticClass(),
			UDMMaterialStageThroughputLayerBlend::InputMaskSource, 
			FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
			0, 
			FDMMaterialStageConnectorChannel::FOURTH_CHANNEL
		);
	}

	return NewLayer;
}

UDMMaterialLayerObject* UDMMaterialSlotFunctionLibrary::AddNewLayer_Renderer(UDMMaterialSlot* InSlot, 
	TSubclassOf<UDMRenderTargetRenderer> InRendererClass)
{
	if (!ensure(IsValid(InSlot)))
	{
		return nullptr;
	}

	UDMMaterialStage* NewBase = UDMMaterialStageBlend::CreateStage(UDMMaterialStageBlendNormal::StaticClass());
	UDMMaterialLayerObject* NewLayer = AddNewLayer(InSlot, NewBase);

	UDMMaterialStageFunctionLibrary::SetStageInputToRenderer(NewBase, InRendererClass, UDMMaterialStageBlend::InputB);

	return NewLayer;
}

#undef LOCTEXT_NAMESPACE
