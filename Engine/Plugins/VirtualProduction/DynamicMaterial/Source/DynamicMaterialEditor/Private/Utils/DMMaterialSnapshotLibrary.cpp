// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialSnapshotLibrary.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DMAlphaOneMinusPS.h"
#include "DynamicMaterialEditorModule.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Materials/MaterialInterface.h"
#include "RenderGraphBuilder.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "RHI.h"
#include "SceneView.h"
#include "GlobalShader.h"

#define LOCTEXT_NAMESPACE "DMMaterialShapshotLibrary"

void FDMMaterialShapshotLibrary::RenderMaterialToRenderTarget(UMaterialInterface* InMaterial, UTextureRenderTarget2D* InRenderTarget)
{
	InMaterial->EnsureIsComplete();

	FTextureRenderTargetResource* RenderTargetResource = InRenderTarget->GameThread_GetRenderTargetResource();

	UCanvas* Canvas = NewObject<UCanvas>(GetTransientPackage());

	FCanvas RenderCanvas(
		RenderTargetResource,
		nullptr,
		FGameTime::CreateUndilated(0, 0),
		GEngine->GetDefaultWorldFeatureLevel()
	);

	Canvas->Init(InRenderTarget->SizeX, InRenderTarget->SizeY, nullptr, &RenderCanvas);

	{
		RHI_BREADCRUMB_EVENT_GAMETHREAD_F("DrawMaterialToRenderTarget", "DrawMaterialToRenderTarget: %s", InRenderTarget->GetFName());

		ENQUEUE_RENDER_COMMAND(FlushDeferredResourceUpdateCommand)(
			[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
			{
				RenderTargetResource->FlushDeferredResourceUpdate(RHICmdList);
			});

		Canvas->K2_DrawMaterial(InMaterial, FVector2D(0, 0), FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY), FVector2D(0, 0));

		RenderCanvas.Flush_GameThread();
		Canvas->Canvas = nullptr;

		//UpdateResourceImmediate must be called here to ensure mips are generated.
		InRenderTarget->UpdateResourceImmediate(false);

		ENQUEUE_RENDER_COMMAND(ResetSceneTextureExtentHistory)(
			[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
			{
				RenderTargetResource->ResetSceneTextureExtentsHistory();
			});
	}
}

UTextureRenderTarget2D* FDMMaterialShapshotLibrary::CreateSnapshotRenderTarget(const FIntPoint& InTextureSize)
{
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	check(RenderTarget);
	RenderTarget->RenderTargetFormat = RTF_RGBA32f;
	RenderTarget->ClearColor = FLinearColor::Black;
	RenderTarget->bAutoGenerateMips = false;
	RenderTarget->bCanCreateUAV = false;
	RenderTarget->InitAutoFormat(InTextureSize.X, InTextureSize.Y);
	RenderTarget->UpdateResourceImmediate(true);

	return RenderTarget;
}

void FDMMaterialShapshotLibrary::ApplyAlphaOneMinusShader(FRHICommandListImmediate& InRHICmdList, FTextureRenderTargetResource* InSourceTextureResource, FTextureRenderTargetResource* InDestTargetResource)
{
	FRDGBuilder GraphBuilder(InRHICmdList);

	const FTextureRHIRef& SourceTexture = InSourceTextureResource->GetRenderTargetTexture();
	const FTextureRHIRef& DestTarget = InDestTargetResource->GetRenderTargetTexture();

	FRDGTexture* InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTexture, TEXT("SourceTexture")));
	FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestTarget, TEXT("DestTarget")));
	const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	// The formats or sizes differ to pixel shader stuff
	// Configure source/output viewport to get the right UV scaling from source texture to output texture
	const FScreenPassTextureViewport InputViewport(InputTexture);
	const FScreenPassTextureViewport OutputViewport(OutputTexture);

	const TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

	// Rectangle area to use from source
	const FIntRect ViewRect(FIntPoint(0, 0), InputTexture->Desc.Extent);

	//Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
	const FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime()));
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = FMatrix::Identity;
	const FSceneView ViewInfo(ViewInitOptions);

	const TShaderMapRef<FDMAlphaOneMinusPS> PixelShader(GlobalShaderMap);
	FDMAlphaOneMinusPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(GraphBuilder, InputTexture, OutputTexture);
	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("ApplyAlphaOneMinusShader"), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);

	GraphBuilder.Execute();
}

UTextureRenderTarget2D* FDMMaterialShapshotLibrary::ApplyAlphaOneMinusShader(UTextureRenderTarget2D* InRenderTarget)
{
	FTextureRenderTargetResource* Source = InRenderTarget->GameThread_GetRenderTargetResource();

	UTextureRenderTarget2D* OutRenderTarget = CreateSnapshotRenderTarget({InRenderTarget->SizeX, InRenderTarget->SizeY});
	FTextureRenderTargetResource* Target = OutRenderTarget->GameThread_GetRenderTargetResource();

	ENQUEUE_RENDER_COMMAND(ApplyAlphaOneMinusCommand)(
		[Source, Target](FRHICommandListImmediate& InRHICmdList)
		{
			ApplyAlphaOneMinusShader(InRHICmdList, Source, Target);
		});

	return OutRenderTarget;
}

bool FDMMaterialShapshotLibrary::SnapshotMaterial(UMaterialInterface* InMaterial, const FIntPoint& InTextureSize, const FString& InSavePath)
{
	const FString PackageName = FPaths::GetBaseFilename(*InSavePath, false);
	const FString AssetName = FPaths::GetBaseFilename(*InSavePath, true);

	UTextureRenderTarget2D* RenderTarget = CreateSnapshotRenderTarget(InTextureSize);

	if (!RenderTarget)
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "Unable to create render target for material snapshot.");
		return false;
	}

	RenderMaterialToRenderTarget(InMaterial, RenderTarget);

	// Render target has inverted alpha - fix that.
	UTextureRenderTarget2D* FixedAlphaRenderTarget = ApplyAlphaOneMinusShader(RenderTarget);

	//FixedAlphaRenderTarget->ConstructTexture2D
	UTexture2D* Texture = FixedAlphaRenderTarget->ConstructTexture2D(GetTransientPackage(), AssetName, RF_Public | RF_Standalone);

	if (!Texture)
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "Unable to texture for material snapshot.");
		return false;
	}

	RenderTarget->ReleaseResource();
	FixedAlphaRenderTarget->ReleaseResource();

	// Create package and save texture.
	UPackage* Package = CreatePackage(*PackageName);

	if (!Package)
	{
		UE_LOGF(LogDynamicMaterialEditor, Warning, "Unable to create package for material snapshot.");
		return false;
	}

	Texture->Rename(*AssetName, Package, UE::DynamicMaterial::RenameFlags);

	FAssetRegistryModule::AssetCreated(Package);

	// Make a toast!
	const FString HyperLinkText = FPaths::ConvertRelativePathToFull(InSavePath);

	FNotificationInfo Info(LOCTEXT("SnapshotCreated", "Snapshot created."));

	Info.Hyperlink = FSimpleDelegate::CreateStatic([](FString SourceFilePath)
		{
			FPlatformProcess::ExploreFolder(*(FPaths::GetPath(SourceFilePath)));
		}, HyperLinkText);

	Info.HyperlinkText = FText::FromString(HyperLinkText);
	Info.ExpireDuration = 3.0f;

	FSlateNotificationManager::Get().AddNotification(Info);

	return true;
}

#undef LOCTEXT_NAMESPACE
