// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/MathFwd.h"

class UMaterialInterface;
class UTextureRenderTarget2D;
class FRHICommandListImmediate;
class FTextureRenderTargetResource;

class FDMMaterialShapshotLibrary
{
public:
	static bool SnapshotMaterial(UMaterialInterface* InMaterial, const FIntPoint& InTextureSize, const FString& InSavePath);

private:
	static void RenderMaterialToRenderTarget(UMaterialInterface* InMaterial, UTextureRenderTarget2D* InRenderTarget);

	static UTextureRenderTarget2D* CreateSnapshotRenderTarget(const FIntPoint& InTextureSize);

	static void ApplyAlphaOneMinusShader(FRHICommandListImmediate& InRHICmdList, FTextureRenderTargetResource* InSourceTextureResource,
		FTextureRenderTargetResource* InDestTargetResource);

	static UTextureRenderTarget2D* ApplyAlphaOneMinusShader(UTextureRenderTarget2D* InRenderTarget);
};
