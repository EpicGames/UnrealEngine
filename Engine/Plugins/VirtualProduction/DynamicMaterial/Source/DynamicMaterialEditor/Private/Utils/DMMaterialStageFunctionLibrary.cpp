// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMMaterialStageFunctionLibrary.h"

#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialStageThroughput.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/DMRenderTargetRenderer.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueRenderTarget.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMMaterialStageFunctionLibrary)

UDMMaterialStageInputValue* UDMMaterialStageFunctionLibrary::FindDefaultStageOpacityInputValue(UDMMaterialStage* InStage)
{
	if (!InStage)
	{
		return nullptr;
	}

	UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(InStage->GetSource());

	if (!Blend)
	{
		return nullptr;
	}

	return Blend->GetOpacityValue(InStage);
}

void UDMMaterialStageFunctionLibrary::SetStageInputToRenderer(UDMMaterialStage* InStage, TSubclassOf<UDMRenderTargetRenderer> InRendererClass, int32 InInputIndex)
{
	UDMMaterialStageInputExpression* InputExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		InStage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		InInputIndex,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::THREE_CHANNELS
	);

	if (!ensure(InputExpression))
	{
		return;
	}

	UDMMaterialSubStage* SubStage = InputExpression->GetSubStage();

	if (!ensure(SubStage))
	{
		return;
	}

	UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		SubStage,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		UDMMaterialValueRenderTarget::StaticClass(),
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (!ensure(InputValue))
	{
		return;
	}

	UDMMaterialValueRenderTarget* RenderTargetValue = Cast<UDMMaterialValueRenderTarget>(InputValue->GetValue());

	if (!RenderTargetValue)
	{
		return;
	}

	UDMRenderTargetRenderer::CreateRenderTargetRenderer(InRendererClass, RenderTargetValue);
}
