// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMPrivate.h"

#include "Components/DMMaterialLayer.h"
#include "DMDefs.h"
#include "DynamicMaterialEditorModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "MaterialExpressionIO.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FDMMaterialLayerReference"

namespace UE::DynamicMaterialEditor::Private
{
	void SetMask(FExpressionInput& InInputConnector, const FExpressionOutput& InOutputConnector, int32 InChannelOverride)
	{
		const bool bUseOutputMask = (InChannelOverride != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL);

		if (!bUseOutputMask)
		{
			InInputConnector.SetMask(InOutputConnector.Mask, InOutputConnector.MaskR, InOutputConnector.MaskG, InOutputConnector.MaskB, InOutputConnector.MaskA);
		}
		else
		{
			int32 MaskR = (!!(InChannelOverride & FDMMaterialStageConnectorChannel::FIRST_CHANNEL)) * (!!(InOutputConnector.MaskR));
			int32 MaskG = (!!(InChannelOverride & FDMMaterialStageConnectorChannel::SECOND_CHANNEL)) * (!!(InOutputConnector.MaskG));
			int32 MaskB = (!!(InChannelOverride & FDMMaterialStageConnectorChannel::THIRD_CHANNEL)) * (!!(InOutputConnector.MaskB));
			int32 MaskA = (!!(InChannelOverride & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL)) * (!!(InOutputConnector.MaskA));

			int32 MatchingMasks = MaskR + MaskG + MaskB + MaskA;

			if (MatchingMasks == 0)
			{
				InInputConnector.SetMask(InOutputConnector.Mask, InOutputConnector.MaskR, InOutputConnector.MaskG, InOutputConnector.MaskB, InOutputConnector.MaskA);
			}
			else
			{
				InInputConnector.SetMask(1, MaskR, MaskG, MaskB, MaskA);
			}
		}
	}

	int32 ChannelIndexToChannelBit(int32 InChannelIndex)
	{
		switch (InChannelIndex)
		{
			case 0:
				return FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;

			case 1:
				return FDMMaterialStageConnectorChannel::FIRST_CHANNEL;

			case 2:
				return FDMMaterialStageConnectorChannel::SECOND_CHANNEL;

			case 3:
				return FDMMaterialStageConnectorChannel::THIRD_CHANNEL;

			case 4:
				return FDMMaterialStageConnectorChannel::FOURTH_CHANNEL;

			default:
				checkNoEntry();
				return 0;
		}
	}

	int32 ChannelBitToChannelIndex(int32 InChannelBit)
	{
		switch (InChannelBit)
		{
			case FDMMaterialStageConnectorChannel::WHOLE_CHANNEL:
				return 0;

			case FDMMaterialStageConnectorChannel::FIRST_CHANNEL:
				return 1;

			case FDMMaterialStageConnectorChannel::SECOND_CHANNEL:
				return 2;

			case FDMMaterialStageConnectorChannel::THIRD_CHANNEL:
				return 3;

			case FDMMaterialStageConnectorChannel::FOURTH_CHANNEL:
				return 4;

			default:
				checkNoEntry();
				return 0;
		}
	}

	bool IsCustomMaterialProperty(EDMMaterialPropertyType InMaterialProperty)
	{
		return InMaterialProperty >= EDMMaterialPropertyType::Custom1 && InMaterialProperty <= EDMMaterialPropertyType::Custom4;
	}

	static bool bAllowUIFeedback = false;
	static FText LogErrorObjectFormat = LOCTEXT("LogErrorObjectFormat", "{0} (Source: {1})");

	void LogError(const FString& InMessage, bool bInToast, const UObject* InSource)
	{
		const FString Message = IsValid(InSource)
			? FString::Printf(TEXT("%s (Source: %s)"), *InMessage, *InSource->GetPathName())
			: InMessage;

		UE_LOGF(LogDynamicMaterialEditor, Error, "%ls", *Message);

		if (bAllowUIFeedback && bInToast)
		{
			const FText MessageText = IsValid(InSource)
				? FText::Format(LogErrorObjectFormat, FText::FromString(InMessage), FText::FromString(InSource->GetPathName()))
				: FText::FromString(InMessage);

			FNotificationInfo Info(MessageText);
			Info.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	FText GetMaterialPropertyLongDisplayName(EDMMaterialPropertyType InMaterialProperty)
	{
		return StaticEnum<EDMMaterialPropertyType>()->GetDisplayNameTextByValue(static_cast<int64>(InMaterialProperty));
	}

	FText GetMaterialPropertyShortDisplayName(EDMMaterialPropertyType InMaterialProperty)
	{
		constexpr const TCHAR* ShortNameName = TEXT("ShortName");

		UEnum* PropertyEnum = StaticEnum<EDMMaterialPropertyType>();

		const FString ShortName = PropertyEnum->GetMetaData(ShortNameName, PropertyEnum->GetIndexByValue(static_cast<int64>(InMaterialProperty)));

		if (!ShortName.IsEmpty())
		{
			return FText::FromString(ShortName);
		}

		return GetMaterialPropertyLongDisplayName(InMaterialProperty);
	}
}

FDMMaterialLayerReference::FDMMaterialLayerReference()
	: FDMMaterialLayerReference(nullptr)
{
}

FDMMaterialLayerReference::FDMMaterialLayerReference(UDMMaterialLayerObject* InLayer)
{
	LayerWeak = InLayer;
}

UDMMaterialLayerObject* FDMMaterialLayerReference::GetLayer() const
{
	return LayerWeak.Get();
}

bool FDMMaterialLayerReference::IsValid() const
{
	if (UDMMaterialLayerObject* Layer = GetLayer())
	{
		return Layer->FindIndex() != INDEX_NONE;
	}

	return false;
}

FDMScopedUITransaction::FDMScopedUITransaction(const FText& InSessionName, bool bInShouldActuallyTransact)
	: Transaction(FScopedTransaction(InSessionName, bInShouldActuallyTransact))
	, UIFeedbackGuard(TGuardValue<bool>(UE::DynamicMaterialEditor::Private::bAllowUIFeedback, true))
{
}

#undef LOCTEXT_NAMESPACE
