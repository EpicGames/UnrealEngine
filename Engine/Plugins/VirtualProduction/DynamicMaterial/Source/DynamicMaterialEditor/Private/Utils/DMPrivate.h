// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "ScopedTransaction.h"
#include "UI/Menus/DMMaterialSlotLayerMenus.h"
#include "UI/Widgets/Editor/SDMMaterialSlotEditor.h"
#include "UObject/WeakObjectPtr.h"

class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialStageInput;
enum class EDMMaterialPropertyType : uint8;
struct FExpressionInput;
struct FExpressionOutput;

namespace UE::DynamicMaterialEditor::Private
{
	struct FDMInputInputs
	{
		int32 InputIndex;
		TArray<UDMMaterialStageInput*> ChannelInputs;
	};

	void SetMask(FExpressionInput& InInputConnector, const FExpressionOutput& InOutputConnector, int32 InChannelOverride);

	/** Converts 0,1,2,3,4 to 0,1,2,4,8 */
	int32 ChannelIndexToChannelBit(int32 InChannelIndex);

	int32 ChannelBitToChannelIndex(int32 InChannelBit);

	bool IsCustomMaterialProperty(EDMMaterialPropertyType InMaterialProperty);

	void LogError(const FString& InMessage, bool bInToast = true, const UObject* InSource = nullptr);

	FText GetMaterialPropertyLongDisplayName(EDMMaterialPropertyType InMaterialProperty);

	FText GetMaterialPropertyShortDisplayName(EDMMaterialPropertyType InMaterialProperty);
}

struct FDMMaterialLayerReference
{
	TWeakObjectPtr<UDMMaterialLayerObject> LayerWeak;

	FDMMaterialLayerReference();
	FDMMaterialLayerReference(UDMMaterialLayerObject* InLayer);

	UDMMaterialLayerObject* GetLayer() const;

	bool IsValid() const;
};

struct FDMScopedUITransaction
{
	FScopedTransaction Transaction;
	TGuardValue<bool> UIFeedbackGuard;

	FDMScopedUITransaction(const FText& InSessionName, bool bInShouldActuallyTransact = true);
};
