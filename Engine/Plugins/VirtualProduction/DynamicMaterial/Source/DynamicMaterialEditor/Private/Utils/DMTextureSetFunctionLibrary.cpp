// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/DMTextureSetFunctionLibrary.h"

#include "DMMaterialTexture.h"
#include "DMTextureSet.h"
#include "DMTextureSetMaterialProperty.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialProperty.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStage.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "Engine/Texture.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#include "Utils/DMUtils.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSetFunctionLibrary)

#define LOCTEXT_NAMESPACE "DMTextureSetFunctionLibrary"

bool UDMTextureSetFunctionLibrary::AddTextureSetToModel(UDynamicMaterialModelEditorOnlyData* InEditorOnlyData, UDMTextureSet* InTextureSet,
	bool bInReplaceSlots)
{
	if (!InEditorOnlyData || !InTextureSet)
	{
		return false;
	}

	bool bMadeChange = false;

	TArray<UDMMaterialProperty*> InvalidProperties;

	for (const TPair<EDMTextureSetMaterialProperty, FDMMaterialTexture>& MaterialTexture : InTextureSet->GetTextures())
	{
		if (MaterialTexture.Value.Texture.IsNull())
		{
			continue;
		}

		const EDMMaterialPropertyType PropertyType = FDMUtils::TextureSetMaterialPropertyToMaterialPropertyType(MaterialTexture.Key);

		if (PropertyType == EDMMaterialPropertyType::None)
		{
			continue;
		}

		UDMMaterialProperty* Property = InEditorOnlyData->GetMaterialProperty(PropertyType);

		if (!Property)
		{
			continue;
		}

		if (!Property->IsEnabled())
		{
			Property->SetEnabled(true);
		}

		if (!Property->IsValidForModel(*InEditorOnlyData))
		{
			InvalidProperties.Add(Property);
		}

		UDMMaterialSlot* Slot = InEditorOnlyData->GetSlotForMaterialProperty(PropertyType);

		if (!Slot)
		{
			Slot = InEditorOnlyData->AddSlotForMaterialProperty(PropertyType);

			if (!Slot)
			{
				continue;
			}
		}

		UTexture* Texture = MaterialTexture.Value.Texture.LoadSynchronous();

		if (!Texture)
		{
			continue;
		}

		if (GUndo)
		{
			Slot->Modify();
		}

		UDMMaterialLayerObject* Layer = nullptr;

		{
			const FDMUpdateGuard Guard;

			Layer = Slot->AddDefaultLayer(PropertyType);

			if (!ensure(Layer))
			{
				continue;
			}

			bMadeChange = true;

			UDMMaterialStage* Stage = Layer->GetStage(EDMMaterialLayerStage::Base);

			if (!ensure(Stage))
			{
				continue;
			}

			UDMMaterialStageInputExpression* NewExpression = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
				Stage,
				UDMMaterialStageExpressionTextureSample::StaticClass(),
				UDMMaterialStageBlend::InputB,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
				0,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);

			if (!ensure(NewExpression))
			{
				continue;
			}

			UDMMaterialSubStage* SubStage = NewExpression->GetSubStage();

			if (!ensure(SubStage))
			{
				continue;
			}

			UDMMaterialStageInputValue* InputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
				SubStage,
				0,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
				EDMValueType::VT_Texture,
				FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
			);

			if (ensure(InputValue))
			{
				UDMMaterialValueTexture* InputTexture = Cast<UDMMaterialValueTexture>(InputValue->GetValue());

				if (ensure(InputTexture))
				{

					InputTexture->SetValue(Texture);
				}
			}

			if (UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(Stage->GetSource()))
			{
				EAvaColorChannel AvaColorChannel = EAvaColorChannel::None;

				if (EnumHasAnyFlags(MaterialTexture.Value.TextureChannel, EDMTextureChannelMask::Red))
				{
					AvaColorChannel |= EAvaColorChannel::Red;
				}

				if (EnumHasAnyFlags(MaterialTexture.Value.TextureChannel, EDMTextureChannelMask::Green))
				{
					AvaColorChannel |= EAvaColorChannel::Green;
				}

				if (EnumHasAnyFlags(MaterialTexture.Value.TextureChannel, EDMTextureChannelMask::Blue))
				{
					AvaColorChannel |= EAvaColorChannel::Blue;
				}

				if (EnumHasAnyFlags(MaterialTexture.Value.TextureChannel, EDMTextureChannelMask::Alpha))
				{
					AvaColorChannel |= EAvaColorChannel::Alpha;
				}

				if (AvaColorChannel != EAvaColorChannel::RGBA)
				{
					Blend->SetBaseChannelOverride(AvaColorChannel);
				}
			}

			if (bInReplaceSlots)
			{
				for (int32 Index = Slot->GetLayers().Num() - 1; Index >= 0; --Index)
				{
					UDMMaterialLayerObject* LayerIter = Slot->GetLayer(Index);

					if (!LayerIter || LayerIter->GetStage(EDMMaterialLayerStage::Base) == Stage)
					{
						continue;
					}

					Slot->RemoveLayer(LayerIter);
				}
			}
		}

		Layer->Update(Layer, EDMUpdateType::Structure);
	}

	/**
	 * This entire function should not be here. It should be something on the texture set.
	 * This warning should be there.
	 */
	if (!InvalidProperties.IsEmpty())
	{
		const FText WarningFormat = LOCTEXT(
			"AddTextureSetFormat",
			"The following channels contain textures but are not valid for the current material settings:\n\n{0}"""
		);

		const FText IndividualWarningFormat = LOCTEXT("AddTextureSetIndividualFormat", "- {0}\n");

		TArray<FText> ErrorStrings;
		ErrorStrings.Reserve(InvalidProperties.Num());

		for (UDMMaterialProperty* Property : InvalidProperties)
		{
			ErrorStrings.Add(FText::Format(IndividualWarningFormat, Property->GetDescription()));
		}

		const FText FullWarning = FText::Format(WarningFormat, FText::Join(FText::GetEmpty(), ErrorStrings));

		FNotificationInfo Info(FullWarning);
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	return bMadeChange;
}

#undef LOCTEXT_NAMESPACE
