// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"
#include "Utils/DMJsonUtils.h"
#include "Components/DMMaterialStage.h"
#include "DMDefs.h"
#include "DMMaterialEffect.generated.h"

class UDMMaterialEffect;
class UDMMaterialEffectStack;
class UDMMaterialLayerObject;
class UDMMaterialValue;
class UMaterialExpression;
struct FDMMaterialBuildState;

UENUM(BlueprintType)
enum class EDMMaterialEffectTarget : uint8
{
	None      = 0,
	BaseStage = 1 << 0,
	MaskStage = 1 << 1,
	TextureUV = 1 << 2,
	Slot      = 1 << 3
};

UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Effect"))
class UDMMaterialEffect : public UDMMaterialComponent, public IDMJsonSerializable
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API EDMMaterialEffectTarget StageTypeToEffectType(EDMMaterialLayerStage InStageType);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialEffect* CreateEffect(UDMMaterialEffectStack* InEffectStack, TSubclassOf<UDMMaterialEffect> InEffectClass);

	template<typename InEffectClass>
	static InEffectClass* CreateEffect(UDMMaterialEffectStack* InEffectStack)
	{
		return Cast<InEffectClass>(CreateEffect(InEffectStack, InEffectClass::StaticClass()));
	}

	DYNAMICMATERIALEDITOR_API UDMMaterialEffect();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialEffectStack* GetEffectStack() const;

	/** Retrieves the index of this effect in the effect stack. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API int32 FindIndex() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsEnabled() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetEnabled(bool bInIsEnabled);

	/** Returns the type of nodes which this effect targets. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialEffectTarget GetEffectTarget() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual FText GetEffectName() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual FText GetEffectDescription() const PURE_VIRTUAL(UDMMaterialEffect::ApplyTo, return FText::GetEmpty();)

	/** Test whether this effect is compatible with another effect. */
	virtual bool IsCompatibleWith(UDMMaterialEffect* InEffect) const { return true; }

	/** Apply this effect to the output of something, such as a stage, slot or texture. */
	virtual void ApplyTo(const TSharedRef<FDMMaterialBuildState>& InBuildState, TArray<UMaterialExpression*>& InOutExpressions, 
		int32& InOutLastExpressionOutputChannel, int32& InLastExpressionOutputIndex) const PURE_VIRTUAL(UDMMaterialEffect::ApplyTo)

	/** Returns the asset associated with this effect, if any. */
	virtual UObject* GetAsset() const PURE_VIRTUAL(UMaterialEffect::GetAsset, return nullptr;)

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetParentComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMMaterialEffectTarget EffectTarget;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bEnabled;
};
