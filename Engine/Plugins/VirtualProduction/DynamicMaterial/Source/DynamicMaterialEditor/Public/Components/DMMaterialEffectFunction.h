// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialEffect.h"
#include "DMMaterialEffectFunction.generated.h"

class UMaterialFunctionInterface;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", Meta = (DisplayName = "Material Designer Effect Function"))
class UDMMaterialEffectFunction : public UDMMaterialEffect
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString InputsPathToken;

	DYNAMICMATERIALEDITOR_API UDMMaterialEffectFunction();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UMaterialFunctionInterface* GetMaterialFunction() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetMaterialFunction(UMaterialFunctionInterface* InFunction);

	/** Returns the value used as the function input. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialValue* GetInputValue(int32 InIndex) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Get Input Values"))
	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialValue*> BP_GetInputValues() const;

	DYNAMICMATERIALEDITOR_API const TArray<TObjectPtr<UDMMaterialValue>>& GetInputValues() const;

#if WITH_EDITOR
	//~ Begin IDMJsonSerializable
	DYNAMICMATERIALEDITOR_API virtual TSharedPtr<FJsonValue> JsonSerialize() const override;
	DYNAMICMATERIALEDITOR_API virtual bool JsonDeserialize(const TSharedPtr<FJsonValue>& InJsonValue) override;
	//~ End IDMJsonSerializable
#endif

	//~ Begin UDMMaterialEffect
	DYNAMICMATERIALEDITOR_API virtual FText GetEffectName() const override;
	DYNAMICMATERIALEDITOR_API virtual FText GetEffectDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsCompatibleWith(UDMMaterialEffect* InEffect) const override;
	DYNAMICMATERIALEDITOR_API virtual void ApplyTo(const TSharedRef<FDMMaterialBuildState>& InBuildState, TArray<UMaterialExpression*>& InOutExpressions,
		int32& InOutLastExpressionOutputChannel, int32& InOutLastExpressionOutputIndex) const override;
	virtual UObject* GetAsset() const override;
	//~ End UDMMaterialEffect

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UMaterialFunctionInterface> MaterialFunctionPtr;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialValue>> InputValues;

	/** Re-inits the function. */
	void OnMaterialFunctionChanged();

	/** Adds required inputs. */
	void InitFunction();

	/** Removes inputs. */
	void DeinitFunction();

	/** Returns true if the input values do not match the function's input pins. */
	bool NeedsFunctionInit() const;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
};
