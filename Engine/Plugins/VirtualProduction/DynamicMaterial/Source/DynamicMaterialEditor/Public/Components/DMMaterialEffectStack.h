// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"
#include "DMDefs.h"
#include "Templates/SharedPointer.h"
#include "DMMaterialEffectStack.generated.h"

class FJsonValue;
class UDMMaterialEffect;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UMaterialExpression;
enum class EDMMaterialEffectTarget : uint8;
struct FDMMaterialBuildState;

struct FDMMaterialEffectJson
{
	TSubclassOf<UDMMaterialEffect> Class;
	TSharedPtr<FJsonValue> Data = nullptr;
};

USTRUCT(BlueprintType)
struct FDMMaterialEffectStackJson
{
	GENERATED_BODY()

	bool bEnabled = false;
	TArray<FDMMaterialEffectJson> Effects = {};
};

/**
 * Container for effects. Effects can be applied to either layers (on a per stage basis) or to slots.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", Meta = (DisplayName = "Material Designer Effect Stack"))
class UDMMaterialEffectStack : public UDMMaterialComponent
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString EffectsPathToken;

	using FEffectCallbackFunc = TFunctionRef<void(UDMMaterialEffect*)>;

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Create Effect Stack (For Slot)"))
	static UDMMaterialEffectStack* CreateEffectStackForSlot(UDMMaterialSlot* InSlot)
	{
		return CreateEffectStack(InSlot);
	}

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Create Effect Stack (For Layer)"))
	static UDMMaterialEffectStack* CreateEffectStackForLayer(UDMMaterialLayerObject* InLayer)
	{
		return CreateEffectStack(InLayer);
	}

	DYNAMICMATERIALEDITOR_API static UDMMaterialEffectStack* CreateEffectStack(UDMMaterialSlot* InSlot);
	DYNAMICMATERIALEDITOR_API static UDMMaterialEffectStack* CreateEffectStack(UDMMaterialLayerObject* InLayer);

	DYNAMICMATERIALEDITOR_API UDMMaterialEffectStack();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* GetSlot() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetLayer() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsEnabled() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetEnabled(bool bInIsEnabled);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialEffect* GetEffect(int32 InIndex) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Get Effects"))
	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialEffect*> BP_GetEffects() const;

	DYNAMICMATERIALEDITOR_API const TArray<TObjectPtr<UDMMaterialEffect>>& GetEffects() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasEffect(const UDMMaterialEffect* InEffect) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool AddEffect(UDMMaterialEffect* InEffect);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialEffect* SetEffect(int32 InIndex, UDMMaterialEffect* InEffect);

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Move Effect (By Index)"))
	bool BP_MoveEffectByIndex(int32 InIndex, int32 InNewIndex)
	{
		return MoveEffect(InIndex, InNewIndex);
	}

	DYNAMICMATERIALEDITOR_API bool MoveEffect(int32 InIndex, int32 InNewIndex);

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Move Effect (By Value)"))
	bool BP_MoveEffectByValue(UDMMaterialEffect* InEffect, int32 InNewIndex)
	{
		return MoveEffect(InEffect, InNewIndex);
	}

	DYNAMICMATERIALEDITOR_API bool MoveEffect(UDMMaterialEffect* InEffect, int32 InNewIndex);

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Remove Effect (By Index)"))
	UDMMaterialEffect* BP_RemoveEffectByIndex(int32 InIndex)
	{
		return RemoveEffect(InIndex);
	}

	DYNAMICMATERIALEDITOR_API UDMMaterialEffect* RemoveEffect(int32 InIndex);

	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName = "Remove Effect (By Value)"))
	bool BP_RemoveEffectByValue(UDMMaterialEffect* InEffect)
	{
		return RemoveEffect(InEffect);
	}

	DYNAMICMATERIALEDITOR_API bool RemoveEffect(UDMMaterialEffect* InEffect);

	/** Apply all matching effect types to the expressions and add them to the array. */
	DYNAMICMATERIALEDITOR_API bool ApplyEffects(const TSharedRef<FDMMaterialBuildState>& InBuildState, EDMMaterialEffectTarget InEffectTarget,
		TArray<UMaterialExpression*>& InOutStageExpressions, int32& InOutLastExpressionOutputChannel, int32& InOutLastExpressionOutputIndex) const;

	/** Creates a preset based on the current stage. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API FDMMaterialEffectStackJson CreatePreset();

	/** Apply the given preset to this stack. Does not remove old effects. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void ApplyPreset(const FDMMaterialEffectStackJson& InPreset);

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetParentComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bEnabled;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialEffect>> Effects;

	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialEffect*> GetIncompatibleEffects(UDMMaterialEffect* InEffect);

	/** Returns the removed effects. */
	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialEffect*> RemoveIncompatibleEffects(UDMMaterialEffect* InEffect);

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
};
