// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"

#include "Components/DMMaterialStage.h"
#include "DMDefs.h"

#include "DMMaterialLayer.generated.h"

class UDMMaterialEffectStack;
class UDMMaterialSlot;
class UDMMaterialStage;
class UMaterialExpression;

/** A collection of stages. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", Meta = (DisplayName = "Material Designer Layer"))
class UDMMaterialLayerObject : public UDMMaterialComponent
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString StagesPathToken;
	DYNAMICMATERIALEDITOR_API static const FString BasePathToken;
	DYNAMICMATERIALEDITOR_API static const FString MaskPathToken;
	DYNAMICMATERIALEDITOR_API static const FString EffectStackPathToken;

	using FStageCallbackFunc = TFunctionRef<void(UDMMaterialStage*)>;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* CreateLayer(UDMMaterialSlot* InSlot, EDMMaterialPropertyType InMaterialProperty,
		const TArray<UDMMaterialStage*>& InStages);
	
	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* DeserializeFromString(UDMMaterialSlot* InOuter, const FString& InSerializedString);

	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* GetSlot() const;

	/** Find the index of this layer in the slot. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API int32 FindIndex() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	const FText& GetLayerName() const { return LayerName; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetLayerName(const FText& InName);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsEnabled() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetEnabled(bool bInIsEnabled);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialPropertyType GetMaterialProperty() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetMaterialProperty(EDMMaterialPropertyType InMaterialProperty);

	/** Texture UV Link means that all stages use the same Texture UV from the base stage, if available. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsTextureUVLinkEnabled() const;

	/** Texture UV Link means that all stages use the same Texture UV from the base stage, if available. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetTextureUVLinkEnabled(bool bInValue);

	/** Texture UV Link means that all stages use the same Texture UV from the base stage, if available. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool ToggleTextureUVLinkEnabled();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetPreviousLayer(EDMMaterialPropertyType InUsingProperty, EDMMaterialLayerStage InSearchFor) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetNextLayer(EDMMaterialPropertyType InUsingProperty, EDMMaterialLayerStage InSearchFor) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	bool CanMoveLayerAbove(UDMMaterialLayerObject* InLayer) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	bool CanMoveLayerBelow(UDMMaterialLayerObject* InLayer) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetStage(EDMMaterialLayerStage InStageType = EDMMaterialLayerStage::All, bool bInCheckEnabled = false) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialStage*> GetStages(EDMMaterialLayerStage InStageType = EDMMaterialLayerStage::All, bool bInCheckEnabled = false) const;

	DYNAMICMATERIALEDITOR_API const TArray<TObjectPtr<UDMMaterialStage>>& GetAllStages() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialLayerStage GetStageType(const UDMMaterialStage* InStage) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetFirstValidStage(EDMMaterialLayerStage InStageScope) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetLastValidStage(EDMMaterialLayerStage InStageScope) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasValidStage(const UDMMaterialStage* InStage) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasValidStageOfType(EDMMaterialLayerStage InStageScope = EDMMaterialLayerStage::All) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsStageEnabled(EDMMaterialLayerStage InStageScope = EDMMaterialLayerStage::All) const;

	/** Checks for the first enabled and valid stage. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetFirstEnabledStage(EDMMaterialLayerStage InStageScope) const;

	/** Checks for the last enabled and valid stage. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetLastEnabledStage(EDMMaterialLayerStage InStageScope) const;

	/** Replace the specified stage. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetStage(EDMMaterialLayerStage InStageType, UDMMaterialStage* InStage);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool AreAllStagesValid(EDMMaterialLayerStage InStageScope) const;

	/** Checks if both stages are enabled and valid */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool AreAllStagesEnabled(EDMMaterialLayerStage InStageScope) const;

	/** Iterate over all the valid stages, whether they are enabled or not. */
	DYNAMICMATERIALEDITOR_API void ForEachValidStage(EDMMaterialLayerStage InStageScope, FStageCallbackFunc InCallback) const;

	/** Iterate over only the enabled stages. */
	DYNAMICMATERIALEDITOR_API void ForEachEnabledStage(EDMMaterialLayerStage InStageScope, FStageCallbackFunc InCallback) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialEffectStack* GetEffectStack() const;

	/** Used for copy+pasting. */
	FString SerializeToString() const;

	DYNAMICMATERIALEDITOR_API void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/** Apply the effects from this layer's effect stack to the given expressions based on the type of stage. */
	DYNAMICMATERIALEDITOR_API bool ApplyEffects(const TSharedRef<FDMMaterialBuildState>& InBuildState, const UDMMaterialStage* InStage,
		TArray<UMaterialExpression*>& InOutStageExpressions, int32& InOutLastExpressionOutputChannel, int32& InOutLastExpressionOutputIndex) const;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetParentComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMMaterialPropertyType MaterialProperty = EDMMaterialPropertyType::None;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FText LayerName;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bEnabled;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialStage>> Stages;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialEffectStack> EffectStack;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bLinkedUVs;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
};
