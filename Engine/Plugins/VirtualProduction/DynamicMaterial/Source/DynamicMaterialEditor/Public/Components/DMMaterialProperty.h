// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"

#include "DMEDefs.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointerFwd.h"

#include "DMMaterialProperty.generated.h"

class FText;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStageInputExpression;
class UDynamicMaterialModel;
class UDynamicMaterialModelEditorOnlyData;
class UMaterial;
class UMaterialExpression;
class UMaterialFunctionInterface;
struct FDMMaterialBuildState;

/**
 * Base Color, Specular, Opacity, etc
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", Meta = (DisplayName = "Material Designer Property"))
class UDMMaterialProperty : public UDMMaterialComponent
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString ComponentsPathToken;

	/**
	 * Craetes a material property as a default subobject of the EditorOnlyData object.
	 * @param InModelEditorOnlyData The parent EditorOnlyData object.
	 * @param InMaterialProperty The property type of the material property.
	 * @param InSubObjName The subobject name.
	 * @return The newly created default subobject.
	 */
	DYNAMICMATERIALEDITOR_API static UDMMaterialProperty* CreateCustomMaterialPropertyDefaultSubobject(UDynamicMaterialModelEditorOnlyData* InModelEditorOnlyData,
		EDMMaterialPropertyType InMaterialProperty, const FName& InSubObjName);

	UDMMaterialProperty();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDynamicMaterialModelEditorOnlyData* GetMaterialModelEditorOnlyData() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMMaterialPropertyType GetMaterialProperty() const { return MaterialProperty; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsEnabled() const { return bEnabled; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	void SetEnabled(bool bInEnabled);

	/**
	 * The description of this property based on the EDMMaterialPropertyType enum.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API FText GetDescription() const;

	/**
	 * Whether this property represents a material pin. Only materials properties with a pin will
	 * be generated by default.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsMaterialPin() const;

	/**
	 * @return The value type of this property. Will be either VT_Float1, VT_Float3_RGB or VT_Float3_XYZ.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMValueType GetInputConnectorType() const { return InputConnectorType; }

	/**
	 * @return Get the expressions which connect to this property, mapped per channel.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FDMMaterialStageConnection& GetInputConnectionMap() const { return InputConnectionMap; }

	/**
	 * @return Get the expressions which connect to this property, mapped per channel.
	 */
	FDMMaterialStageConnection& GetInputConnectionMap() { return InputConnectionMap; }

	/**
	 * The output process is an optional material function which is applied in between the property and its inputs.
	 * @return The current output processor.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UMaterialFunctionInterface* GetOutputProcessor() const { return OutputProcessor; }

	/**
	 * The output process is an optional material function which is applied in between the property and its inputs.
	 * @param InFunction The new output processor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetOutputProcessor(UMaterialFunctionInterface* InFunction);

	/**
	 * Check whether the current model settings are valid for this property.
	 * @param InMaterialModel The model with the settings.
	 * @return True if it is valid.
	 */
	DYNAMICMATERIALEDITOR_API virtual bool IsValidForModel(const UDynamicMaterialModelEditorOnlyData& InEditorOnlyData) const;

	/**
	 * Empty the connection map and recalculate the input types.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual void ResetInputConnectionMap();

	/**
	 * Generates a default input node for this property if the layer stack is empty.
	 * @param InBuildState The current build state.
	 * @return The last in the chain of any expressions created (for direct connection to the property).
	 */
	DYNAMICMATERIALEDITOR_API virtual UMaterialExpression* GetDefaultInput(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/**
	 * @return The texture sample type to use for any texture samplers connected to this property.
	 */
	DYNAMICMATERIALEDITOR_API virtual TEnumAsByte<EMaterialSamplerType> GetTextureSamplerType() const;

	/**
	 * Triggered when a slot is added to this property. Should create useful, default layer stack.
	 * @param InSlot The slot that was created.
	 */
	virtual void OnSlotAdded(UDMMaterialSlot* InSlot);

	/**
	 * Adds the default base stage for a layer in this property.
	 */
	virtual void AddDefaultBaseStage(UDMMaterialLayerObject* InLayer);

	/**
	 * Adds the default mask stage for a layer in this property.
	 */
	virtual void AddDefaultMaskStage(UDMMaterialLayerObject* InLayer);

	/**
	 * Adds a component to the component list. Sets component state to added.
	 * @param InName The name of the component in the map.
	 * @param InComponent The component to add.
	 * @return The replaced component from the map.
	 */
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* AddComponent(FName InName, UDMMaterialComponent* InComponent);

	/**
	 * Checks to see if a component with this name exists.
	 * @param InName The name of the component.
	 * @return True if it exists, even if it isn't valid.
	 */
	DYNAMICMATERIALEDITOR_API virtual bool HasComponent(FName InName) const;

	/**
	 * Fetches a component from the component list.
	 * @param InName The name of the component.
	 * @return The component in the map with that name.
	 */
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetComponent(FName InName) const;

	template<typename InClassName>
	InClassName* GetTypedComponent(FName InName) const
	{
		return Cast<InClassName>(GetComponent(InName));
	}

	/**
	 * Removes the component from the component list. Sets component state to removed.
	 * @param InName The name of the component.
	 * @return The removed component.
	 */
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* RemoveComponent(FName InName);

	/**
	 * Generates the base expressions for this slot.
	 */
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/**
	 * Adds expressions to opacity properties based on color channel.
	 */
	DYNAMICMATERIALEDITOR_API static void GenerateOpacityExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState, UDMMaterialSlot* InFromSlot,
		EDMMaterialPropertyType InFromProperty, UMaterialExpression*& OutExpression, int32& OutOutputIndex, int32& OutOutputChannel);

	/**
	 * Adds the alpha multiplier.
	 */
	virtual void AddAlphaMultiplier(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/**
	 * Adds the output processor to the end of the current expression chain.
	 */
	virtual void AddOutputProcessor(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/**
	 * Adds nodes to the given material based on this property's generated expressions.
	 */
	virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial);

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	DYNAMICMATERIALEDITOR_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UDMMaterialComponent

	void LoadDeprecatedModelData(UDMMaterialProperty* InOldProperty);

protected:
	DYNAMICMATERIALEDITOR_API static UMaterialExpression* CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState, float InDefaultValue);
	DYNAMICMATERIALEDITOR_API static UMaterialExpression* CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState, const FVector2d& InDefaultValue);
	DYNAMICMATERIALEDITOR_API static UMaterialExpression* CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState, const FVector3d& InDefaultValue);
	DYNAMICMATERIALEDITOR_API static UMaterialExpression* CreateConstant(const TSharedRef<FDMMaterialBuildState>& InBuildState, const FVector4d& InDefaultValue);

	/**
	 * The property type of this property.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMMaterialPropertyType MaterialProperty;

	/**
	 * Whether this property is enabled. If it is not enabled, it will generate no expressions.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bEnabled;

	/**
	 * The value type used to connect to this property. Will be either VT_Float1, VT_Float3_RGB or VT_Float3_XYZ.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMValueType InputConnectorType;

	/**
	 * The map of expressions connected to this property's input node.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FDMMaterialStageConnection InputConnectionMap;

	/**
	 * An optional material function which is applied in between the property and its inputs.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TObjectPtr<UMaterialFunctionInterface> OutputProcessor;

	UPROPERTY(Transient, TextExportTransient)
	TObjectPtr<UMaterialFunctionInterface> OutputProcessor_PreUpdate;

	/**
	 * Components of this property. Not necessarily owned or controlled by this property.
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TMap<FName, TObjectPtr<UDMMaterialComponent>> Components;

	DYNAMICMATERIALEDITOR_API UDMMaterialProperty(EDMMaterialPropertyType InMaterialProperty, EDMValueType InInputConnectorType);

	/**
	 * Triggered when the output process is updated either through PostEditChangeProperty or via the setter.
	 * Will potentially reset the value to null if it is invalid.
	 */
	void OnOutputProcessorUpdated();

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
};
