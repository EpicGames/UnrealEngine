// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"

#include "DMEDefs.h"

#include "DMMaterialSlot.generated.h"

class FScopedTransaction;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDynamicMaterialModelEditorOnlyData;
class UMaterial;
struct FDMMaterialBuildState;
struct FDMMaterialLayer;

DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnMaterialSlotConnectorsUpdated, UDMMaterialSlot*);
DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnMaterialSlotPropertiesUpdated, UDMMaterialSlot*);
DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnMaterialSlotLayersUpdated, UDMMaterialSlot*);

/**
 * A list of operations/inputs daisy chained together to produce an output.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", Meta = (DisplayName = "Material Designer Slot"))
class UDMMaterialSlot : public UDMMaterialComponent
{
	friend class SDMMaterialSlot;
	friend class SDMMaterialSlotEditor;

	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString LayersPathToken;

	DYNAMICMATERIALEDITOR_API UDMMaterialSlot();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDynamicMaterialModelEditorOnlyData* GetMaterialModelEditorOnlyData() const;

	/** Returns the index of this slot in the model. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	int32 GetIndex() const { return Index; }

	void SetIndex(int32 InNewIndex) { Index = InNewIndex; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API FText GetDescription() const;

	/** Returns the output types for the last layer with the given material property. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API const TArray<EDMValueType>& GetOutputConnectorTypesForMaterialProperty(EDMMaterialPropertyType InMaterialProperty) const;

	/** Returns all possible output connector types. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TSet<EDMValueType> GetAllOutputConnectorTypes() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer", meta = (DisplayName = "Get Layer"))
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetLayer(int32 InLayerIndex) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* FindLayer(const UDMMaterialStage* InBaseOrMask) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer", meta = (DisplayName = "Get Layers"))
	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialLayerObject*> BP_GetLayers() const;

	const TArray<TObjectPtr<UDMMaterialLayerObject>>& GetLayers() const { return LayerObjects; }

	/** Adds the default layer type for this slot based on the given material property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* AddDefaultLayer(EDMMaterialPropertyType InMaterialProperty);

	/** Adds the default layer (with specified base) based on the given material property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* AddLayer(EDMMaterialPropertyType InMaterialProperty, UDMMaterialStage* InNewBase);

	/** Adds a new layer with the specified base and mask layers. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* AddLayerWithMask(EDMMaterialPropertyType InMaterialProperty, UDMMaterialStage* InNewBase,
		UDMMaterialStage* InNewMask);

	/** Adds the specified layer to the end of the layer list. */
	bool PasteLayer(UDMMaterialLayerObject* InLayer);

	/** Can't be removed if it is the last remaining layer. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool CanRemoveLayer(UDMMaterialLayerObject* InLayer) const;

	/** Removes the layer, if possible. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool RemoveLayer(UDMMaterialLayerObject* InLayer);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool MoveLayer(UDMMaterialLayerObject* InLayer, int32 InNewIndex);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool MoveLayerBefore(UDMMaterialLayerObject* InLayer, UDMMaterialLayerObject* InBeforeLayer = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool MoveLayerAfter(UDMMaterialLayerObject* InLayer, UDMMaterialLayerObject* InAfterLayer = nullptr);

	/** Useful for determining output types. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetLastLayerForMaterialProperty(EDMMaterialPropertyType InMaterialProperty) const;

	void UpdateOutputConnectorTypes();

	void UpdateMaterialProperties();

	/** Called when the output connectors for this slot change. */
	FDMOnMaterialSlotConnectorsUpdated::RegistrationType& GetOnConnectorsUpdateDelegate() { return OnConnectorsUpdateDelegate; }

	/** Called when properties of this slot change. */
	FDMOnMaterialSlotPropertiesUpdated::RegistrationType& GetOnPropertiesUpdateDelegate() { return OnPropertiesUpdateDelegate; }

	/** Called whenever the properties of a layer change or when one is added, removed or moved. */
	FDMOnMaterialSlotLayersUpdated::RegistrationType& GetOnLayersUpdateDelegate() { return OnLayersUpdateDelegate; }

	/** Calls OnPropertiesUpdateDelegate when the property for this slot is updated. */
	void OnPropertiesUpdated();

	DYNAMICMATERIALEDITOR_API void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/** Return a map of the slots referencing this slot and how many times that reference exists. */
	const TMap<TWeakObjectPtr<UDMMaterialSlot>, int32>& GetSlotsReferencedBy() const { return SlotsReferencedBy; }

	/** Returns an array of the slots referencing this slot. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer", Meta = (DisplayName="Get Slots Referenced By"))
	TArray<UDMMaterialSlot*> K2_GetSlotsReferencedBy() const;

	/** Returns true if a new associated is created */
	bool ReferencedBySlot(UDMMaterialSlot* InOtherSlot);

	/** Returns true if all associations have been removed */
	bool UnreferencedBySlot(UDMMaterialSlot* InOtherSlot);

	/** Sets the material property of the given layer and changes all other layers matching that property to a different one. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetLayerMaterialPropertyAndReplaceOthers(UDMMaterialLayerObject* InLayer, EDMMaterialPropertyType InPropertyFrom,
		EDMMaterialPropertyType InPropertyTo);

	/** Changes the material property of all matching layers to another. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool ChangeMaterialProperty(EDMMaterialPropertyType InPropertyFrom, EDMMaterialPropertyType InPropertyTo);

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	virtual UDMMaterialComponent* GetParentComponent() const override { return nullptr; }
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 Index;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialLayerObject>> LayerObjects;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TMap<EDMMaterialPropertyType, FDMMaterialSlotOutputConnectorTypes> OutputConnectorTypes;

	UPROPERTY()
	TMap<TWeakObjectPtr<UDMMaterialSlot>, int32> SlotsReferencedBy;

	FDMOnMaterialSlotConnectorsUpdated OnConnectorsUpdateDelegate;
	FDMOnMaterialSlotPropertiesUpdated OnPropertiesUpdateDelegate;
	FDMOnMaterialSlotLayersUpdated OnLayersUpdateDelegate;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent

private:
	UE_DEPRECATED(5.4, "Promoted to full UObjects.")
	UPROPERTY()
	TArray<FDMMaterialLayer> Layers;

	void ConvertDeprecatedLayers(TArray<FDMMaterialLayer>& InLayers);
};
