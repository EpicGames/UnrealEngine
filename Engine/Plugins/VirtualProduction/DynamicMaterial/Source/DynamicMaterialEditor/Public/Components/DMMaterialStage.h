// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"

#include "Components/DMMaterialStageSource.h"
#include "DMEDefs.h"
#include "Templates/SubclassOf.h"

#include "DMMaterialStage.generated.h"

class FAssetThumbnailPool;
class FMenuBuilder;
class SWidget;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialStageBlend;
class UDMMaterialStageExpression;
class UDMMaterialStageFunction;
class UDMMaterialStageGradient;
class UDMMaterialStageInput;
class UDMMaterialStageInputExpression;
class UDMMaterialStageInputFunction;
class UDMMaterialStageInputGradient;
class UDMMaterialStageInputSlot;
class UDMMaterialStageInputTextureUV;
class UDMMaterialStageInputValue;
class UDMMaterialStageSource;
class UDMMaterialStageThroughput;
class UDMMaterialValue;
class UDMTextureUV;
class UMaterial;
class UMaterialInstanceDynamic;
class UMaterialInterface;
enum class EDMMaterialLayerStage : uint8;
struct FDMMaterialBuildState;
struct FDMTextureUV;

/**
 * A component which wraps a source and its inputs.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage"))
class UDMMaterialStage : public UDMMaterialComponent
{
	GENERATED_BODY()

	friend class FDMThroughputPropertyRowGenerator;

public:
	DYNAMICMATERIALEDITOR_API static const FString SourcePathToken;
	DYNAMICMATERIALEDITOR_API static const FString InputsPathToken;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateMaterialStage(UDMMaterialLayerObject* InLayer = nullptr);

	DYNAMICMATERIALEDITOR_API UDMMaterialStage();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetLayer() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMMaterialStageSource* GetSource() const { return Source; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsEnabled() const { return bEnabled; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetEnabled(bool bInEnabled);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool CanChangeSource() const { return bCanChangeSource; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	void SetCanChangeSource(bool bInCanChangeSource) { bCanChangeSource = bInCanChangeSource; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetSource(UDMMaterialStageSource* InSource);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<UDMMaterialStageInput*>& GetInputs() const { return Inputs; }

	/** Determines what connects to what on this stage's Source. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<FDMMaterialStageConnection>& GetInputConnectionMap() const { return InputConnectionMap; }

	TArray<FDMMaterialStageConnection>& GetInputConnectionMap() { return InputConnectionMap; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMValueType GetSourceType(const FDMMaterialStageConnectorChannel& InChannel) const;

	/** Returns true if the given source's input is mapped to an input (or the previous stage). */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool IsInputMapped(int32 InputIndex) const;

	/**
	 * Returns true if the output of the previous stage can connect to this stage.
	 * It is now up to the user to sort this particular problem out because it would do more harm than good
	 * to force correctness in "transition states" while the user is changing settings.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual bool IsCompatibleWithPreviousStage(const UDMMaterialStage* InPreviousStage) const;

	/* @see IsCompatibleWithPreviousStage */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual bool IsCompatibleWithNextStage(const UDMMaterialStage* InNextStage) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void AddInput(UDMMaterialStageInput* InNewInput);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void RemoveInput(UDMMaterialStageInput* InInput);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void RemoveAllInputs();

	/** Called when one of the inputs triggers it's Update event. */
	virtual void InputUpdated(UDMMaterialStageInput* InInput, EDMUpdateType InUpdateType);

	/** Verifies the entire input connection map. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual void ResetInputConnectionMap();

	DYNAMICMATERIALEDITOR_API void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/** Get the last layer for each property type from the previous stages. */
	DYNAMICMATERIALEDITOR_API TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> GetPreviousStagesPropertyMap();

	/** Get the last layer for each property type from all stages. */
	DYNAMICMATERIALEDITOR_API TMap<EDMMaterialPropertyType, UDMMaterialLayerObject*> GetPropertyMap();
	 
	using FSourceInitFunctionPtr = TFunction<void(UDMMaterialStage*, UDMMaterialStageSource*)>;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	UDMMaterialStageSource* ChangeSource(TSubclassOf<UDMMaterialStageSource> InSourceClass)
	{
		return ChangeSource(InSourceClass, nullptr);
	}

	DYNAMICMATERIALEDITOR_API UDMMaterialStageSource* ChangeSource(TSubclassOf<UDMMaterialStageSource> InSourceClass, 
		FSourceInitFunctionPtr InPreInit);

	template<typename InSourceClass>
	InSourceClass* ChangeSource(FSourceInitFunctionPtr InPreInit = nullptr)
	{
		return Cast<InSourceClass>(ChangeSource(InSourceClass::StaticClass(), InPreInit));
	}

	template<typename InSourceClass>
	InSourceClass* ChangeSource(TSubclassOf<UDMMaterialStageSource> InSourceSubclass, FSourceInitFunctionPtr InPreInit = nullptr)
	{
		return Cast<InSourceClass>(ChangeSource(InSourceSubclass, InPreInit));
	}

	using FInputInitFunctionPtr = TFunction<void(UDMMaterialStage*, UDMMaterialStageInput*)>;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	UDMMaterialStageInput* ChangeInput(TSubclassOf<UDMMaterialStageInput> InInputClass, int32 InInputIdx, int32 InInputChannel,
		int32 InOutputIdx, int32 InOutputChannel)
	{
		return ChangeInput(InInputClass, InInputIdx, InInputChannel, InOutputIdx, InOutputChannel, nullptr);
	}

	/**
	 * Creates a new input value and maps it to a specific source input.
	 * @param InInputIdx Index of the source input.
	 * @param InInputChannel The channel of the input that the input connects to.
	 * @param InOutputIdx The output index of the new input.
	 * @param InOutputChannel The channel of the output to connect.
	 * @param InPreInit Called on the new input before initialisation.
	 */
	DYNAMICMATERIALEDITOR_API UDMMaterialStageInput* ChangeInput(TSubclassOf<UDMMaterialStageInput> InInputClass, int32 InInputIdx, int32 InInputChannel,
		int32 InOutputIdx, int32 InOutputChannel, FInputInitFunctionPtr InPreInit);

	template<typename InInputClass>
	InInputClass* ChangeInput(int32 InInputIdx, int32 InInputChannel, int32 InOutputIdx, int32 InOutputChannel, 
		FInputInitFunctionPtr InPreInit = nullptr)
	{
		return Cast<InInputClass>(ChangeInput(InInputClass::StaticClass(), InInputIdx, InInputChannel, InOutputIdx, InOutputChannel, 
			InPreInit));
	}

	template<typename InInputClass>
	InInputClass* ChangeInput(TSubclassOf<UDMMaterialStageInput> InInputSubclass, int32 InInputIdx, int32 InInputChannel, 
		int32 InOutputIdx, int32 InOutputChannel, FInputInitFunctionPtr InPreInit = nullptr)
	{
		return Cast<InInputClass>(ChangeInput(InInputSubclass, InInputIdx, InInputChannel, InOutputIdx, InOutputChannel,
			InPreInit));
	}

	/** Changes the input of the given input index to the output of the previous stage with the given material property. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStageSource* ChangeInput_PreviousStage(int32 InInputIdx, int32 InInputChannel, EDMMaterialPropertyType InPreviousStageProperty,
		int32 InOutputIdx, int32 InOutputChannel);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void RemoveUnusedInputs();

	/** Returns true if any changes were made */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool VerifyAllInputMaps();

	/** Returns true if any changes were made */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool VerifyInputMap(int32 InInputIdx);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void GeneratePreviewMaterial(UMaterial* InPreviewMaterial);

	DYNAMICMATERIALEDITOR_API const FDMMaterialStageConnectorChannel* FindInputChannel(UDMMaterialStageInput* InStageInput);

	/**
	 * Changes the input mapping.
	 * @param InInputIdx Index of the source input.
	 * @param InInputChannel The channel of the input that the input connects to.
	 * @param InOutputIdx The output index of the new input.
	 * @param InOutputChannel The channel of the output to connect.
	 * @param InStageProperty The property for previous stage connections.
	 */
	DYNAMICMATERIALEDITOR_API void UpdateInputMap(int32 InInputIdx, int32 InSourceIndex, int32 InInputChannel, int32 InOutputIdx, 
		int32 InOutputChannel, EDMMaterialPropertyType InStageProperty);

	/** Returns the index of this stage in the layer. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API int32 FindIndex() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetPreviousStage() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetNextStage() const;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetParentComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialStageSource> Source;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialStageInput>> Inputs;

	/** How our inputs connect to the inputs of this stage's source */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<FDMMaterialStageConnection> InputConnectionMap;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bEnabled;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bCanChangeSource;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	DYNAMICMATERIALEDITOR_API virtual void GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent
};
