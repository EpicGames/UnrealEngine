// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialStageThroughput.h"
#include "Model/DMMaterialBuildState.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "Utils/DMUtils.h"
#include "DMMaterialStageBlend.generated.h"

class FMenuBuilder;
class UDMMaterialLayerObject;
class UDMMaterialStage;
class UDMMaterialStageInput;
class UDMMaterialStageInputValue;
class UDMMaterialValueFloat1;
class UMaterial;
enum class EAvaColorChannel : uint8;
struct FDMMaterialBuildState;

/**
 * A node which represents a blend operation.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage Blend"))
class UDMMaterialStageBlend : public UDMMaterialStageThroughput
{
	GENERATED_BODY()

public:
	static constexpr int32 InputAlpha = 0;
	static constexpr int32 InputA = 1;
	static constexpr int32 InputB = 2;

	DYNAMICMATERIALEDITOR_API static UDMMaterialStage* CreateStage(TSubclassOf<UDMMaterialStageBlend> InMaterialStageBlendClass, UDMMaterialLayerObject* InLayer = nullptr);

	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableBlends();

	UDMMaterialStageBlend();

	DYNAMICMATERIALEDITOR_API UDMMaterialValueFloat1* GetInputAlpha() const;
	DYNAMICMATERIALEDITOR_API UDMMaterialStageInput* GetInputB() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EAvaColorChannel GetBaseChannelOverride() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetBaseChannelOverride(EAvaColorChannel InMaskChannel);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FText& GetBlendDescription() const;

	template<class InNodeType>
	static void CreateBlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
		UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
		UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
		TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel);

	/**
	 * When building the opacity (mask) base input, combines the previous layer's alpha output with this ones.
	 */
	virtual void BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
		UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
		UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
		TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const;

	UDMMaterialStageInputValue* GetOpacityValue(UDMMaterialStage* InStage) const;

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual bool CanInputAcceptType(int32 InputIndex, EDMValueType ValueType) const override;
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInput(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInputType(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsInputVisible(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual int32 ResolveInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InputIndex, 
		FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const override;
	virtual void OnPostInputAdded(int32 InInputIdx) override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual FText GetStageDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual bool SupportsLayerMaskTextureUVLink() const override;
	DYNAMICMATERIALEDITOR_API virtual FDMExpressionInput GetLayerMaskLinkTextureUVInputExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual void GetMaskAlphaBlendNode(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
		UMaterialExpression*& OutExpression, int32& OutOutputIndex, int32& OutOutputChannel) const override;
	DYNAMICMATERIALEDITOR_API virtual bool GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
		UMaterialExpression*& OutMaterialExpression, int32& OutputIndex) override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

	//~ Begin FNotifyHook
	DYNAMICMATERIALEDITOR_API virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, class FEditPropertyChain* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	static TArray<TStrongObjectPtr<UClass>> Blends;

	static void GenerateBlendList();

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Getter, Setter, BlueprintGetter = GetBaseChannelOverride,
		Category = "Material Designer", DisplayName = "Channel Mask",
		meta = (NotKeyframeable, ToolTip = "Changes the output channel of the base input."))
	mutable EAvaColorChannel BaseChannelOverride;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Getter, BlueprintGetter = GetBlendDescription,
		Category = "Material Designer")
	FText BlendDescription;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageBlend(const FText& InName, const FText& InDescription);

	/* Returns true if there are any outputs on the base input that have more than 1 channel. */
	bool CanUseBaseChannelOverride() const;

	/* Returns the first output on the base input that has more than 1 channel. */
	int32 GetDefaultBaseChannelOverrideOutputIndex() const;

	/* Returns true if the given base output supports more than 1 channel. */
	bool IsValidBaseChannelOverrideOutputIndex(int32 InIndex) const;

	/* Reads the current output setting from the input map. */
	void PullBaseChannelOverride() const;

	/* Takes the override setting and applies it to the input map. */
	void PushBaseChannelOverride();

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial) override;
	//~ End UDMMaterialStageThroughput
};

template<class InNodeType>
void UDMMaterialStageBlend::CreateBlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
	UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
	UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
	TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel)
{
	InNodeType* Operator = InBuildState->GetBuildUtils().CreateExpression<InNodeType>(UE_DM_NodeComment_Default);
	check(Operator);

	Operator->A.Expression = InBaseLayerOpacityExpression;
	Operator->A.OutputIndex = InBaseOutputIndex;
	Operator->A.Mask = 0;

	if (InBaseOutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		Operator->A.Mask = 1;
		Operator->A.MaskR = !!(InBaseOutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
		Operator->A.MaskG = !!(InBaseOutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
		Operator->A.MaskB = !!(InBaseOutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
		Operator->A.MaskA = !!(InBaseOutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);
	}

	Operator->B.Expression = InMyLayerOpacityExpression;
	Operator->B.OutputIndex = InMyOutputIndex;
	Operator->B.Mask = 0;

	if (InMyOutputChannel != FDMMaterialStageConnectorChannel::WHOLE_CHANNEL)
	{
		Operator->B.Mask = 1;
		Operator->B.MaskR = !!(InMyOutputChannel & FDMMaterialStageConnectorChannel::FIRST_CHANNEL);
		Operator->B.MaskG = !!(InMyOutputChannel & FDMMaterialStageConnectorChannel::SECOND_CHANNEL);
		Operator->B.MaskB = !!(InMyOutputChannel & FDMMaterialStageConnectorChannel::THIRD_CHANNEL);
		Operator->B.MaskA = !!(InMyOutputChannel & FDMMaterialStageConnectorChannel::FOURTH_CHANNEL);
	}

	OutAddedExpressions.Add(Operator);

	// If we have to combine, it will use the Max node's output info
	OutOutputIndex = 0;
	OutOutputChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL;
}
