// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlend.h"
#include "DMMaterialStageBlendFunction.generated.h"

class UMaterialExpression;
class UMaterialFunctionInterface;
struct FDMMaterialBuildState;

/** A blending stage based on a material function. */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = "Material Designer")
class UDMMaterialStageBlendFunction : public UDMMaterialStageBlend
{
	GENERATED_BODY()

public:
	UDMMaterialStageBlendFunction();

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual void ConnectOutputToInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InInputIdx,
		int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel) override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialStageSource

protected:
	UPROPERTY()
	TObjectPtr<UMaterialFunctionInterface> MaterialFunction;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageBlendFunction(const FText& InName, const FText& InDescription, UMaterialFunctionInterface* InMaterialFunction);

	/** Loads the function asset. */
	DYNAMICMATERIALEDITOR_API UDMMaterialStageBlendFunction(const FText& InName, const FText& InDescription, const FName& InFunctionName, const FString& InFunctionPath);
};
