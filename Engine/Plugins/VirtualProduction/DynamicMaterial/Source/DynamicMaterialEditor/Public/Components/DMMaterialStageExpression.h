// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialStageThroughput.h"
#include "UObject/StrongObjectPtr.h"
#include "DMMaterialStageExpression.generated.h"

class FMenuBuilder;
class SDMMaterialSlot;
class UDMMaterialLayerObject;
class UDMMaterialStage;
class UMaterialExpression;
struct FDMMaterialBuildState;

UENUM(BlueprintType)
enum class EDMExpressionMenu : uint8
{
	Camera,
	Decal,
	Geometry,
	Landscape,
	Math,
	Object,
	Other,
	Particle,
	Texture,
	Time,
	WorldSpace
};

/**
 * A node which directly represents a material expression (or function).
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage Expression"))
class UDMMaterialStageExpression : public UDMMaterialStageThroughput
{
	GENERATED_BODY()

public:
	static TSet<FName> BlockedMaterialExpressionClasses;

	DYNAMICMATERIALEDITOR_API static UDMMaterialStage* CreateStage(TSubclassOf<UDMMaterialStageExpression> InMaterialStageExpressionClass, 
		UDMMaterialLayerObject* InLayer = nullptr);

	/** Some expressions are either exported or not at random. Use this to find ones that aren't. */
	DYNAMICMATERIALEDITOR_API static TSubclassOf<UMaterialExpression> FindClass(FString InClassName);

	UDMMaterialStageExpression();

	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableSourceExpressions();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageExpression* ChangeStageSource_Expression(UDMMaterialStage* InStage,
		TSubclassOf<UDMMaterialStageExpression> InExpressionClass);

	template<typename InExpressionClass>
	static UDMMaterialStageExpression* ChangeStageSource_Expression(UDMMaterialStage* InStage)
	{
		return ChangeStageSource_Expression(InStage, InExpressionClass::StaticClass());
	}

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	TSubclassOf<UMaterialExpression> GetMaterialExpressionClass() const { return MaterialExpressionClass; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<EDMExpressionMenu>& GetMenus() const { return Menus; }

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialStageSource

protected:
	static TArray<TStrongObjectPtr<UClass>> SourceExpressions;

	static void GenerateExpressionList();

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TSubclassOf<UMaterialExpression> MaterialExpressionClass;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<EDMExpressionMenu> Menus;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageExpression(const FText& InName, TSubclassOf<UMaterialExpression> InClass);
};


/**
 * TODO:

 Import material functions (Engine/Content/Functions/Etc.)

AbsorptionMediumMaterialOutput
AtmosphericFogColor
BentNormalCustomOutput
BlackBody
BumpOffset
DBufferTexture
DDX
DDY
DecalDerivative
DecalLifetimeOpacity
DecalMipmapLevel
DepthFade
DepthOfFieldFunction
DeriveNormalZ
Desaturation
Distance
DistanceCullFade
CloudLayer
EyeAdaptation
EyeAdaptationInverse
Frac
Fresnel
FunctionInput
FunctionOutput
GIReplace
HairAttributes
HairColor
PrecomputedAOMask
LightmassReplace
MaterialProxyReplace
NaniteReplace
Noise
Normalize
PreSkinnedLocalBounds
ShadowReplace
PerInstanceFadeAmount
PerInstanceRandom
PerInstanceCustomData
PreSkinnedNormal
PreSkinnedPosition
RuntimeVirtualTextureOutput
RuntimeVirtualTextureReplace
RuntimeVirtualTextureSample
Saturate
SceneDepthWithoutWater --
ISceneTexelSize
SceneTexture
SingleLayerWaterMaterialOutput
ThinTranslucentMaterialOutput
Sobol
SpeedTree
SphereMask
SphericalParticleOpacity
TangentOutput
TemporalSobol
TextureObject
TextureProperty
AntialiasedTextureMask
TwoSidedSign
VectorNoise
VertexInterpolator
ViewProperty
DistanceToNearestSurface
DistanceFieldGradient
AtmosphericLightVector
AtmosphericLightColor
SkyAtmosphereLightIlluminance
SkyAtmosphereLightDirection
SkyAtmosphereViewLuminance
SkyLightEnvMapSample
MapARPassthroughCameraUV
SamplePhysicsField
GetLocal

 */
