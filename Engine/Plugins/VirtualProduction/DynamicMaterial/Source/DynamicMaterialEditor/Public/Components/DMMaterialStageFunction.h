// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageThroughput.h"
#include "DMMaterialStageFunction.generated.h"

class UDMMaterialLayerObject;
class UMaterialFunctionInterface;

/** Represents a material function which can be added directly to a stage. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageFunction : public UDMMaterialStageThroughput
{
	GENERATED_BODY()

public:
	static constexpr int32 InputPreviousStage = 0;

	DYNAMICMATERIALEDITOR_API static TSoftObjectPtr<UMaterialFunctionInterface> NoOp;

	DYNAMICMATERIALEDITOR_API static UDMMaterialStage* CreateStage(UDMMaterialLayerObject* InLayer = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageFunction* ChangeStageSource_Function(UDMMaterialStage* InStage,
		UMaterialFunctionInterface* InMaterialFunction);

	DYNAMICMATERIALEDITOR_API static UMaterialFunctionInterface* GetNoOpFunction();

	UDMMaterialStageFunction();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UMaterialFunctionInterface* GetMaterialFunction() const { return MaterialFunction.Get(); }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialValue* GetInputValue(int32 InIndex) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TArray<UDMMaterialValue*> GetInputValues() const;

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInput(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInputType(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsInputVisible(int32 InputIndex) const override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	//~ End UObject

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetMaterialFunction, Category = "Material Designer",
		meta = (DisplayThumbnail = true, AllowPrivateAccess = "true", HighPriority, NotKeyframeable, NoCreate))
	TObjectPtr<UMaterialFunctionInterface> MaterialFunction;

	UPROPERTY(Transient, TextExportTransient)
	TObjectPtr<UMaterialFunctionInterface> MaterialFunction_PreEdit;

	void OnMaterialFunctionChanged();

	void InitFunction();

	void DeinitFunction();

	bool NeedsFunctionInit() const;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	//~ End UDMMaterialComponent
};
