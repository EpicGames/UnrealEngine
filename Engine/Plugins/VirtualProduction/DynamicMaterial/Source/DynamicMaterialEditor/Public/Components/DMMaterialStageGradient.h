// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialStageThroughput.h"

#include "Math/Color.h"
#include "UObject/StrongObjectPtr.h"

#include "DMMaterialStageGradient.generated.h"

class FMenuBuilder;
class UDMMaterialLayerObject;
class UDMMaterialStageInput;
class UMaterialFunctionInterface;
struct FDMMaterialBuildState;

/**
 * A node which represents UV-based gradient.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage Gradient"))
class UDMMaterialStageGradient : public UDMMaterialStageThroughput
{
	GENERATED_BODY()

public:
	static constexpr int32 InputUV = 0;
	static constexpr int32 InputStart = 1;
	static constexpr int32 InputEnd = 2;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(TSubclassOf<UDMMaterialStageGradient> InMaterialStageGradientClass, UDMMaterialLayerObject* InLayer = nullptr);

	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableGradients();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageGradient* ChangeStageSource_Gradient(UDMMaterialStage* InStage,
		TSubclassOf<UDMMaterialStageGradient> InGradientClass);

	template<typename InGradientClass>
	static UDMMaterialStageGradient* ChangeStageSource_Gradient(UDMMaterialStage* InStage)
	{
		return ChangeStageSource_Gradient(InStage, InGradientClass::StaticClass());
	}

	UDMMaterialStageGradient();

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInputType(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const override;
	//~ End UDMMaterialStageThroughput
	
	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	virtual bool SupportsLayerMaskTextureUVLink() const override { return true; }
	virtual int32 GetLayerMaskTextureUVLinkInputIndex() const override { return 0; }
	//~ End UDMMaterialStageSource

protected:
	static TArray<TStrongObjectPtr<UClass>> Gradients;

	static void GenerateGradientList();

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Material Designer",
		meta = (DisplayThumbnail = true, AllowPrivateAccess = "true", NoCreate))
	TObjectPtr<UMaterialFunctionInterface> MaterialFunction;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageGradient(const FText& InName);

	/** Always set the function. Returns whether an update was called. */
	bool SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction);
};