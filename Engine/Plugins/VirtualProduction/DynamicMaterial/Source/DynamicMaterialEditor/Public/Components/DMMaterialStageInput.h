// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialStageSource.h"
#include "DMMaterialStageInput.generated.h"

class UDMMaterialSlot;
class UMaterial;
struct FDMMaterialStageConnectorChannel;

/**
 * A node which produces an output (e.g. Texture coordinate.)
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage Input"))
class UDMMaterialStageInput : public UDMMaterialStageSource
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString StageInputPrefixStr;

	virtual FText GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel)
		PURE_VIRTUAL(UDMMaterialStageInput::GetChannelDescription, return FText::GetEmpty();)
		
	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

protected:
	/** Updates the output connectors based on the type of input. */
	virtual void UpdateOutputConnectors() {}

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial) override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const override;
	//~ End UDMMaterialComponent
};
