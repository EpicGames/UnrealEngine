// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialComponent.h"
#include "DMEDefs.h"
#include "UObject/StrongObjectPtr.h"
#include "DMMaterialStageSource.generated.h"

class UDMMaterialSlot;
class UDMMaterialStageInputTextureUV;
class UMaterialExpression;
struct FDMMaterialBuildState;

/**
 * A node which produces an output.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage Source"))
class UDMMaterialStageSource : public UDMMaterialComponent
{
	GENERATED_BODY()

	friend class UDMMaterialStage;

public:
	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableSourceClasses();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetStage() const;

	/* Returns a description of the stage for which this is the source. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual FText GetStageDescription() const { return GetComponentDescription(); }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<FDMMaterialStageConnector>& GetOutputConnectors() const { return OutputConnectors; }

	/** For pre-multiplied alpha, this retrieves the node which should be multiplied against. */
	DYNAMICMATERIALEDITOR_API virtual void GetMaskAlphaBlendNode(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
		UMaterialExpression*& OutExpression, int32& OutOutputIndex, int32& OutOutputChannel) const;

	virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const
		PURE_VIRTUAL(UDMMaterialStageSource::GenerateExpressions)

	/**
	 * Adds non-expression based node properties, such as Clamp Texture.
	 * Is called after expressions are generated.
	 */
	virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& Expressions) const {}

	/** Generates a material representing just this node. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial);

	/** 
	 * Returns the output index (channel WHOLE_CHANNEL) if this expression has pre-masked outputs.
	 * Returns INDEX_NONE if it is not supported.
	 */
	DYNAMICMATERIALEDITOR_API virtual int32 GetInnateMaskOutput(int32 OutputIndex, int32 OutputChannels) const;

	/**
	 * Given an output index, may return an override for output channels on that output.
	 * E.g. The texture sample alpha output may override to FOURTH_CHANNEL.
	 * Returns INDEX_NONE with no override.
	 */
	virtual int32 GetOutputChannelOverride(int32 InOutputIndex) const { return INDEX_NONE; }

	/** Generates a preview material based on this source, as owned by the given stage. */
	DYNAMICMATERIALEDITOR_API virtual bool GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
		UMaterialExpression*& OutMaterialExpression, int32& OutputIndex);

	//~ Begin FNotifyHook
	DYNAMICMATERIALEDITOR_API virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, class FEditPropertyChain* InPropertyThatChanged) override;
	//~ End FNotifyHook

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetParentComponent() const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	//~ End UObject

protected:
	static TArray<TStrongObjectPtr<UClass>> SourceClasses;

	static void GenerateClassList();

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<FDMMaterialStageConnector> OutputConnectors;	

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	//~ End UDMMaterialComponent
};
