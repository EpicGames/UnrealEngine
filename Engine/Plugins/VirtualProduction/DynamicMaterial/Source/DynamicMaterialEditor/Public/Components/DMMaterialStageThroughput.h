// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialStageSource.h"
#include "UObject/StrongObjectPtr.h"
#include "DMMaterialStageThroughput.generated.h"

class FMenuBuilder;
class UDMMaterialStageInput;
class UMaterialExpression;
struct FDMExpressionInput;
struct FDMMaterialBuildState;

struct FDMExpressionInput
{
	TArray<UMaterialExpression*> OutputExpressions = {};
	int32 OutputIndex = INDEX_NONE;
	int32 OutputChannel = INDEX_NONE;

	bool IsValid() const
	{
		return OutputExpressions.IsEmpty() == false && OutputIndex != INDEX_NONE && OutputChannel != INDEX_NONE;
	}
};

/**
 * A node which take one or more inputs and produces an output (e.g. Multiply)
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Stage Throughput"))
class UDMMaterialStageThroughput : public UDMMaterialStageSource
{
	GENERATED_BODY()

public:
	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableThroughputs();

	DYNAMICMATERIALEDITOR_API UDMMaterialStageThroughput();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FText& GetDescription() const { return Name; }

	/** Returns true if input is required to successfully compile this node. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool IsInputRequired() const { return bInputRequired; }

	/** Returns true if this node's inputs can have their own inputs. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool AllowsNestedInputs() const { return bAllowNestedInputs; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<FDMMaterialStageConnector>& GetInputConnectors() const { return InputConnectors; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual bool CanInputAcceptType(int32 InThroughputInputIndex, EDMValueType InValueType) const;

	/**
	 * Whether the given output connector can connect to this node.
	 * @param bInCheckSingleFloat If the initial compatibility check fails, it will again check against a single float.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual bool CanInputConnectTo(int32 InThroughputInputIndex, const FDMMaterialStageConnector& InOutputConnector, 
		int32 InOutputChannel, bool bInCheckSingleFloat = false);

	/** Whether the value of the given input change. */
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInput(int32 InThroughputInputIndex) const;

	/** Whether you can change the type of the given input. */
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInputType(int32 InThroughputInputIndex) const;

	/** Whether this input will show up in the Material Designer editor. */
	DYNAMICMATERIALEDITOR_API virtual bool IsInputVisible(int32 InThroughputInputIndex) const;

	/**
	 * Connect the output of a node to the given input of this node.
	 * @param InExpressionInputIndex The input index of this node.
	 * @param InSourceExpression The node to take the input from.
	 * @param InSourceOutputIndex The output index of the source expression.
	 * @param InSourceOutputChannel The channel of the output (RGBA).
	 */
	DYNAMICMATERIALEDITOR_API virtual void ConnectOutputToInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex, 
		int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel);

	/** Returns true if the layer and mask can have their Texture UV linked. */
	virtual bool SupportsLayerMaskTextureUVLink() const { return false; }

	/** Returns the input index for the default implementation of the below method. */
	DYNAMICMATERIALEDITOR_API virtual int32 GetLayerMaskTextureUVLinkInputIndex() const;

	/** 
	 * Returns all the material nodes requires to create this node's Texture UV input. 
	 * If you override this method, you do not need to override GetLayerMaskTextureUVLinkInputIndex.
	 */
	DYNAMICMATERIALEDITOR_API virtual FDMExpressionInput GetLayerMaskLinkTextureUVInputExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/**
	 * Override this to redirect inputs to other nodes.
	 * Returns the first node in the array by default
	 * --> In [ ]-[ ]-[ ] Out -->
	 */
	DYNAMICMATERIALEDITOR_API virtual UMaterialExpression* GetExpressionForInput(const TArray<UMaterialExpression*>& InStageSourceExpressions, 
		int32 InThroughputInputIndex, int32 InExpressionInputIndex);

	/** When the node is instantiated, this method adds default input values based on type. */
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const;

	/**
	 * Generates (or retrieves) expressions that produce this input for the node.
	 * @return the actual output index of the material expression
	 */
	DYNAMICMATERIALEDITOR_API virtual int32 ResolveInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex, 
		FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const;

	/** If this is on a Mask stage and it is the UV input index, this method is used to retrieve the base stage's UV input. */
	DYNAMICMATERIALEDITOR_API virtual int32 ResolveLayerMaskTextureUVLinkInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
		int32 InThroughputInputIndex, FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const;

	/** This is called when the input value of this node's stage is updated. */
	virtual void OnInputUpdated(int32 InThroughputInputIndex, EDMUpdateType InUpdateType) {}

	virtual void OnPostInputAdded(int32 InInputIdx) {}

	//~ Begin UDMMaterialComponent
	virtual FText GetComponentDescription() const override { return GetDescription(); }
	//~ End UDMMaterialComponent

protected:
	static TArray<TStrongObjectPtr<UClass>> Throughputs;

	static void GenerateThroughputList();

	/** @See ResolveLayerMaskTextureUVLinkInput */
	DYNAMICMATERIALEDITOR_API static int32 ResolveLayerMaskTextureUVLinkInputImpl(const TSharedRef<FDMMaterialBuildState>& InBuildState, 
		const UDMMaterialStageSource* InStageSource, FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions);

	DYNAMICMATERIALEDITOR_API UDMMaterialStageThroughput(const FText& InName);

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FText Name;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bInputRequired;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bAllowNestedInputs;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<FDMMaterialStageConnector> InputConnectors;

	/** When the stage's source is changed, whether the given of input from the previous source should be kept. */
	DYNAMICMATERIALEDITOR_API virtual bool ShouldKeepInput(int32 InThroughputInputIndex);

	/** @See ConnectOutputToInput */
	DYNAMICMATERIALEDITOR_API void ConnectOutputToInput_Internal(const TSharedRef<FDMMaterialBuildState>& InBuildState, UMaterialExpression* InTargetExpression,
		int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel) const;

	/** Finds the input for the individual channel. @see ResolveInput */
	DYNAMICMATERIALEDITOR_API virtual int32 ResolveInputChannel(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InThroughputInputIndex,
		int32 InChannelIndex, FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const;

	/** Generates a material based on the output of just this node. */
	DYNAMICMATERIALEDITOR_API virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial);

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	//~ End UDMMaterialComponent
};
