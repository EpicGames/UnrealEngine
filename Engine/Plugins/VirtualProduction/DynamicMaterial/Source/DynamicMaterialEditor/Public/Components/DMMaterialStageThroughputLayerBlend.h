// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageThroughput.h"
#include "DMMaterialStageThroughputLayerBlend.generated.h"

class UDMMaterialLayerObject;
class UMaterial;
class UMaterialExpression;
enum class EAvaColorChannel : uint8;

/** Used as the source for mask stages. */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageThroughputLayerBlend : public UDMMaterialStageThroughput
{
	GENERATED_BODY()

public:
	static constexpr int32 InputPreviousLayer = 0;
	static constexpr int32 InputBaseStage = 1;
	static constexpr int32 InputMaskSource = 2;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(UDMMaterialLayerObject* InLayer = nullptr);

	DYNAMICMATERIALEDITOR_API UDMMaterialStageThroughputLayerBlend();

	/** Returns the input connected to the Mask input. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStageInput* GetInputMask() const;

	/** Filters the output of the mask input node with the given channel. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EAvaColorChannel GetMaskChannelOverride() const;

	/** Filters the output of the mask input node with the given channel. */
	UFUNCTION(BlueprintCallable, Category="Material Designer")
	DYNAMICMATERIALEDITOR_API void SetMaskChannelOverride(EAvaColorChannel InMaskChannel);

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	DYNAMICMATERIALEDITOR_API virtual bool IsPropertyVisible(FName InProperty) const override;
	//~ End UDMMaterialComponent

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual FText GetStageDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual bool GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
		UMaterialExpression*& OutMaterialExpression, int32& OutputIndex) override;
	//~ End UDMMaterialStageSource

	//~ Begin FNotifyHook
	DYNAMICMATERIALEDITOR_API virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, 
		class FEditPropertyChain* InPropertyThatChanged) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End FNotifyHook

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInput(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsInputVisible(int32 InputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual int32 ResolveInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InputIndex, 
		FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const override;
	DYNAMICMATERIALEDITOR_API virtual void ConnectOutputToInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InInputIdx, 
		int32 InExpressionInputIndex, UMaterialExpression* InSourceExpression, int32 InSourceOutputIndex, int32 InSourceOutputChannel) override;
	virtual void OnPostInputAdded(int32 InInputIdx) override;
	//~ End UDMMaterialStageThroughput

	/** Resolves the Mask input. */
	DYNAMICMATERIALEDITOR_API void GetMaskOutput(const TSharedRef<FDMMaterialBuildState>& InBuildState, UMaterialExpression*& OutExpression, int32& OutOutputIndex, int32& OutOutputChannel) const;

	/* When true, the base stage's output will be multiplied by this stage (darkening it where it is translucent). */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool UsePremultiplyAlpha() const { return bPremultiplyAlpha; }

	/* When true, the base stage's output will be multiplied by this stage (darkening it where it is translucent). */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetPremultiplyAlpha(bool bInValue);

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditImport() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	//~ End UObject

protected:
	/** Changes the output channel of the mask input. */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Getter=GetMaskChannelOverride, Setter=SetMaskChannelOverride, BlueprintGetter=GetMaskChannelOverride,
		Category = "Material Designer", DisplayName = "Channel Mask", meta=(NotKeyframeable))
	mutable EAvaColorChannel MaskChannelOverride;

	/** Additionally multiplies the output of the RGB Stage by the output from this Stage. */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer", meta=(NotKeyframeable))
	bool bPremultiplyAlpha;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer",
		meta=(NotKeyframeable))
	bool bIsAlphaOnlyBlend;

	/** When doing an update, this will prevent recursive calls. */
	bool bBlockUpdate;

	DYNAMICMATERIALEDITOR_API virtual int32 ResolveMaskInput(const TSharedRef<FDMMaterialBuildState>& InBuildState, int32 InputIndex, 
		FDMMaterialStageConnectorChannel& OutChannel, TArray<UMaterialExpression*>& OutExpressions) const;

	/** Updates the value of bIsAlphaOnlyBlend based on current conditions. */
	DYNAMICMATERIALEDITOR_API virtual void UpdateAlphaOnlyMaskStatus();

	/** Called when the owning stage is updated. */
	DYNAMICMATERIALEDITOR_API virtual void OnStageUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	/** Update this and all parent layer mask stages that are only-only (no based). */
	DYNAMICMATERIALEDITOR_API virtual void UpdateAlphaOnlyMasks(EDMUpdateType InUpdateType);

	DYNAMICMATERIALEDITOR_API void InitBlendStage();

	DYNAMICMATERIALEDITOR_API void GenerateMainExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;
	DYNAMICMATERIALEDITOR_API void GeneratePreviewExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const;

	/** If this shades any inputs with the base stage, update the base stage. */
	DYNAMICMATERIALEDITOR_API void UpdateLinkedInputStage(EDMUpdateType InUpdateType);

	/* Returns true if there are any outputs on the mask input that have more than 1 channel. */
	DYNAMICMATERIALEDITOR_API bool CanUseMaskChannelOverride() const;

	/* Returns the first output on the mask input that has more than 1 channel. */
	DYNAMICMATERIALEDITOR_API int32 GetDefaultMaskChannelOverrideOutputIndex() const;

	/* Returns true if the given mask output supports more than 1 channel. */
	DYNAMICMATERIALEDITOR_API bool IsValidMaskChannelOverrideOutputIndex(int32 InIndex) const;

	/* Reads the current output setting from the input map. */
	void PullMaskChannelOverride() const;

	/* Takes the override setting and applies it to the input map. */
	void PushMaskChannelOverride();

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial) override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialComponent
	virtual void OnComponentAdded() override;
	//~ End UDMMaterialComponent
};
