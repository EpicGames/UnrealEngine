// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMMaterialStage.h"
#include "DMMaterialSubStage.generated.h"

/**
 * A stage that is a subobject of another stage, such as when an input throughput has its own inputs.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer", meta = (DisplayName = "Material Designer Sub Stage"))
class UDMMaterialSubStage : public UDMMaterialStage
{
	GENERATED_BODY()

	friend class UDMMaterialStageInputThroughput;

public:
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialSubStage* CreateMaterialSubStage(UDMMaterialStage* InParentStage);

	/** Returns the parent stage of this stage, which is probably not its direct parent. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetParentStage() const;

	/** Recursively calls GetParentStage() to find the outer stage. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetParentMostStage() const;

	/** Sets which object directly owns this component in the hierarchy. */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetParentComponent(UDMMaterialComponent* InParentComponent);

	//~ Begin UDMMaterialStage
	DYNAMICMATERIALEDITOR_API virtual bool IsCompatibleWithPreviousStage(const UDMMaterialStage* PreviousStage) const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsCompatibleWithNextStage(const UDMMaterialStage* NextStage) const override;
	//~ End UDMMaterialStage

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual FString GetComponentPathComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetParentComponent() const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TObjectPtr<UDMMaterialStage> ParentStage;

	UPROPERTY()
	TObjectPtr<UDMMaterialComponent> ParentComponent;

	UDMMaterialSubStage()
		: ParentStage(nullptr)
		, ParentComponent(nullptr)
	{
	}
};
