// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlendFunction.h"
#include "DMMSBAdd.generated.h"

// The same as linear dodge
UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageBlendAdd : public UDMMaterialStageBlendFunction
{
	GENERATED_BODY()

public:
	UDMMaterialStageBlendAdd();

	//~ Begin UDMMaterialStageBlend
	virtual void BlendOpacityLayer(const TSharedRef<FDMMaterialBuildState>& InBuildState,
		UMaterialExpression* InBaseLayerOpacityExpression, int32 InBaseOutputIndex, int32 InBaseOutputChannel,
		UMaterialExpression* InMyLayerOpacityExpression, int32 InMyOutputIndex, int32 InMyOutputChannel,
		TArray<UMaterialExpression*>& OutAddedExpressions, int32& OutOutputIndex, int32& OutOutputChannel) const;
	//~ End UDMMaterialStageBlend
};
