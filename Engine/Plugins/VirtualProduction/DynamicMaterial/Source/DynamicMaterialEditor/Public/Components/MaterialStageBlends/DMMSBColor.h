// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlendFunction.h"
#include "DMMSBColor.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageBlendColor : public UDMMaterialStageBlendFunction
{
	GENERATED_BODY()

public:
	UDMMaterialStageBlendColor();
};
