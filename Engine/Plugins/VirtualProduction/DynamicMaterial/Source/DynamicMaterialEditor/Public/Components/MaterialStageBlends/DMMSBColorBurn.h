// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlendFunction.h"
#include "DMMSBColorBurn.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageBlendColorBurn : public UDMMaterialStageBlendFunction
{
	GENERATED_BODY()

public:
	UDMMaterialStageBlendColorBurn();
};
