// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlendFunction.h"
#include "DMMSBColorDodge.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageBlendColorDodge : public UDMMaterialStageBlendFunction
{
	GENERATED_BODY()

public:
	UDMMaterialStageBlendColorDodge();
};
