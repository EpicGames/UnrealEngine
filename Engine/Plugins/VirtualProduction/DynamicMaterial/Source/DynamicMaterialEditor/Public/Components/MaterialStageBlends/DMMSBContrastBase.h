// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlend.h"
#include "DMMSBContrastBase.generated.h"

class UMaterialExpression;
struct FDMMaterialBuildState;

UCLASS(MinimalAPI, BlueprintType, Abstract, ClassGroup = "Material Designer")
class UDMMaterialStageBlendContrastBase : public UDMMaterialStageBlend
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API UDMMaterialStageBlendContrastBase(const FText& InName, const FText& InDescription);

private:
	UDMMaterialStageBlendContrastBase();
};
