// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageBlendFunction.h"
#include "DMMSBDarkenColor.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageBlendDarkenColor : public UDMMaterialStageBlendFunction
{
	GENERATED_BODY()

public:
	UDMMaterialStageBlendDarkenColor();
};
