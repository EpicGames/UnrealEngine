// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"

#include "DMMSEMathBase.generated.h"

class UDMMaterialStageInput;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionMathBase : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API UDMMaterialStageExpressionMathBase(const FText& InName, TSubclassOf<UMaterialExpression> InClass);

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual bool CanInputAcceptType(int32 InInputIndex, EDMValueType InValueType) const override;
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual bool GenerateStagePreviewMaterial(UDMMaterialStage* InStage, UMaterial* InPreviewMaterial, 
		UMaterialExpression*& OutMaterialExpression, int32& OutputIndex) override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

protected:
	/** Whether this just works with Scalars. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bSingleChannelOnly;

	/** How many inputs this should automatically create. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 VariadicInputCount;

	/** Allow matching of single floats to match other types of floats. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bAllowSingleFloatMatch;

	UDMMaterialStageExpressionMathBase();

	/** Add automatic inputs (A-O (15))) */
	DYNAMICMATERIALEDITOR_API void SetupInputs(int32 InCount);

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual void GeneratePreviewMaterial(UMaterial* InPreviewMaterial) override;
	//~ End UDMMaterialStageThroughput
};
