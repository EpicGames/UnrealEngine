// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSETextureCoordinate.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionTextureCoordinate : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionTextureCoordinate();

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;
	//~ End UDMMaterialStageSource

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 CoordinateIndex;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	float UTiling;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	float VTiling;
};
