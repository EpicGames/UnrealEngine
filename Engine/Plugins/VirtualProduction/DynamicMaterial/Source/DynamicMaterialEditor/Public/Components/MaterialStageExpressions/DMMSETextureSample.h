// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageExpressions/DMMSETextureSampleBase.h"

#include "DMMSETextureSample.generated.h"

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionTextureSample : public UDMMaterialStageExpressionTextureSampleBase
{
	GENERATED_BODY()

	friend class SDMMaterialComponentEditor;

public:
	UDMMaterialStageExpressionTextureSample();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool GetUseBaseTexture() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetUseBaseTexture(bool bInUseBaseTexture);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool CanUseBaseTexture() const;

	DYNAMICMATERIALEDITOR_API const UDMMaterialStageExpressionTextureSample* GetBaseTextureSample() const;

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsPropertyVisible(FName InProperty) const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject

protected:
	/*
	 * When both the base and mask stages are set to a texture and toggle layer UV link is on, use the texture sampler from
	 * base stage directly in the mask stage, with no mask stage texture sampler.
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Getter = GetUseBaseTexture, Setter = SetUseBaseTexture, BlueprintGetter = GetUseBaseTexture,
		Category = "Material Designer", DisplayName = "Use Base Texture", meta = (NotKeyframeable))
	bool bUseBaseTexture;

	void OnUseBaseTextureChanged();
};
