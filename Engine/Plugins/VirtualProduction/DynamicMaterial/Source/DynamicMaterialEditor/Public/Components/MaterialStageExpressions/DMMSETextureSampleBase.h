// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "DMMSETextureSampleBase.generated.h"

UCLASS(minimalAPI, BlueprintType, Blueprintable, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionTextureSampleBase : public UDMMaterialStageExpression
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionTextureSampleBase();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual bool IsClampTextureEnabled() const { return bClampTexture; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual void SetClampTextureEnabled(bool bInValue);

	//~ Begin UDMMaterialStageExpression
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	//~ End UDMMaterialStageExpression

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual bool IsPropertyVisible(FName Property) const override;
	DYNAMICMATERIALEDITOR_API virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;
	DYNAMICMATERIALEDITOR_API virtual int32 GetInnateMaskOutput(int32 InOutputIndex, int32 InOutputChannels) const override;
	DYNAMICMATERIALEDITOR_API virtual int32 GetOutputChannelOverride(int32 InOutputIndex) const override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInputType(int32 InInputIndex) const override;
	virtual bool SupportsLayerMaskTextureUVLink() const override { return true; }
	virtual int32 GetLayerMaskTextureUVLinkInputIndex() const override { return 1; }
	DYNAMICMATERIALEDITOR_API virtual void OnInputUpdated(int32 InInputIndex, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialComponent
	virtual FText GetComponentDescription() const override;
	virtual FSlateIcon GetComponentIcon() const override;
	//~ End UDMMaterialComponent;

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter = SetClampTextureEnabled, BlueprintSetter = SetClampTextureEnabled, Category = "Material Designer",
		DisplayName = "Clamp Texture UV", meta = (ToolTip = "Forces a material rebuild.", LowPriority, NotKeyframeable, AllowPrivateAccess = "true"))
	bool bClampTexture;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageExpressionTextureSampleBase(const FText& InName, TSubclassOf<UMaterialExpression> InClass);

	void UpdateMask();
};
