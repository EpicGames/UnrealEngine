// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageExpression.h"
#include "IDMParameterContainer.h"
#include "DMMSETextureSampleEdgeColor.generated.h"

UENUM(BlueprintType)
enum class EDMEdgeLocation : uint8
{
	TopLeft,
	Top,
	TopRight,
	Left,
	Center,
	Right,
	BottomLeft,
	Bottom,
	BottomRight,
	Custom
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageExpressionTextureSampleEdgeColor : public UDMMaterialStageExpression, public IDMParameterContainer
{
	GENERATED_BODY()

public:
	UDMMaterialStageExpressionTextureSampleEdgeColor();

	//~ Begin UDMMaterialStageThroughput
	DYNAMICMATERIALEDITOR_API virtual bool CanChangeInputType(int32 InInputIndex) const override;
	DYNAMICMATERIALEDITOR_API virtual void AddDefaultInput(int32 InInputIndex) const override;
	//~ End UDMMaterialStageThroughput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void AddExpressionProperties(const TArray<UMaterialExpression*>& InExpressions) const override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void Update(UDMMaterialComponent* InSource, EDMUpdateType InUpdateType) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PreEditChange(FEditPropertyChain& InPropertyAboutToChange) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

	//~ Begin IDMParameterContainer
	virtual void CopyParametersFrom_Implementation(UObject* InOther) override;
	//~ End IDMParameterContainer

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer", meta=(AllowPrivateAccess = "true"))
	EDMEdgeLocation EdgeLocation;

	EDMEdgeLocation PreEditEdgeLocation;

	void OnEdgeLocationChanged();
};
