// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageGradient.h"
#include "DMMSGLinear.generated.h"

struct FDMMaterialBuildState;
class UMaterialExpression;

UENUM(BlueprintType)
enum class ELinearGradientTileType : uint8
{
	NoTile,
	Tile,
	TileAndMirror
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageGradientLinear : public UDMMaterialStageGradient
{
	GENERATED_BODY()

public:
	UDMMaterialStageGradientLinear();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	virtual ELinearGradientTileType GetTilingType() const { return Tiling; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API virtual void SetTilingType(ELinearGradientTileType InType);

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	//~ End UObject

protected:
	static TSoftObjectPtr<UMaterialFunctionInterface> LinearGradientNoTileFunction;
	static TSoftObjectPtr<UMaterialFunctionInterface> LinearGradientTileFunction;
	static TSoftObjectPtr<UMaterialFunctionInterface> LinearGradientTileAndMirrorFunction;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = GetTilingType, Setter=SetTilingType, BlueprintSetter = SetTilingType, 
		Category = "Material Designer")
	ELinearGradientTileType Tiling;

	UMaterialFunctionInterface* GetMaterialFunctionForTilingType(ELinearGradientTileType) const;

	void OnTilingChanged();
};
