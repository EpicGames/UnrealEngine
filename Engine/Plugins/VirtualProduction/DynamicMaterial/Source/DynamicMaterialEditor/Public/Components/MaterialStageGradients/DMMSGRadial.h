// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageGradient.h"
#include "DMMSGRadial.generated.h"

struct FDMMaterialBuildState;
class UMaterialExpression;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageGradientRadial : public UDMMaterialStageGradient
{
	GENERATED_BODY()

	UDMMaterialStageGradientRadial();

public:
	static TSoftObjectPtr<UMaterialFunctionInterface> RadialGradientFunction;
};
