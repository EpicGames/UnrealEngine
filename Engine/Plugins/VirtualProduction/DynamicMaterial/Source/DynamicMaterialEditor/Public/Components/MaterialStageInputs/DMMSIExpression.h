// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Templates/SubclassOf.h"
#include "UObject/StrongObjectPtr.h"
#include "DMMSIExpression.generated.h"

class UDMMaterialLayerObject;
class UDMMaterialStageExpression;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputExpression : public UDMMaterialStageInputThroughput
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(TSubclassOf<UDMMaterialStageExpression> InMaterialStageExpressionClass, UDMMaterialLayerObject* InLayer = nullptr);

	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableInputExpressions();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputExpression* ChangeStageSource_Expression(UDMMaterialStage* InStage,
		TSubclassOf<UDMMaterialStageExpression> InExpressionClass);

	template<typename InExpressionClass>
	static UDMMaterialStageInputExpression* ChangeStageSource_Expression(UDMMaterialStage* InStage)
	{
		return ChangeStageSource_Expression(InStage, InExpressionClass::StaticClass());
	}

	/**
	 * Change the input type of an input on a stage to an expression.
	 * @param InInputIdx Index of the source input.
	 * @param InInputChannel The channel of the input that the input connects to.
	 * @param InOutputIdx The output index of the new input.
	 * @param InOutputChannel The channel of the output to connect.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputExpression* ChangeStageInput_Expression(UDMMaterialStage* InStage,
		TSubclassOf<UDMMaterialStageExpression> InExpressionClass, int32 InInputIdx, int32 InInputChannel, int32 InOutputIdx,
		int32 InOutputChannel);

	template<typename InExpressionClass>
	static UDMMaterialStageInputExpression* ChangeStageInput_Expression(UDMMaterialStage* InStage, int32 InInputIdx, 
		int32 InInputChannel, int32 InOutputIdx, int32 InOutputChannel)
	{
		return ChangeStageSource_Expression(InStage, InExpressionClass::StaticClass(), InInputIdx, InInputChannel, InOutputIdx,
			InOutputChannel);
	}

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TSubclassOf<UDMMaterialStageExpression> GetMaterialStageExpressionClass() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetMaterialStageExpressionClass(TSubclassOf<UDMMaterialStageExpression> InMaterialStageExpressionClass);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStageExpression* GetMaterialStageExpression() const;

protected:
	static TArray<TStrongObjectPtr<UClass>> InputExpressions;

	static void GenerateExpressionList();
};
