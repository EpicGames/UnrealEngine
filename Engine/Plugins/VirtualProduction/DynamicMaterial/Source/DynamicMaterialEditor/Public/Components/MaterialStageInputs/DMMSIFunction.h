// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "DMMSIFunction.generated.h"

class UDMMaterialLayerObject;
class UDMMaterialStageFunction;
class UMaterialFunctionInterface;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputFunction : public UDMMaterialStageInputThroughput
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(UDMMaterialLayerObject* InLayer = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputFunction* ChangeStageSource_Function(UDMMaterialStage* InStage,
		UMaterialFunctionInterface* InMaterialFunction);

	/**
	 * Change the input type of an input on a stage to a function.
	 * @param InInputIdx Index of the source input.
	 * @param InInputChannel The channel of the input that the input connects to.
	 * @param InOutputIdx The output index of the new input.
	 * @param InOutputChannel The channel of the output to connect.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputFunction* ChangeStageInput_Function(UDMMaterialStage* InStage,
		UMaterialFunctionInterface* InMaterialFunction, int32 InInputIdx, int32 InInputChannel, int32 InOutputIdx,
		int32 InOutputChannel);

	void Init();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStageFunction* GetMaterialStageFunction() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UMaterialFunctionInterface* GetMaterialFunction() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction);
};
