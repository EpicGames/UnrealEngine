// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MaterialStageInputs/DMMSIThroughput.h"
#include "Templates/SubclassOf.h"
#include "UObject/StrongObjectPtr.h"
#include "DMMSIGradient.generated.h"

class UDMMaterialLayerObject;
class UDMMaterialStageGradient;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputGradient : public UDMMaterialStageInputThroughput
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(TSubclassOf<UDMMaterialStageGradient> InMaterialStageGradientClass, UDMMaterialLayerObject* InLayer = nullptr);

	static const TArray<TStrongObjectPtr<UClass>>& GetAvailableGradients();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputGradient* ChangeStageSource_Gradient(UDMMaterialStage* InStage,
		TSubclassOf<UDMMaterialStageGradient> InGradientClass);

	template<typename InGradientClass>
	static UDMMaterialStageInputGradient* ChangeStageSource_Gradient(UDMMaterialStage* InStage)
	{
		return ChangeStageSource_Gradient(InStage, InGradientClass::StaticClass());
	}

	/**
	 * Change the input type of an input on a stage to a gradient.
	 * @param InInputIdx Index of the source input.
	 * @param InInputChannel The channel of the input that the input connects to.
	 * @param InOutputChannel The channel of the output to connect.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputGradient* ChangeStageInput_Gradient(UDMMaterialStage* InStage,
		TSubclassOf<UDMMaterialStageGradient> InGradientClass, int32 InInputIdx, int32 InInputChannel, int32 InOutputChannel);

	template<typename InGradientClass>
	static UDMMaterialStageInputGradient* ChangeStageInput_Gradient(UDMMaterialStage* InStage, int32 InInputIdx,
		int32 InInputChannel, int32 InOutputChannel)
	{
		return ChangeStageSource_Gradient(InStage, InGradientClass::StaticClass(), InInputIdx, InInputChannel, InOutputChannel);
	}

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TSubclassOf<UDMMaterialStageGradient> GetMaterialStageGradientClass() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetMaterialStageGradientClass(TSubclassOf<UDMMaterialStageGradient> InMaterialStageGradientClass);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStageGradient* GetMaterialStageGradient() const;

protected:
	static TArray<TStrongObjectPtr<UClass>> Gradients;

	static void GenerateGradientList();
};
