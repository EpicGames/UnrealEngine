// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageInput.h"
#include "Templates/SubclassOf.h"
#include "DMMSISlot.generated.h"

class UDMMaterialLayerObject;
class UDMMaterialStage;
class UDynamicMaterialModel;
struct FDMMaterialBuildState;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputSlot : public UDMMaterialStageInput
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString SlotPathToken;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(UDMMaterialSlot* InSourceSlot, EDMMaterialPropertyType InMaterialProperty,
		UDMMaterialLayerObject* InLayer = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputSlot* ChangeStageSource_Slot(UDMMaterialStage* InStage, UDMMaterialSlot* InSlot,
		EDMMaterialPropertyType InProperty);

	/**
	 * Change the input type of an input on a stage to the output of another slot.
	 * @param InInputIdx Index of the source input.
	 * @param InInputChannel The channel of the input that the input connects to.
	 * @param InProperty The property of the slot to use.
	 * @param InOutputIdx The output index of the new input.
	 * @param InOutputChannel The channel of the output to connect.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputSlot* ChangeStageInput_Slot(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel,
		UDMMaterialSlot* InSlot, EDMMaterialPropertyType InProperty, int32 InOutputIdx, int32 InOutputChannel);

	UDMMaterialStageInputSlot();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMMaterialSlot* GetSlot() const { return Slot; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMMaterialPropertyType GetMaterialProperty() const { return MaterialProperty; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetSlot(UDMMaterialSlot* InSlot);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetMaterialProperty(EDMMaterialPropertyType InMaterialProperty);

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual FText GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel) override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditImport() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TObjectPtr<UDMMaterialSlot> Slot;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMMaterialPropertyType MaterialProperty;

	virtual ~UDMMaterialStageInputSlot() override = default;

	void OnSlotUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);
	void OnSlotConnectorsUpdated(UDMMaterialSlot* InSlot);
	void OnSlotRemoved(UDMMaterialComponent* InComponent, EDMComponentLifetimeState InLifetimeState);
	void OnParentSlotRemoved(UDMMaterialComponent* InComponent, EDMComponentLifetimeState InLifetimeState);

	void InitSlot();
	void DeinitSlot();

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual void UpdateOutputConnectors() override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent
};
