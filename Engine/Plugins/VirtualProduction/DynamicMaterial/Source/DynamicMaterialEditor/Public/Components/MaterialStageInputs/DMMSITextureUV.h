// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageInput.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "DMMSITextureUV.generated.h"

class SWidget;
class UDMMaterialLayerObject;
class UDMMaterialParameter;
class UDMMaterialStage;
class UDMTextureUV;
class UDynamicMaterialModel;
class UMaterialExpressionScalarParameter;
class UMaterialInstanceDynamic;
enum class EDMMaterialParameterGroup : uint8;
struct FDMMaterialBuildState;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputTextureUV : public UDMMaterialStageInput
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString TextureUVPathToken;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(UDynamicMaterialModel* InMaterialModel, UDMMaterialLayerObject* InLayer = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputTextureUV* ChangeStageSource_UV(UDMMaterialStage* InStage, bool bInDoUpdate);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputTextureUV* ChangeStageInput_UV(UDMMaterialStage* InStage, int32 InInputIdx, 
		int32 InInputChannel, int32 InOutputChannel);

	DYNAMICMATERIALEDITOR_API UDMMaterialStageInputTextureUV();

	virtual ~UDMMaterialStageInputTextureUV() override = default;

	void Init(UDynamicMaterialModel* InMaterialModel);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMTextureUV* GetTextureUV() { return TextureUV; }

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual FText GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel) override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditImport() override;
	//~ End UObject

protected:
	DYNAMICMATERIALEDITOR_API static UMaterialExpressionScalarParameter* CreateScalarParameter(const TSharedRef<FDMMaterialBuildState>& InBuildState, FName InParamName,
		EDMMaterialParameterGroup InParameterGroup, float InValue = 0.f);

	static TArray<UMaterialExpression*> CreateTextureUVExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState, UDMTextureUV* InTextureUV);

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMTextureUV> TextureUV;

	void OnTextureUVUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	void InitTextureUV();

	void AddEffects(const TSharedRef<FDMMaterialBuildState>& InBuildState, TArray<UMaterialExpression*>& InOutExpressions) const;

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual void UpdateOutputConnectors() override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent
};
