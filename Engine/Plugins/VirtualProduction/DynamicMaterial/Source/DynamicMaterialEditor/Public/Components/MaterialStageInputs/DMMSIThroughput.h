// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageInput.h"
#include "DMMSIThroughput.generated.h"

class UDMMaterialStage;
class UDMMaterialStageThroughput;
class UDMMaterialSubStage;
class UDynamicMaterialModel;
struct FDMMaterialBuildState;

UCLASS(MinimalAPI, Abstract, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputThroughput : public UDMMaterialStageInput
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static const FString SubStagePathToken;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageInputThroughput();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TSubclassOf<UDMMaterialStageThroughput> GetMaterialStageThroughputClass() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialStageThroughput* GetMaterialStageThroughput() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMMaterialSubStage* GetSubStage() const { return SubStage; }

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual FText GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel) override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual int32 GetInnateMaskOutput(int32 OutputIndex, int32 OutputChannels) const;
	DYNAMICMATERIALEDITOR_API virtual int32 GetOutputChannelOverride(int32 InOutputIndex) const override;
	//~ End UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual bool IsPropertyVisible(FName Property) const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditImport() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialSubStage> SubStage;

	void OnSubStageUpdated(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	void SetMaterialStageThroughputClass(TSubclassOf<UDMMaterialStageThroughput> InMaterialStageThroughputClass);

	void InitSubStage();

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	DYNAMICMATERIALEDITOR_API virtual void GetComponentPathInternal(TArray<FString>& OutChildComponentPathComponents) const override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialComponent
};
