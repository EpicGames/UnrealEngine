// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMMaterialStageInput.h"

#include "Templates/SubclassOf.h"

#include "DMMSIValue.generated.h"

class UDMMaterialLayerObject;
class UDMMaterialStage;
class UDynamicMaterialModel;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMMaterialStageInputValue : public UDMMaterialStageInput
{
	GENERATED_BODY()
		
public:
	DYNAMICMATERIALEDITOR_API static const FString ValuePathToken;

	static FName GetValuePropertyName() { return GET_MEMBER_NAME_CHECKED(UDMMaterialStageInputValue, Value); }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStage* CreateStage(UDMMaterialValue* InValue, UDMMaterialLayerObject* InLayer = nullptr);

	static UDMMaterialStageInputValue* ChangeStageSource_NewLocalValue(UDMMaterialStage* InStage, EDMValueType InValueType);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageSource_NewLocalValue(UDMMaterialStage* InStage, TSubclassOf<UDMMaterialValue> InValueClass);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageSource_Value(UDMMaterialStage* InStage, UDMMaterialValue* InValue);

	static UDMMaterialStageInputValue* ChangeStageSource_NewValue(UDMMaterialStage* InStage, EDMValueType InValueType);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageSource_NewValue(UDMMaterialStage* InStage, TSubclassOf<UDMMaterialValue> InValueClass);

	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageInput_NewLocalValue(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel,
		EDMValueType InValueType, int32 InOutputChannel);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageInput_NewLocalValue(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel,
		TSubclassOf<UDMMaterialValue> InValueClass, int32 InOutputChannel);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageInput_Value(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel,
		UDMMaterialValue* InValue, int32 InOutputChannel);

	static UDMMaterialStageInputValue* ChangeStageInput_NewValue(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel,
		EDMValueType InValueType, int32 InOutputChannel);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue* ChangeStageInput_NewValue(UDMMaterialStage* InStage, int32 InInputIdx, int32 InInputChannel,
		TSubclassOf<UDMMaterialValue> InValueClass, int32 InOutputChannel);

	DYNAMICMATERIALEDITOR_API UDMMaterialStageInputValue();

	virtual ~UDMMaterialStageInputValue() override = default;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDMMaterialValue* GetValue() const { return Value; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetValue(UDMMaterialValue* InValue);

	DYNAMICMATERIALEDITOR_API void ApplyDefaultLayerSettings();

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual FText GetChannelDescription(const FDMMaterialStageConnectorChannel& Channel) override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual int32 GetInnateMaskOutput(int32 OutputIndex, int32 OutputChannels) const override;
	//~ Begin UDMMaterialStageSource

	//~ Begin UDMMaterialComponent
	DYNAMICMATERIALEDITOR_API virtual FText GetComponentDescription() const override;
	DYNAMICMATERIALEDITOR_API virtual FSlateIcon GetComponentIcon() const override;
	DYNAMICMATERIALEDITOR_API virtual void GenerateExpressions(const TSharedRef<FDMMaterialBuildState>& InBuildState) const override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate(UDynamicMaterialModel* InMaterialModel, UDMMaterialComponent* InParent) override;
	DYNAMICMATERIALEDITOR_API virtual bool IsPropertyVisible(FName InProperty) const override;
	//~ End UDMMaterialComponent

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual bool Modify(bool bInAlwaysMarkDirty = true) override;
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditImport() override;
	//~ End UObject

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialValue> Value;

	void OnValueUpdated(UDMMaterialComponent* InComponent, EDMUpdateType InUpdateType);

	void InitInputValue();
	void DeinitInputValue();

	bool IsSharedStageValue() const;

	void ApplyWholeLayerValue();

	bool IsTextureValueVisible() const;

	//~ Begin UDMMaterialStageInput
	DYNAMICMATERIALEDITOR_API virtual void UpdateOutputConnectors() override;
	//~ End UDMMaterialStageInput

	//~ Begin UDMMaterialStageSource
	DYNAMICMATERIALEDITOR_API virtual void OnComponentAdded() override;
	DYNAMICMATERIALEDITOR_API virtual void OnComponentRemoved() override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End UDMMaterialStageSource
};
