// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "DMDefs.h"
#include "DMValueDefinition.h"
#include "IDetailPropertyRow.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

#include "DMEDefs.generated.h"

class FAssetThumbnailPool;
class FProperty;
class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class SDMMaterialComponentEditor;
class SDMMaterialStageEdit;
class SVerticalBox;
class SWidget;
class UDMMaterialStageInput;
class UDMMaterialStageSource;
class UDMMaterialValue;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
class UPrimitiveComponent;

UINTERFACE(MinimalAPI, Blueprintable, BlueprintType)
class UDMBuildable : public UInterface
{
public:
	GENERATED_BODY()
};

class IDMBuildable
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Material Designer")
	void DoBuild(bool bInDirtyAssets);

	virtual void DoBuild_Implementation(bool bInDirtyAssets) PURE_VIRTUAL(IDMBuildable::DoBuild_Implementation);
};

/**
 * An input or output form a material source/stage (e.g. RGB out.)
 */
USTRUCT(BlueprintType, Category = "Material Designer", meta = (DisplayName = "Material Designer Stage Connector"))
struct FDMMaterialStageConnector
{
	GENERATED_BODY()

	/** This is the index of the input connector on the UMaterialExpression node (not on the stage's input array or the inputconnectors array.) */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	int32 Index = INDEX_NONE;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FText Name = FText::GetEmpty();

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	EDMValueType Type = EDMValueType::VT_Float1;

	bool IsCompatibleWith(const FDMMaterialStageConnector& OtherConnector,
		int32 MyChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		int32 OtherChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL) const
	{
		return IsCompatibleWith(OtherConnector.Type, MyChannel, OtherChannel);
	}

	bool IsCompatibleWith(EDMValueType OtherType,
		int32 MyChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		int32 OtherChannel = FDMMaterialStageConnectorChannel::WHOLE_CHANNEL) const
	{
		return UDMValueDefinitionLibrary::AreTypesCompatible(Type, OtherType, MyChannel, OtherChannel);
	}

	bool operator==(const FDMMaterialStageConnector& Other) const
	{
		/** Comparing anything but Index here is unneeded! */
		return Index == Other.Index;
	}
};

/**
 * Represents the channels(channel = float, texture, etc.) that connect to an input.
 *
 * Multiple float channels can be combined to create a single put (e.g. R, G, B -> RGB)
 *
 * Individual source channels, from potentially multiple sources, can be directed to specific
 * input channels (e.g. T1.R, T2.B, T3.G -> {T2.B, T1.R, T3.G, T2.B})
 */
USTRUCT(BlueprintType, Category = "Material Designer", meta = (DisplayName = "Material Designer Stage Connection"))
struct FDMMaterialStageConnection
{
	GENERATED_BODY()

	/**
	 * This struct represents the connections made to a single input connector.
	 * Can connect single outputs or combine them. Append nodes will be used to combine channels.
	 * Combining channels should only be used for float types!
	 */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<FDMMaterialStageConnectorChannel> Channels;
};

USTRUCT(BlueprintType, Category = "Material Designer", meta = (DisplayName = "Material Designer Slot Output Connector Types"))
struct FDMMaterialSlotOutputConnectorTypes
{
	GENERATED_BODY()

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TArray<EDMValueType> ConnectorTypes;
};

enum class EDMPropertyHandlePriority : uint8
{
	Low,
	Normal,
	High
};

struct FDMPropertyGeneratorHandle
{
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;
	TSharedPtr<IDetailTreeNode> DetailTreeNode;
	TSharedPtr<IPropertyHandle> PropertyHandle;
};

struct FDMPropertyHandle
{
	FDMPropertyGeneratorHandle OriginalHandle;
	FDMPropertyGeneratorHandle PreviewHandle;
	TOptional<FText> NameOverride;
	TOptional<FText> NameToolTipOverride;
	TOptional<FResetToDefaultOverride> ResetToDefaultOverride;
	TSharedPtr<SWidget> ValueWidget;
	FName ValueName = NAME_None;
	FName CategoryOverrideName = NAME_None;
	TOptional<float> MaxWidth;
	bool bEnabled = true;
	EDMPropertyHandlePriority Priority = EDMPropertyHandlePriority::Normal;
	bool bKeyframeable = true;
};

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EDMMaterialLayerStage : uint8
{
	None = 0 UMETA(Hidden),
	Base = 1 << 0,
	Mask = 1 << 1,
	All  = Base | Mask
};
ENUM_CLASS_FLAGS(EDMMaterialLayerStage);

UENUM(BlueprintType)
enum class EAvaColorChannel : uint8
{
	None = 0,
	Red = 1 << 0,
	Green = 1 << 1,
	Blue = 1 << 2,
	Alpha = 1 << 3,
	RGB = Red|Green|Blue UMETA(Hidden),
	RGBA = Red|Green|Blue|Alpha UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EAvaColorChannel)
