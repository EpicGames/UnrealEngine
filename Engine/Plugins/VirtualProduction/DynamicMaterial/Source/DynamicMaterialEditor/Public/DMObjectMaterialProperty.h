// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "DMObjectMaterialProperty.generated.h"

class FProperty;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
class UDynamicMaterialModelBase;
class UMaterialInterface;
class UObject;
class UPrimitiveComponent;

/**
 * Overrides the material setter algorithm.
 * @returns true means that the material was set. False if it was not. If false is returned, the normal material setting algorithm is used.
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool /* Set Material */, FDMSetMaterialObjectProperty, const FDMObjectMaterialProperty& /* Object Property */, UDynamicMaterialInstance* /* Material to set */)

/**
 * Defines a material property slot that can be a Material Designer Material.
 */
USTRUCT()
struct FDMObjectMaterialProperty
{
	GENERATED_BODY()

	friend class UDMMaterialInstanceFunctionLibrary;

	FDMObjectMaterialProperty();

	/** UPrimitiveComponent Material Index */
	DYNAMICMATERIALEDITOR_API FDMObjectMaterialProperty(UPrimitiveComponent* InOuter, int32 InIndex);

	/** Class Property (including potential array index) */
	DYNAMICMATERIALEDITOR_API FDMObjectMaterialProperty(UObject* InOuter, FProperty* InProperty, int32 InIndex = INDEX_NONE);

	DYNAMICMATERIALEDITOR_API UObject* GetOuter() const;

	DYNAMICMATERIALEDITOR_API FProperty* GetProperty() const;

	DYNAMICMATERIALEDITOR_API int32 GetIndex() const;

	DYNAMICMATERIALEDITOR_API UDynamicMaterialModelBase* GetMaterialModelBase() const;

	DYNAMICMATERIALEDITOR_API UDynamicMaterialInstance* GetMaterial() const;
	
	DYNAMICMATERIALEDITOR_API UMaterialInterface* GetMaterialInterface() const;

	DYNAMICMATERIALEDITOR_API bool IsValid() const;

	DYNAMICMATERIALEDITOR_API FText GetPropertyName(bool bInIgnoreNewStatus) const;

	DYNAMICMATERIALEDITOR_API void Reset();

	/** Whether this is a property on the object. */
	DYNAMICMATERIALEDITOR_API bool IsProperty() const;

	/** Whether this is an element of a primitive component's material override array. */
	DYNAMICMATERIALEDITOR_API bool IsElement() const;

	template<typename InClass>
	InClass* GetTypedOuter() const
	{
		if (UObject* Outer = OuterWeak.Get())
		{
			if (InClass* CastOuter = Cast<InClass>(Outer))
			{
				return CastOuter;
			}

			return Outer->GetTypedOuter<InClass>();
		}

		return nullptr;
	}

	/* The delegate can be used to override (or preprocess) the material setter function */
	DYNAMICMATERIALEDITOR_API void SetMaterialSetterDelegate(const FDMSetMaterialObjectProperty& InDelegate);

protected:
	UPROPERTY()
	TWeakObjectPtr<UObject> OuterWeak = nullptr;

	/** C++ version of property */
	FProperty* Property = nullptr;

	/** Component or array property index. */
	int32 Index = INDEX_NONE;

	FDMSetMaterialObjectProperty MaterialSetterDelegate;

	void SetMaterial(UDynamicMaterialInstance* InDynamicMaterial);
};
