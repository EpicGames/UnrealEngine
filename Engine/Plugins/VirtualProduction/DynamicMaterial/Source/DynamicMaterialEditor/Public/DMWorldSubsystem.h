// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Subsystems/WorldSubsystem.h"

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

#include "DMWorldSubsystem.generated.h"

class AActor;
class IDetailKeyframeHandler;
class UDynamicMaterialInstance;
class UDynamicMaterialModelBase;
struct FDMObjectMaterialProperty;

DECLARE_DELEGATE_RetVal(UDynamicMaterialModelBase*, FDMGetMaterialModelDelegate)
DECLARE_DELEGATE_OneParam(FDMSetMaterialModelDelegate, UDynamicMaterialModelBase*)
DECLARE_DELEGATE_OneParam(FDMSetMaterialObjectPropertyDelegate, const FDMObjectMaterialProperty&)
DECLARE_DELEGATE_OneParam(FDMSetMaterialActorDelegate, AActor*)
DECLARE_DELEGATE_RetVal_OneParam(bool, FDMIsValidDelegate, UDynamicMaterialModelBase*)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FDMSetMaterialValueDelegate, const FDMObjectMaterialProperty&, UDynamicMaterialInstance*)
DECLARE_DELEGATE(FDMInvokeTabDelegate)

UCLASS(MinimalAPI)
class UDMWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
 
public:
	UDMWorldSubsystem();
 
	const TSharedPtr<IDetailKeyframeHandler>& GetKeyframeHandler() const
	{
		return KeyframeHandler;
	}

	void SetKeyframeHandler(const TSharedPtr<IDetailKeyframeHandler>& InKeyframeHandler)
	{
		KeyframeHandler = InKeyframeHandler;
	}

	/** Gets the material in a custom editor tab. */
	FDMGetMaterialModelDelegate::RegistrationType& GetGetCustomEditorModelDelegate() const
	{
		return CustomModelEditorGetDelegate;
	}

	UDynamicMaterialModelBase* ExecuteGetCustomEditorModelDelegate();

	/** Sets material in a custom editor tab. */
	FDMSetMaterialModelDelegate::RegistrationType& GetSetCustomEditorModelDelegate() const
	{
		return CustomModelEditorSetDelegate;
	}

	void ExecuteSetCustomEditorModelDelegate(UDynamicMaterialModelBase* InMaterialModel);

	/** Sets the object property in custom editor tab. */
	FDMSetMaterialObjectPropertyDelegate::RegistrationType& GetCustomObjectPropertyEditorDelegate() const
	{
		return CustomObjectPropertyEditorDelegate;
	}

	void ExecuteCustomObjectPropertyEditorDelegate(const FDMObjectMaterialProperty& InObjectProperty);

	/** Sets actor in a custom editor tab. */
	FDMSetMaterialActorDelegate::RegistrationType& GetSetCustomEditorActorDelegate() const
	{
		return CustomActorEditorDelegate;
	}

	void ExecuteSetCustomEditorActorDelegate(AActor* InActor);

	/** Returns true if the supplied material is valid for this world. */
	FDMIsValidDelegate::RegistrationType& GetIsValidDelegate() const
	{
		return IsValidDelegate;
	}

	bool ExecuteIsValidDelegate(UDynamicMaterialModelBase* InMaterialModel);

	/** Used to redirect SetMaterial to different objects/paths. */
	FDMSetMaterialValueDelegate::RegistrationType& GetMaterialValueSetterDelegate() const
	{
		return SetMaterialValueDelegate;
	}

	bool ExecuteMaterialValueSetterDelegate(const FDMObjectMaterialProperty& InObjectProperty, UDynamicMaterialInstance* InMaterialInstance);

	/** Used to show the tab to the user. */
	FDMInvokeTabDelegate::RegistrationType& GetInvokeTabDelegate() const
	{
		return InvokeTabDelegate;
	}

	void ExecuteInvokeTabDelegate();

protected:
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler;
	mutable FDMGetMaterialModelDelegate CustomModelEditorGetDelegate;
	mutable FDMSetMaterialModelDelegate CustomModelEditorSetDelegate;
	mutable FDMSetMaterialObjectPropertyDelegate CustomObjectPropertyEditorDelegate;
	mutable FDMSetMaterialActorDelegate CustomActorEditorDelegate;
	mutable FDMIsValidDelegate IsValidDelegate;
	mutable FDMSetMaterialValueDelegate SetMaterialValueDelegate;
	mutable FDMInvokeTabDelegate InvokeTabDelegate;
};
