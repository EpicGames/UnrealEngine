// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Misc/NotNull.h"
#include "Templates/SharedPointerFwd.h"
#include "UI/Utils/IDMWidgetLibrary.h"

class AActor;
class FNotifyHook;
class SDMMaterialComponentEditor;
class UClass;
class UDMMaterialComponent;
class UDynamicMaterialInstance;
class UDynamicMaterialModelBase;
class UObject;
struct FDMObjectMaterialProperty;
struct FDMPropertyHandle;
struct FDMPropertyHandleGenerateParams;
struct IDMOnWizardCompleteCallback;

struct FDMComponentPropertyRowGeneratorParams
{
	const SWidget* Owner = nullptr;
	FNotifyHook* NotifyHook = nullptr;
	UDynamicMaterialModelBase* PreviewMaterialModelBase = nullptr;
	UDynamicMaterialModelBase* OriginalMaterialModelBase = nullptr;
	UObject* Object = nullptr;
	TNotNull<TArray<FDMPropertyHandle>*> PropertyRows;
	TSet<UObject*>& ProcessedObjects;

	FDMComponentPropertyRowGeneratorParams(TArray<FDMPropertyHandle>& InPropertyRows, TSet<UObject*>& InProcessedObjects)
		: PropertyRows(&InPropertyRows)
		, ProcessedObjects(InProcessedObjects)
	{
	}

	FDMPropertyHandleGenerateParams CreatePropertyHandleParams(FName InPropertyName) const
	{
		return {Owner, NotifyHook, PreviewMaterialModelBase, OriginalMaterialModelBase, Object, InPropertyName};
	}
};

DECLARE_DELEGATE_RetVal_OneParam(TArray<FDMObjectMaterialProperty>, FDMGetObjectMaterialPropertiesDelegate, UObject* InObject)

/** Creates property rows in the edit widget. */
DECLARE_DELEGATE_OneParam(FDMComponentPropertyRowGeneratorDelegate, FDMComponentPropertyRowGeneratorParams&)

/** Material Designer - Build your own materials in a slimline editor! */
class IDynamicMaterialEditorModule : public IModuleInterface
{
protected:
	static constexpr const TCHAR* ModuleName = TEXT("DynamicMaterialEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IDynamicMaterialEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDynamicMaterialEditorModule>(ModuleName);
	}

	virtual void RegisterComponentPropertyRowGeneratorDelegate(UClass* InClass, FDMComponentPropertyRowGeneratorDelegate InComponentPropertyRowGeneratorDelegate) = 0;

	template <class InObjClass, class InGenClass>
	void RegisterComponentPropertyRowGeneratorDelegate()
	{
		RegisterComponentPropertyRowGeneratorDelegate(
			InObjClass::StaticClass(),
			FDMComponentPropertyRowGeneratorDelegate::CreateSP(
				InGenClass::Get(),
				&InGenClass::AddComponentProperties
			)
		);
	}

	virtual void RegisterCustomMaterialPropertyGenerator(UClass* InClass, FDMGetObjectMaterialPropertiesDelegate InGenerator) = 0;

	virtual void RegisterMaterialModelCreatedCallback(const TSharedRef<IDMOnWizardCompleteCallback> InCallback) = 0;

	virtual void UnregisterMaterialModelCreatedCallback(const TSharedRef<IDMOnWizardCompleteCallback> InCallback) = 0;

	template<typename InCallbackType, typename... InArgsType
		UE_REQUIRES(std::is_base_of_v<IDMOnWizardCompleteCallback, InCallbackType>)>
	TSharedRef<InCallbackType> RegisterMaterialModelCreatedCallback(InArgsType&&... InArgs)
	{
		TSharedRef<InCallbackType> NewCallback = MakeShared<InCallbackType>(Forward<InArgsType>(InArgs)...);
		RegisterMaterialModelCreatedCallback(NewCallback);
		return NewCallback;
	}

	virtual void OpenEditor(UWorld* InWorld) const = 0;

	virtual UDynamicMaterialModelBase* GetOpenedMaterialModel(UWorld* InWorld) const = 0;

	virtual void OpenMaterialModel(UDynamicMaterialModelBase* InMaterialModel, UWorld* InWorld, bool bInInvokeTab) const = 0;

	virtual void OpenMaterialObjectProperty(const FDMObjectMaterialProperty& InObjectProperty, UWorld* InWorld, bool bInInvokeTab) const = 0;

	virtual void OpenMaterial(UDynamicMaterialInstance* InInstance, UWorld* InWorld, bool bInInvokeTab) const = 0;

	virtual void OnActorSelected(AActor* InActor, UWorld* InWorld, bool bInInvokeTab) const = 0;

	virtual void ClearDynamicMaterialModel(UWorld* InWorld) const = 0;
	
	virtual IDMWidgetLibrary& GetWidgetLibrary() const = 0;
};
