// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "DynamicMaterialInstanceFactory.generated.h"

UCLASS(MinimalAPI)
class UDynamicMaterialInstanceFactory : public UFactory
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API UDynamicMaterialInstanceFactory();

	//~ Begin UFactory
	DYNAMICMATERIALEDITOR_API virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, 
		UObject* InContext, FFeedbackContext* InWarn) override;
	DYNAMICMATERIALEDITOR_API virtual FText GetDisplayName() const override;
	DYNAMICMATERIALEDITOR_API virtual FText GetToolTip() const override;
	//~ End UFactory
};
