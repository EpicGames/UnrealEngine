// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/IDMMaterialBuildStateInterface.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Model/DMMaterialBuildUtils.h"

class UDMMaterialLayerObject;
class UDMMaterialProperty;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialStageInput;
class UDMMaterialStageSource;
class UDMMaterialStageThroughput;
class UDynamicMaterialModel;
class UMaterial;
class UMaterialExpression;
class UMaterialExpressionAppendVector;
class UMaterialExpressionComponentMask;
struct FDMExpressionInput;
struct FDMMaterialBuildUtils;
struct FDMMaterialStageConnection;

/**
 * BuildState is a class that stores the current state of a material that is being built.
 * It stores useful lists of UMaterialExpressions relating to various object within the builder, such a Stages or Sources.
 * It is an entirely transient object. It is not meant to be saved outside of the material building processs.
 * It also provides some helper functions for creating UMaterialExpressions.
 */
struct FDMMaterialBuildState : public IDMMaterialBuildStateInterface, public TSharedFromThis<FDMMaterialBuildState>
{
	DYNAMICMATERIALEDITOR_API FDMMaterialBuildState(UMaterial* InDynamicMaterial, UDynamicMaterialModel* InMaterialModel, bool bInDirtyAssets = true);

	virtual ~FDMMaterialBuildState() override;

	DYNAMICMATERIALEDITOR_API virtual UMaterial* GetDynamicMaterial() const override;

	DYNAMICMATERIALEDITOR_API virtual UDynamicMaterialModel* GetMaterialModel() const override;

	/**
	 * The current material property being generated for.
	 * May be null in the case of global values or global parameters.
	 */
	DYNAMICMATERIALEDITOR_API const UDMMaterialProperty* GetCurrentMaterialProperty() const;

	DYNAMICMATERIALEDITOR_API void SetCurrentMaterialProperty(const UDMMaterialProperty* InProperty);

	/** Whether assets can potentially be dirtied by the build process. */
	bool ShouldDirtyAssets() const { return bDirtyAssets; }

	/** If ignoring UVs is on, UV nodes will not be processed. Useful for preview materials. */
	bool IsIgnoringUVs() const { return bIgnoreUVs; }

	DYNAMICMATERIALEDITOR_API void SetIgnoreUVs();

	/** Whether the build state is building a full material or a preview. */
	UObject* GetPreviewObject() const { return PreviewObject; }

	DYNAMICMATERIALEDITOR_API void SetPreviewObject(UObject* InObject);

	/** A handy set of tools for creating material expressions. */
	DYNAMICMATERIALEDITOR_API virtual IDMMaterialBuildUtilsInterface& GetBuildUtils() const override;

	/** Returns the FExpressionInput for the given material property on the material's editor only data. */
	DYNAMICMATERIALEDITOR_API FExpressionInput* GetMaterialProperty(EDMMaterialPropertyType InProperty) const;

	/** Slots */
	DYNAMICMATERIALEDITOR_API bool HasSlot(const UDMMaterialSlot* InSlot) const;

	DYNAMICMATERIALEDITOR_API const TArray<UMaterialExpression*>& GetSlotExpressions(const UDMMaterialSlot* InSlot) const;

	DYNAMICMATERIALEDITOR_API UMaterialExpression* GetLastSlotExpression(const UDMMaterialSlot* InSlot) const;

	DYNAMICMATERIALEDITOR_API void AddSlotExpressions(const UDMMaterialSlot* InSlot, const TArray<UMaterialExpression*>& InSlotExpressions);

	DYNAMICMATERIALEDITOR_API bool HasSlotProperties(const UDMMaterialSlot* InSlot) const;

	DYNAMICMATERIALEDITOR_API void AddSlotPropertyExpressions(const UDMMaterialSlot* InSlot, const TMap<EDMMaterialPropertyType,
		TArray<UMaterialExpression*>>& InSlotPropertyExpressions);

	DYNAMICMATERIALEDITOR_API const TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>>& GetSlotPropertyExpressions(const UDMMaterialSlot* InSlot);

	DYNAMICMATERIALEDITOR_API UMaterialExpression* GetLastSlotPropertyExpression(const UDMMaterialSlot* InSlot, EDMMaterialPropertyType InMaterialProperty) const;

	DYNAMICMATERIALEDITOR_API TArray<const UDMMaterialSlot*> GetSlots() const;

	DYNAMICMATERIALEDITOR_API const TMap<const UDMMaterialSlot*, TArray<UMaterialExpression*>>& GetSlotMap() const;

	/** Layers */
	DYNAMICMATERIALEDITOR_API bool HasLayer(const UDMMaterialLayerObject* InLayer) const;

	DYNAMICMATERIALEDITOR_API const TArray<UMaterialExpression*>& GetLayerExpressions(const UDMMaterialLayerObject* InLayer) const;

	DYNAMICMATERIALEDITOR_API UMaterialExpression* GetLastLayerExpression(const UDMMaterialLayerObject* InLayer) const;

	DYNAMICMATERIALEDITOR_API void AddLayerExpressions(const UDMMaterialLayerObject* InLayer, const TArray<UMaterialExpression*>& InLayerExpressions);

	DYNAMICMATERIALEDITOR_API TArray<const UDMMaterialLayerObject*> GetLayers() const;

	DYNAMICMATERIALEDITOR_API const TMap<const UDMMaterialLayerObject*, TArray<UMaterialExpression*>>& GetLayerMap() const;

	/** Stages */
	DYNAMICMATERIALEDITOR_API bool HasStage(const UDMMaterialStage* InStage) const;

	DYNAMICMATERIALEDITOR_API const TArray<UMaterialExpression*>& GetStageExpressions(const UDMMaterialStage* InStage) const;

	DYNAMICMATERIALEDITOR_API UMaterialExpression* GetLastStageExpression(const UDMMaterialStage* InStage) const;

	DYNAMICMATERIALEDITOR_API void AddStageExpressions(const UDMMaterialStage* InStage, const TArray<UMaterialExpression*>& InStageExpressions);

	DYNAMICMATERIALEDITOR_API TArray<const UDMMaterialStage*> GetStages() const;

	DYNAMICMATERIALEDITOR_API const TMap<const UDMMaterialStage*, TArray<UMaterialExpression*>>& GetStageMap() const;

	/** Stage Sources */
	DYNAMICMATERIALEDITOR_API bool HasStageSource(const UDMMaterialStageSource* InStageSource) const;

	DYNAMICMATERIALEDITOR_API const TArray<UMaterialExpression*>& GetStageSourceExpressions(const UDMMaterialStageSource* InStageSource) const;

	DYNAMICMATERIALEDITOR_API UMaterialExpression* GetLastStageSourceExpression(const UDMMaterialStageSource* InStageSource) const;

	DYNAMICMATERIALEDITOR_API void AddStageSourceExpressions(const UDMMaterialStageSource* InStageSource,
		const TArray<UMaterialExpression*>& InStageSourceExpressions);

	DYNAMICMATERIALEDITOR_API TArray<const UDMMaterialStageSource*> GetStageSources() const;

	DYNAMICMATERIALEDITOR_API const TMap<const UDMMaterialStageSource*, TArray<UMaterialExpression*>>& GetStageSourceMap() const;

	/** Material Values */
	DYNAMICMATERIALEDITOR_API virtual bool HasValue(const UDMMaterialValue* InValue) const override;

	DYNAMICMATERIALEDITOR_API virtual const TArray<UMaterialExpression*>& GetValueExpressions(const UDMMaterialValue* InValue) const override;

	DYNAMICMATERIALEDITOR_API virtual UMaterialExpression* GetLastValueExpression(const UDMMaterialValue* InValue) const override;

	DYNAMICMATERIALEDITOR_API virtual void AddValueExpressions(const UDMMaterialValue* InValue, const TArray<UMaterialExpression*>& InValueExpressions) override;

	DYNAMICMATERIALEDITOR_API virtual TArray<const UDMMaterialValue*> GetValues() const override;

	DYNAMICMATERIALEDITOR_API virtual const TMap<const UDMMaterialValue*, TArray<UMaterialExpression*>>& GetValueMap() const override;

	/** Other */
	DYNAMICMATERIALEDITOR_API virtual void AddOtherExpressions(const TArray<UMaterialExpression*>& InOtherExpressions) override;

	DYNAMICMATERIALEDITOR_API virtual const TSet<UMaterialExpression*>& GetOtherExpressions() override;

	/** Global Expressions */
	DYNAMICMATERIALEDITOR_API UMaterialExpression* GetGlobalExpression(FName InName) const;

	DYNAMICMATERIALEDITOR_API void SetGlobalExpression(FName InName, UMaterialExpression* InExpression);

private:
	UMaterial* DynamicMaterial = nullptr;
	UDynamicMaterialModel* MaterialModel = nullptr;
	const UDMMaterialProperty* CurrentProperty = nullptr;
	bool bDirtyAssets;
	bool bIgnoreUVs;
	UObject* PreviewObject = nullptr;
	TSharedRef<FDMMaterialBuildUtils> Utils;

	TMap<const UDMMaterialValue*, TArray<UMaterialExpression*>> Values;
	TMap<const UDMMaterialSlot*, TArray<UMaterialExpression*>> Slots;
	TMap<const UDMMaterialSlot*, TMap<EDMMaterialPropertyType, TArray<UMaterialExpression*>>> SlotProperties;
	TMap<const UDMMaterialLayerObject*, TArray<UMaterialExpression*>> Layers;
	TMap<const UDMMaterialStage*, TArray<UMaterialExpression*>> Stages;
	TMap<const UDMMaterialStageSource*, TArray<UMaterialExpression*>> StageSources;
	TSet<UMaterialExpression*> OtherExpressions;
	TMap<FName, UMaterialExpression*> GlobalExpressions;

	void BuildMaterial();
};
