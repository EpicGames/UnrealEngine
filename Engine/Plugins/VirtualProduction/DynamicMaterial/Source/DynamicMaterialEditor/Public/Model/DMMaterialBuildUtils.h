// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/IDMMaterialBuildUtilsInterface.h"

struct FDMMaterialBuildState;

/** Utilities for creating material expressions. */
struct FDMMaterialBuildUtils : public IDMMaterialBuildUtilsInterface
{
	DYNAMICMATERIALEDITOR_API FDMMaterialBuildUtils(FDMMaterialBuildState& InBuildState);

	/** Creates a default expression outputing 0 on a single channel. */
	DYNAMICMATERIALEDITOR_API UMaterialExpression* CreateDefaultExpression() const;

	/**
	 * Create an expression with the comment as its description.
	 * @param InAsset For things like texture nodes, will set the default value of that asset property.
	 */
	DYNAMICMATERIALEDITOR_API virtual UMaterialExpression* CreateExpression(TSubclassOf<UMaterialExpression> InExpressionClass, const FString& InComment,
		UObject* InAsset = nullptr) const override;

	/** @See CreateExpression */
	template<typename InExpressionClass
		UE_REQUIRES(std::derived_from<InExpressionClass, UMaterialExpression>)>
	InExpressionClass* CreateExpression(const FString& InComment, UObject* InAsset = nullptr) const
	{
		return Cast<InExpressionClass>(CreateExpression(TSubclassOf<UMaterialExpression>(InExpressionClass::StaticClass()), InComment, InAsset));
	}

	/**
	 * Create a parameter expression with the comment as its description.
	 * @param InParameterName The name of the parameter exposed in the material.
	 * @param InParameterGroup Determines the type of group assigned to the parameter.
	 * @param InAsset For things like texture nodes, will set the default value of that asset property.
	 */
	DYNAMICMATERIALEDITOR_API virtual UMaterialExpression* CreateExpressionParameter(TSubclassOf<UMaterialExpression> InExpressionClass, FName InParameterName,
		EDMMaterialParameterGroup InParameterGroup, const FString& InComment, UObject* InAsset = nullptr) const override;

	/** @See CreateExpressionParameter */
	template<typename InExpressionClass
		UE_REQUIRES(std::derived_from<InExpressionClass, UMaterialExpression>)>
	InExpressionClass* CreateExpressionParameter(FName InParameterName, EDMMaterialParameterGroup InParameterGroup, const FString& InComment, UObject* InAsset = nullptr) const
	{
		return Cast<InExpressionClass>(CreateExpressionParameter(TSubclassOf<UMaterialExpression>(InExpressionClass::StaticClass()), InParameterName, InParameterGroup, InComment, InAsset));
	}

	/** Creates a series of nodes that try to render every single input on different parts of the material. */
	DYNAMICMATERIALEDITOR_API virtual TArray<UMaterialExpression*> CreateExpressionInputs(const TArray<FDMMaterialStageConnection>& InInputConnectionMap,
		int32 InStageSourceInputIdx, const TArray<UDMMaterialStageInput*>& InStageInputs, int32& OutOutputIndex, 
		int32& OutOutputChannel) const override;

	/** Creates a series of nodes that display a single input. */
	DYNAMICMATERIALEDITOR_API virtual TArray<UMaterialExpression*> CreateExpressionInput(UDMMaterialStageInput* InInput) const override;

	/** Searches the outputs of an expression to see if there is an appropriate output to match a requested channel mask. Returns INDEX_NONE on failure. */
	DYNAMICMATERIALEDITOR_API virtual int32 FindOutputForBitmask(UMaterialExpression* InExpression, int32 InOutputChannels) const override;

	/** Creates a mask expression with the given channels exposed. */
	DYNAMICMATERIALEDITOR_API virtual UMaterialExpressionComponentMask* CreateExpressionBitMask(UMaterialExpression* InExpression, int32 InOutputIndex,
		int32 InOutputChannels) const override;

	/** Creates an append expression to combine vectors/scalars together to create larger vectors. */
	DYNAMICMATERIALEDITOR_API virtual UMaterialExpressionAppendVector* CreateExpressionAppend(UMaterialExpression* InExpressionA, int32 InOutputIndexA,
		UMaterialExpression* InExpressionB, int32 InOutputIndexB) const override;

	/** Updates the emissive channel of the give material to show the output of the given expression. */
	DYNAMICMATERIALEDITOR_API virtual void UpdatePreviewMaterial(UMaterialExpression* InLastExpression, int32 InOutputIndex, int32 InOutputChannel,
		int32 InSize) const override;

protected:
	FDMMaterialBuildState& BuildState;
};
