// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class AActor;
class UActorComponent;
class UDynamicMaterialModelEditorOnlyData;
class UDynamicMaterialModel;
class UObject;

struct FDMOnWizardCompleteCallbackParams
{
	UDynamicMaterialModel* MaterialModel;
	UDynamicMaterialModelEditorOnlyData* EditorOnlyData;
	UObject* Outer;
	UActorComponent* OuterComponent;
	AActor* OuterActor;
};

// Callback interface
struct IDMOnWizardCompleteCallback : public TSharedPtr<IDMOnWizardCompleteCallback>
{
	virtual ~IDMOnWizardCompleteCallback() = default;

	virtual uint32 GetPriority() const = 0;
	virtual void OnModelCreated(const FDMOnWizardCompleteCallbackParams& InParams) = 0;

	virtual bool operator<(const IDMOnWizardCompleteCallback& InOther) const
	{
		return GetPriority() < InOther.GetPriority();
	}
};

// Default implementation
struct FDMMaterialModelCreatedCallbackBase : public IDMOnWizardCompleteCallback
{
	FDMMaterialModelCreatedCallbackBase(uint32 InPriority);

	virtual ~FDMMaterialModelCreatedCallbackBase() override = default;

	//~ Begin IDMMaterialModelCreatedCallback
	virtual uint32 GetPriority() const override;
	//~ End IDMMaterialModelCreatedCallback

protected:
	uint32 Priority;
};

// Default implementation
struct FDMMaterialModelCreatedCallbackDelegate : public FDMMaterialModelCreatedCallbackBase
{
	DECLARE_DELEGATE_OneParam(FOnModelCreated, const FDMOnWizardCompleteCallbackParams&)

	DYNAMICMATERIALEDITOR_API FDMMaterialModelCreatedCallbackDelegate(uint32 InPriority, const FOnModelCreated& InOnModelCreatedDelegate);

	virtual ~FDMMaterialModelCreatedCallbackDelegate() override = default;

	//~ Begin IDMMaterialModelCreatedCallback
	DYNAMICMATERIALEDITOR_API virtual void OnModelCreated(const FDMOnWizardCompleteCallbackParams& InParams) override;
	//~ End IDMMaterialModelCreatedCallback

protected:
	FOnModelCreated OnModelCreatedDelegate;
};
