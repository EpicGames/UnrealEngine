// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMEDefs.h"
#include "Misc/NotifyHook.h"
#include "Model/IDynamicMaterialModelEditorOnlyDataInterface.h"
#include "UObject/Object.h"

#include "Engine/EngineTypes.h"
#include "MaterialDomain.h"
#include "MaterialEditingLibrary.h"
#include "UObject/WeakObjectPtrFwd.h"

#include "DynamicMaterialModelEditorOnlyData.generated.h"

class UDMMaterialComponent;
class UDMMaterialParameter;
class UDMMaterialProperty;
class UDMMaterialSlot;
class UDMMaterialValue;
class UDMMaterialValueFloat1;
class UDMTextureSet;
class UDMTextureUV;
class UDynamicMaterialModel;
class UDynamicMaterialModelBase;
class UDynamicMaterialModelEditorOnlyData;
class UMaterialExpression;
struct FDMComponentPath;
struct FDMComponentPathSegment;
struct FDMMaterialBuildState;
struct FDMMaterialChannelListPreset;

DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnMaterialBuilt, UDynamicMaterialModelBase*);
DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnValueListUpdated, UDynamicMaterialModelBase*);
DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnSlotListUpdated, UDynamicMaterialModelBase*);
DECLARE_MULTICAST_DELEGATE_OneParam(FDMOnPropertyUpdated, UDynamicMaterialModelBase*);

UENUM(BlueprintType)
enum class EDMState : uint8
{
	Idle,
	Building
};

/** Convert engine linear enum to bitflag enum. */
UENUM(BlueprintType, meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum EDMMaterialFlagsUsage : int32
{
	DMMFU_None                   = 0                                                    UMETA(Hidden),
	/** Indicates that the material and its instances can be used on static meshes. */
	DMMFU_StaticMesh             = 1 << EMaterialUsage::MATUSAGE_StaticMesh             UMETA(DisplayName = "Static Mesh"),
	/** Indicates that the material and its instances can be used with skeletal meshes. */
	DMMFU_SkeletalMesh           = 1 << EMaterialUsage::MATUSAGE_SkeletalMesh           UMETA(DisplayName = "Skeletal Mesh"),
	/** Indicates that the material and its instances can be used with spline meshes. */
	DMMFU_SplineMesh             = 1 << EMaterialUsage::MATUSAGE_SplineMesh             UMETA(DisplayName = "Spline Mesh"),
	/** Indicates that the material and its instances can be used with instanced static meshes. */
	DMMFU_InstancedStaticMeshes  = 1 << EMaterialUsage::MATUSAGE_InstancedStaticMeshes  UMETA(DisplayName = "Instanced Static Mesh"),
	/** Indicates that the material and its instances can be used with particle sprites. */
	DMMFU_ParticleSprites        = 1 << EMaterialUsage::MATUSAGE_ParticleSprites        UMETA(DisplayName = "Particle Sprites"),
	/** Indicates that the material and its instances can be used with beam trails. */
	DMMFU_BeamTrails             = 1 << EMaterialUsage::MATUSAGE_BeamTrails             UMETA(DisplayName = "Beam Trails"),
	/** Indicates that the material and its instances can be used with mesh particles. */
	DMMFU_MeshParticles          = 1 << EMaterialUsage::MATUSAGE_MeshParticles          UMETA(DisplayName = "Mesh Particles"),
	/** Indicates that the material and its instances can be used with static lighting. */
	DMMFU_StaticLighting         = 1 << EMaterialUsage::MATUSAGE_StaticLighting         UMETA(DisplayName = "Static Lighting"),
	/** Indicates that the material and its instances can be used with morph targets. */
	DMMFU_MorphTargets           = 1 << EMaterialUsage::MATUSAGE_MorphTargets           UMETA(DisplayName = "Morph Targets"),
	/** Indicates that the material and its instances can be use with geometry collections. */
	DMMFU_GeometryCollections    = 1 << EMaterialUsage::MATUSAGE_GeometryCollections    UMETA(DisplayName = "Geometry Collections"),
	/** Indicates that the material and its instances can be use with geometry caches. */
	DMMFU_GeometryCache          = 1 << EMaterialUsage::MATUSAGE_GeometryCache          UMETA(DisplayName = "Geometry Cache"),
	/** Indicates that the material and its instances can be used with clothing. */
	DMMFU_Clothing               = 1 << EMaterialUsage::MATUSAGE_Clothing               UMETA(DisplayName = "Clothing"),
	/** Indicates that the material and its instances can be used with Niagara sprites. */
	DMMFU_NiagaraSprites         = 1 << EMaterialUsage::MATUSAGE_NiagaraSprites         UMETA(DisplayName = "Niagara Sprites"),
	/** Indicates that the material and its instances can be used with Niagara ribbons. */
	DMMFU_NiagaraRibbons         = 1 << EMaterialUsage::MATUSAGE_NiagaraRibbons         UMETA(DisplayName = "Niagara Ribbons"),
	/** Indicates that the material and its instances can be used with Niagara meshes. */
	DMMFU_NiagaraMeshParticles   = 1 << EMaterialUsage::MATUSAGE_NiagaraMeshParticles   UMETA(DisplayName = "Niagara Mesh Particles"),
	/** Indicates that the material and its instances can be use with water. */
	DMMFU_Water                  = 1 << EMaterialUsage::MATUSAGE_Water                  UMETA(DisplayName = "Water"),
	/** Indicates that the material and its instances can be use with hair strands. */
	DMMFU_HairStrands            = 1 << EMaterialUsage::MATUSAGE_HairStrands            UMETA(DisplayName = "HairStrands"),
	/** Indicates that the material and its instances can be use with LiDAR Point Clouds. */
	DMMFU_LidarPointCloud        = 1 << EMaterialUsage::MATUSAGE_LidarPointCloud        UMETA(DisplayName = "Lidar Point Cloud"),
	/** Indicates that the material and its instances can be used with Virtual Heightfield Mesh. (Deprecated). */
	DMMFU_VirtualHeightfieldMesh = 1 << EMaterialUsage::MATUSAGE_VirtualHeightfieldMesh UMETA(DisplayName = "Virtual Heightfield Mesh"),
	/** Indicates that the material and its instances can be used with Nanite meshes. */
	DMMFU_Nanite                 = 1 << EMaterialUsage::MATUSAGE_Nanite                 UMETA(DisplayName = "Nanite"),
	/** Indicates that the material and its instances can be used with Nanite voxel meshes. */
	DMMFU_Voxels                 = 1 << EMaterialUsage::MATUSAGE_Voxels                 UMETA(DisplayName = "Voxels"),
	/** Indicates that the material and its instances with volumetric cloud. Without that flag, it can only be used on volumetric fog. */
	DMMFU_VolumetricCloud        = 1 << EMaterialUsage::MATUSAGE_VolumetricCloud        UMETA(DisplayName = "Volumetric Clouds"),
	/** Indicates that the material and its instances with heterogeneous volumes. Without that flag, it can only be used on volumetric fog. */
	DMMFU_HeterogeneousVolumes   = 1 << EMaterialUsage::MATUSAGE_HeterogeneousVolumes   UMETA(DisplayName = "Hetergeneous Volumes"),
	/** Indicates that the material and its instances can be used with editor compositing. */
	DMMFU_EditorCompositing      = 1 << EMaterialUsage::MATUSAGE_EditorCompositing      UMETA(DisplayName = "Editor Compositing"),
	/** Indicates that the material and its instances can be used with neural network engine. */
	DMMFU_NeuralNetworks         = 1 << EMaterialUsage::MATUSAGE_NeuralNetworks         UMETA(DisplayName = "Neural Networks")
};

UENUM(BlueprintType, meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EDMMaterialFlagsGeneral : uint8
{
	None                             = 0      UMETA(HIdden),
	TwoSided                         = 1 << 0 UMETA(ToolTip = "Indicates that the material should be rendered without backface culling and the normal should be flipped for backfaces."),
	TangentSpaceNormal               = 1 << 1 UMETA(ToolTip = "Whether the material takes a tangent space normal or a world space normal as input. (TangentSpace requires extra instructions but is often more convenient)."),
	UsesDistortion                   = 1 << 2 UMETA(ToolTip = "Indicates that the material and its instances can be used with distortion. This will result in the shaders required to support distortion being compiled which will increase shader compile time and memory usage."),
	NormalCurvatureToRoughness       = 1 << 3 UMETA(ToolTip = "Reduce roughness based on screen space normal changes."),
	FullyRough                       = 1 << 4 UMETA(ToolTip = "Forces the material to be completely rough. Saves a number of instructions and one sampler."),
	DitheredLODTransition            = 1 << 5 UMETA(ToolTip = "Whether meshes rendered with the material should support dithered LOD transitions."),
	EnableDisplacementFade           = 1 << 6 UMETA(ToolTip = "Enables fading out and disabling of dynamic displacement in the distance, as displacement becomes unnoticeable."),
	GenerateSphericalParticleNormals = 1 << 7 UMETA(ToolTip = "Whether to generate spherical normals for particles that use this material.")
};
ENUM_CLASS_FLAGS(EDMMaterialFlagsGeneral)

UENUM(BlueprintType, meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EDMMaterialFlagsLighting : uint8
{
	None                              = 0      UMETA(HIdden),
	CastDynamicShadowAsMasked         = 1 << 0 UMETA(ToolTip = "Whether the material should cast shadows as masked even though it has a translucent blend mode."),
	ContactShadows                    = 1 << 1 UMETA(ToolTip = "Contact shadows on translucency."),
	UseEmissiveForDynamicAreaLighting = 1 << 2 UMETA(ToolTip = "Whether the material's emissive colour is injected into the LightPropagationVolume."),
	CompatibleWithLumenCardSharing    = 1 << 3 UMETA(ToolTip = "Whether to allow to share Lumen Cards between different instances even when material uses world position or per instance data, which may change material look per instance. All materials on a component needs this flag set for sharing to work."),
	AllowTranslucentLocalLightShadow  = 1 << 4 UMETA(ToolTip = "Allows a translucent material to receive local light shadows."),
	UseLightmapDirectionality         = 1 << 5 UMETA(ToolTip = "Use lightmap directionality and per pixel normals. If disabled, lighting from lightmaps will be flat but cheaper."),
	AllowNegativeEmissiveColor        = 1 << 6 UMETA(ToolTip = "Whether the material should allow outputting negative emissive color values.  Only allowed on unlit materials.")
};
ENUM_CLASS_FLAGS(EDMMaterialFlagsLighting)

UENUM(BlueprintType, meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EDMMaterialFlagsTranslucency : uint8
{
	None                              = 0      UMETA(HIdden),
	DisableDepthTest                  = 1 << 0 UMETA(ToolTip = "Whether to draw on top of opaque pixels even if behind them. This only has meaning for translucency."),
	ScreenSpaceReflections            = 1 << 1 UMETA(ToolTip = "SSR on translucency."),
	AllowTranslucentCustomDepthWrites = 1 << 2 UMETA(ToolTip = "Allows a translucent material to be used with custom depth writing by compiling additional shaders."),
	AllowFrontLayerTranslucency       = 1 << 3 UMETA(ToolTip = "Allows a translucent material to be used with Front Layer Translucency by compiling additional shaders. Useful for controlling what should be included in Front Layer Translucency."),
	WriteOnlyAlpha                    = 1 << 4 UMETA(ToolTip = "Whether the transluency pass should write its alpha, and only the alpha, into the framebuffer."),
	DitherOpacityMask                 = 1 << 5 UMETA(ToolTip = "Dither opacity mask. When combined with Temporal AA this can be used as a form of limited translucency which supports all lighting features.")
};
ENUM_CLASS_FLAGS(EDMMaterialFlagsTranslucency)

UENUM(BlueprintType, meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EDMMaterialFlagsMotion : uint8
{
	None                      = 0      UMETA(HIdden),
	PixelAnimation            = 1 << 0 UMETA(ToolTip = "Whether the opaque material has any pixel animations happening, that isn't included in the geometric velocities. This allows to disable renderer's heuristics that assumes animation is fully described with motion vector, such as TSR's anti-flickering heuristic."),
	ResponsiveAA              = 1 << 1 UMETA(ToolTip = "Indicates that the material should be rendered using responsive anti-aliasing. Improves sharpness of small moving particles such as sparks. Only use for small moving features because it will cause aliasing of the background."),
	OutputTranslucentVelocity = 1 << 2 UMETA(ToolTip = "Whether ranslucent materials will output motion vectors and write to depth buffer in velocity pass.")
};
ENUM_CLASS_FLAGS(EDMMaterialFlagsMotion)

UENUM(BlueprintType, meta = (BitFlags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EDMMaterialFlagsForwardRenderer : uint8
{
	None                                        = 0      UMETA(HIdden),
	MobileSeparateTranslucency                  = 1 << 0 UMETA(ToolTip = "Indicates that the translucent material should not be affected by bloom or DOF. (Note: Depth testing is not available)."),
	AlphaToCoverage                             = 1 << 1 UMETA(ToolTip = "Use alpha to coverage for masked material on mobile, make sure MSAA is enabled as well."),
	ForwardRenderUsePreintegratedGFForSimpleIBL = 1 << 2 UMETA(ToolTip = "Use preintegrated GF lut for simple IBL, but will use one more sampler."),
	HighQualityForwardReflections               = 1 << 3 UMETA(ToolTip = "Enables multiple parallax-corrected reflection captures that blend together."),
	ForwardBlendsSkyLightCubemaps               = 1 << 4 UMETA(ToolTip = "Enables blending of sky light cubemap textures. When disabled, the secondary cubemap is only visible when the blend factor is 1."),
	PlanarForwardReflections                    = 1 << 5 UMETA(ToolTip = "Enabling this setting reduces the number of samplers available to the material as one more sampler will be used for the planar reflection.")
};
ENUM_CLASS_FLAGS(EDMMaterialFlagsForwardRenderer)

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, DefaultToInstanced, ClassGroup = "Material Designer")
class UDynamicMaterialModelEditorOnlyData : public UObject, public IDynamicMaterialModelEditorOnlyDataInterface, public FNotifyHook, public IDMBuildable
{
	GENERATED_BODY()

	friend class UDynamicMaterialModelFactory;
	friend class FDMMaterialModelPropertyRowGenerator;

public:
	DYNAMICMATERIALEDITOR_API static const FString SlotsPathToken;
	DYNAMICMATERIALEDITOR_API static const FString BaseColorSlotPathToken;
	DYNAMICMATERIALEDITOR_API static const FString EmissiveSlotPathToken;
	DYNAMICMATERIALEDITOR_API static const FString OpacitySlotPathToken;
	DYNAMICMATERIALEDITOR_API static const FString RoughnessPathToken;
	DYNAMICMATERIALEDITOR_API static const FString SpecularPathToken;
	DYNAMICMATERIALEDITOR_API static const FString MetallicPathToken;
	DYNAMICMATERIALEDITOR_API static const FString NormalPathToken;
	DYNAMICMATERIALEDITOR_API static const FString PixelDepthOffsetPathToken;
	DYNAMICMATERIALEDITOR_API static const FString WorldPositionOffsetPathToken;
	DYNAMICMATERIALEDITOR_API static const FString AmbientOcclusionPathToken;
	DYNAMICMATERIALEDITOR_API static const FString AnisotropyPathToken;
	DYNAMICMATERIALEDITOR_API static const FString RefractionPathToken;
	DYNAMICMATERIALEDITOR_API static const FString TangentPathToken;
	DYNAMICMATERIALEDITOR_API static const FString DisplacementPathToken;
	DYNAMICMATERIALEDITOR_API static const FString SubsurfaceColorPathToken;
	DYNAMICMATERIALEDITOR_API static const FString SurfaceThicknessPathToken;
	DYNAMICMATERIALEDITOR_API static const FString Custom1PathToken;
	DYNAMICMATERIALEDITOR_API static const FString Custom2PathToken;
	DYNAMICMATERIALEDITOR_API static const FString Custom3PathToken;
	DYNAMICMATERIALEDITOR_API static const FString Custom4PathToken;
	DYNAMICMATERIALEDITOR_API static const FString PropertiesPathToken;

	DYNAMICMATERIALEDITOR_API static const TArray<EMaterialDomain> SupportedDomains;
	DYNAMICMATERIALEDITOR_API static const TArray<EBlendMode> SupportedBlendModes;

	DYNAMICMATERIALEDITOR_API static const FName AlphaValueName;

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(UDynamicMaterialModelBase* InModelBase);
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(const TWeakObjectPtr<UDynamicMaterialModelBase>& InModelBaseWeak);
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(UDynamicMaterialModel* InModel);
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(const TWeakObjectPtr<UDynamicMaterialModel>& InModelWeak);
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(const TScriptInterface<IDynamicMaterialModelEditorOnlyDataInterface>& InInterface);
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(IDynamicMaterialModelEditorOnlyDataInterface* InInterface);
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelEditorOnlyData* Get(UDynamicMaterialInstance* InInstance);

	UDynamicMaterialModelEditorOnlyData();

	void Initialize();

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UDynamicMaterialModel* GetMaterialModel() const { return MaterialModel; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UMaterial* GetGeneratedMaterial() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMState GetState() const { return State; }

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	TEnumAsByte<EMaterialDomain> GetDomain() const { return Domain; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetDomain(TEnumAsByte<EMaterialDomain> InDomain);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	TEnumAsByte<EBlendMode> GetBlendMode() const { return BlendMode; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetBlendMode(TEnumAsByte<EBlendMode> InBlendMode);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	EDMMaterialShadingModel GetShadingModel() const { return ShadingModel; }

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetShadingModel(EDMMaterialShadingModel InShadingModel);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialFlagsUsage GetUsageFlags() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasUsageFlag(EDMMaterialFlagsUsage InFlag) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetUsageFlag(EDMMaterialFlagsUsage InFlag, bool bInSet);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialFlagsGeneral GetGeneralFlags() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasGeneralFlag(EDMMaterialFlagsGeneral InFlag) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetGeneralFlag(EDMMaterialFlagsGeneral InFlag, bool bInSet);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialFlagsLighting GetLightingFlags() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasLightingFlag(EDMMaterialFlagsLighting InFlag) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetLightingFlag(EDMMaterialFlagsLighting InFlag, bool bInSet);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialFlagsTranslucency GetTranslucencyFlags() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasTranslucencyFlag(EDMMaterialFlagsTranslucency InFlag) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetTranslucencyFlag(EDMMaterialFlagsTranslucency InFlag, bool bInSet);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialFlagsMotion GetMotionFlags() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasMotionFlag(EDMMaterialFlagsMotion InFlag) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetMotionFlag(EDMMaterialFlagsMotion InFlag, bool bInSet);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API EDMMaterialFlagsForwardRenderer GetForwardRendererFlags() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool HasForwardRendererFlag(EDMMaterialFlagsForwardRenderer InFlag) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SetForwardRendererFlag(EDMMaterialFlagsForwardRenderer InFlag, bool bInSet);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API float GetOpacityMaskClipValue() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetOpacityMaskClipValue(float InClipValue);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API float GetDisplacementCenter() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetDisplacementCenter(float InCenter);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API float GetDisplacementMagnitude() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetDisplacementMagnitude(float InMagnitude);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void SetChannelListPreset(FName InPresetName);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const FMaterialStatistics& GetMaterialStats() const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void OpenMaterialEditor() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TMap<EDMMaterialPropertyType, UDMMaterialProperty*> GetMaterialProperties() const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialProperty* GetMaterialProperty(EDMMaterialPropertyType InMaterialProperty) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	const TArray<UDMMaterialSlot*>& GetSlots() const { return Slots; }

	/** Gets slot by index. Highly recommended to use GetSlotForMaterialProperty(PropertyType). */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* GetSlot(int32 Index) const;

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* GetSlotForMaterialProperty(EDMMaterialPropertyType InType) const;

	/** Same as the above method, but will only return the slot if the material property is enabled. */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* GetSlotForEnabledMaterialProperty(EDMMaterialPropertyType InType) const;

	/** Adds the next available slot. Highly recommended to use AddSlotForMaterialProperty(PropertyType). */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* AddSlot();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* AddSlotForMaterialProperty(EDMMaterialPropertyType InType);

	/** Removes the next slot by index. Highly recommended to use RemoveSlotForMaterialProperty(PropertyType). */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* RemoveSlot(int32 Index);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* RemoveSlotForMaterialProperty(EDMMaterialPropertyType InType);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TArray<EDMMaterialPropertyType> GetMaterialPropertiesForSlot(const UDMMaterialSlot* InSlot) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void AssignMaterialPropertyToSlot(EDMMaterialPropertyType InProperty, UDMMaterialSlot* InSlot);

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API void UnassignMaterialProperty(EDMMaterialPropertyType InProperty);

	UFUNCTION(BlueprintPure, Category = "Material Designer")
	bool HasBuildBeenRequested() const;
	
	FDMOnMaterialBuilt::RegistrationType& GetOnMaterialBuiltDelegate() { return OnMaterialBuiltDelegate; }
	FDMOnValueListUpdated::RegistrationType& GetOnValueListUpdateDelegate() { return OnValueListUpdateDelegate; }
	FDMOnSlotListUpdated::RegistrationType& GetOnSlotListUpdateDelegate() { return OnSlotListUpdateDelegate; }
	FDMOnPropertyUpdated::RegistrationType& GetOnPropertyUpdateDelegate() { return OnPropertyUpdateDelegate; }

	void OnPropertyUpdate(UDMMaterialProperty* InProperty);

	DYNAMICMATERIALEDITOR_API TSharedRef<FDMMaterialBuildState> CreateBuildState(UMaterial* InMaterialToBuild, bool bInDirtyAssets = true) const;

	bool NeedsWizard() const;

	DYNAMICMATERIALEDITOR_API void OnWizardComplete();

	void SaveEditor();

	//~ Begin FNotifyHook
	DYNAMICMATERIALEDITOR_API virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, class FEditPropertyChain* InPropertyThatChanged);
	//~ End FNotifyHook

	//~ Begin UObject
	DYNAMICMATERIALEDITOR_API virtual void PostLoad() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditUndo() override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditImport() override;
	DYNAMICMATERIALEDITOR_API virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	DYNAMICMATERIALEDITOR_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	DYNAMICMATERIALEDITOR_API virtual void Serialize(FArchive& InAr) override;
	//~ End UObject

	//~ Begin IDMBuildable
	virtual void DoBuild_Implementation(bool bInDirtyAssets) override;
	//~ End IDMBuildable

	//~ Begin IDynamicMaterialModelEditorOnlyDataInterface
	DYNAMICMATERIALEDITOR_API virtual void PostEditorDuplicate() override;
	DYNAMICMATERIALEDITOR_API virtual void RequestMaterialBuild(EDMBuildRequestType InRequestType = EDMBuildRequestType::Preview) override;
	DYNAMICMATERIALEDITOR_API virtual void OnValueListUpdate() override;
	DYNAMICMATERIALEDITOR_API virtual void OnValueUpdated(UDMMaterialValue* InValue, EDMUpdateType InUpdateType) override;
	DYNAMICMATERIALEDITOR_API virtual void OnTextureUVUpdated(UDMTextureUV* InTextureUV) override;
	DYNAMICMATERIALEDITOR_API virtual TSharedRef<IDMMaterialBuildStateInterface> CreateBuildStateInterface(UMaterial* InMaterialToBuild) const override;
	DYNAMICMATERIALEDITOR_API virtual void SetPropertyComponent(EDMMaterialPropertyType InPropertyType, FName InComponentName, UDMMaterialComponent* InComponent) override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath) const override;
	DYNAMICMATERIALEDITOR_API virtual UDMMaterialComponent* GetSubComponentByPath(FDMComponentPath& InPath, const FDMComponentPathSegment& InPathSegment) const override;
	//~ End IDynamicMaterialModelEditorOnlyDataInterface

	/*
	 * Deprecated
	 */

	UE_DEPRECATED(5.8, "Use HasMotionFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use HasMotionFlag()."))
	DYNAMICMATERIALEDITOR_API bool GetHasPixelAnimation() const;

	UE_DEPRECATED(5.8, "Use SetMotionFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetMotionFlag()."))
	DYNAMICMATERIALEDITOR_API void SetHasPixelAnimation(bool bInHasAnimation);

	UE_DEPRECATED(5.8, "Use HasGeneralFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use HasGeneralFlag()."))
	DYNAMICMATERIALEDITOR_API bool GetIsTwoSided() const;

	UE_DEPRECATED(5.8, "Use SetGeneralFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetGeneralFlag()."))
	DYNAMICMATERIALEDITOR_API void SetIsTwoSided(bool bInEnabled);

	UE_DEPRECATED(5.8, "Use HasMotionFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use HasMotionFlag()."))
	DYNAMICMATERIALEDITOR_API bool IsOutputTranslucentVelocityEnabled() const;

	UE_DEPRECATED(5.8, "Use SetMotionFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetMotionFlag()."))
	DYNAMICMATERIALEDITOR_API void SetOutputTranslucentVelocityEnabled(bool bInEnabled);

	UE_DEPRECATED(5.8, "Use HasUsageFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use HasUsageFlag()."))
	DYNAMICMATERIALEDITOR_API bool IsNaniteTessellationEnabled() const;

	UE_DEPRECATED(5.8, "Use SetUsageFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetUsageFlag()."))
	DYNAMICMATERIALEDITOR_API void SetNaniteTessellationEnabled(bool bInEnabled);

	UE_DEPRECATED(5.8, "Use HasMotionFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use HasMotionFlag()."))
	DYNAMICMATERIALEDITOR_API bool IsResponsiveAAEnabled() const;

	UE_DEPRECATED(5.8, "Use SetMotionFlag().")
	UFUNCTION(meta = (DeprecatedFunction = "Note", DeprecationMessage = "Use SetMotionFlag()."))
	DYNAMICMATERIALEDITOR_API void SetResponsiveAAEnabled(bool bInEnabled);

protected:
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	TObjectPtr<UDynamicMaterialModel> MaterialModel;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, TextExportTransient, Category = "Material Designer")
	EDMState State;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer", 
		meta = (NotKeyframeable, ValidEnumValues = "MD_Surface,MD_PostProcess,MD_DeferredDecal,MD_LightFunction"))
	TEnumAsByte<EMaterialDomain> Domain;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer",
		meta = (NotKeyframeable, ValidEnumValues = "BLEND_Opaque,BLEND_Translucent,BLEND_Masked,BLEND_Additive,BLEND_Modulate"))
	TEnumAsByte<EBlendMode> BlendMode;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer", 
		meta = (NotKeyframeable, ValidEnumValues = "Unlit,DefaultLit"))
	EDMMaterialShadingModel ShadingModel;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|Usage",
		meta = (NotKeyframeable, Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsUsage"))
	int32 UsageFlags;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|General",
		meta = (NotKeyframeable, Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsGeneral"))
	uint8 GeneralFlags;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|Lighting",
		meta = (NotKeyframeable, Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsLighting",
			EditCondition = "ShadingModel == EDMMaterialShadingModel::DefaultLit"))
	uint8 LightingFlags;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|Translucency",
		meta = (NotKeyframeable, Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsTranslucency",
			EditCondition = "BlendMode == EBlendMode::BLEND_Masked || BlendMode == EBlendMode::BLEND_Translucent"))
	uint8 TranslucencyFlags;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|Motion",
		meta = (NotKeyframeable, Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsMotion"))
	uint8 MotionFlags;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|Forward Renderer",
		meta = (NotKeyframeable, Bitmask, BitmaskEnum = "/Script/DynamicMaterialEditor.EDMMaterialFlagsForwardRenderer"))
	uint8 ForwardRendererFlags;

	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|Translucency",
		meta = (NotKeyframeable, UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1,
			EditCondition = "BlendMode == EBlendMode::BLEND_Masked"))
	float OpacityMaskClipValue;

	/** Mid point for displacement in the range 0-1. */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|General", 
		meta = (NotKeyframeable, UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1,
			EditCondition = "IsNaniteEnabled()"))
	float DisplacementCenter;

	/** Multipler for displacement values. */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Category = "Material Designer|General", 
		meta = (NotKeyframeable, EditCondition = "IsNaniteEnabled()"))
	float DisplacementMagnitude;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, TextExportTransient, Category = "Material Designer")
	TMap<EDMMaterialPropertyType, TObjectPtr<UDMMaterialSlot>> PropertySlotMap;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UDMMaterialSlot>> Slots;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = "Material Designer")
	TArray<TObjectPtr<UMaterialExpression>> Expressions;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	bool bCreateMaterialPackage;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = "Material Designer")
	FMaterialStatistics MaterialStats;

	FDMOnMaterialBuilt OnMaterialBuiltDelegate;
	FDMOnValueListUpdated OnValueListUpdateDelegate;
	FDMOnSlotListUpdated OnSlotListUpdateDelegate;
	FDMOnPropertyUpdated OnPropertyUpdateDelegate;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> BaseColor;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> EmissiveColor;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Opacity;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> OpacityMask;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Roughness;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Specular;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Metallic;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Normal;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> PixelDepthOffset;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> WorldPositionOffset;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> AmbientOcclusion;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Anisotropy;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Refraction;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Tangent;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Displacement;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> SubsurfaceColor;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> SurfaceThickness;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Custom1;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Custom2;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Custom3;

	UPROPERTY(BlueprintReadOnly, Instanced, Category = "Material Designer")
	TObjectPtr<UDMMaterialProperty> Custom4;

	bool bBuildRequested;

	void CreateMaterial();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	void BuildMaterial(bool bInDirtyAssets);

	void SetMaterialFlags(UMaterial* InMaterial);

	FString GetMaterialAssetPath() const;
	FString GetMaterialAssetName() const;
	FString GetMaterialPackageName(const FString& InMaterialBaseName) const;

	void OnSlotConnectorsUpdated(UDMMaterialSlot* InSlot);

	/** Swaps the material properties from one slot to another, unless both slots exist and/or are the same. */
	void SwapSlotMaterialProperty(EDMMaterialPropertyType InPropertyFrom, EDMMaterialPropertyType InPropertyTo);

	/** 
	 * Swaps the material properties from one slot to another, unless both slots exist and/or are the same. 
	 * Ensuring that the To Property exists.
	 */
	void EnsureSwapSlotMaterialProperty(EDMMaterialPropertyType InPropertyFrom, EDMMaterialPropertyType InPropertyTo);

	void AssignPropertyAlphaValues();

	void OnDomainChanged();
	void OnBlendModeChanged();
	void OnShadingModelChanged();
	void OnMaterialFlagChanged();
	void OnMaterialSettingsChanged();

	//~ Begin IDynamicMaterialModelEditorOnlyDataInterface
	virtual void ReinitComponents() override;
	//~ End IDynamicMaterialModelEditorOnlyDataInterface

	/*
	 * Deprecated
	 */

	void ConvertDeprecatedVarsToFlags();

	UE_DEPRECATED(5.8, "Moved to GeneralFlags")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Moved to GeneralFlags"))
	bool bTwoSided_DEPRECATED;

	UE_DEPRECATED(5.8, "Moved to MotionFlags")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Moved to MotionFlags"))
	bool bHasPixelAnimation_DEPRECATED;

	UE_DEPRECATED(5.8, "Moved to MotionFlags")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Moved to MotionFlags"))
	bool bOutputTranslucentVelocityEnabled_DEPRECATED;

	UE_DEPRECATED(5.8, "Moved to UsageFlags")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Moved to UsageFlags"))
	bool bNaniteTessellationEnabled_DEPRECATED;

	UE_DEPRECATED(5.8, "Moved to MotionFlags")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Moved to MotionFlags"))
	bool bResponsiveAAEnabled_DEPRECATED;

private:
	UFUNCTION()
	bool IsNaniteEnabled() const
	{
		return UsageFlags & EDMMaterialFlagsUsage::DMMFU_Nanite;
	}
};
