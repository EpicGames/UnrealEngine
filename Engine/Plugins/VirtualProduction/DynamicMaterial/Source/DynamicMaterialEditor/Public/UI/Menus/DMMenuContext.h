// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Templates/SharedPointer.h"

#include "DMMenuContext.generated.h"

class FName;
class SDMMaterialEditor;
class SDMMaterialStage;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialStageBlend;
class UDMMaterialStageSource;
class UDynamicMaterialModel;
class UDynamicMaterialModelBase;
class UToolMenu;

UCLASS(MinimalAPI)
class UDMMenuContext : public UObject
{
	GENERATED_BODY()

public:
	static UDMMenuContext* CreateEmpty();

	static UDMMenuContext* CreateEditor(const TWeakPtr<SDMMaterialEditor>& InEditorWidget);

	static UDMMenuContext* CreateLayer(const TWeakPtr<SDMMaterialEditor>& InEditorWidget, UDMMaterialLayerObject* InLayerObject);

	static UDMMenuContext* CreateStage(const TWeakPtr<SDMMaterialEditor>& InEditorWidget, const TWeakPtr<SDMMaterialStage>& InStageWidget);

	static UToolMenu* GenerateContextMenuDefault(const FName InMenuName);

	static UToolMenu* GenerateContextMenuEditor(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget);

	static UToolMenu* GenerateContextMenuLayer(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget, 
		UDMMaterialLayerObject* InLayerObject);

	static UToolMenu* GenerateContextMenuStage(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget, 
		const TWeakPtr<SDMMaterialStage>& InStageWidget);

	const TSharedPtr<SDMMaterialEditor> GetEditorWidget() const { return EditorWidgetWeak.Pin(); }

	const TSharedPtr<SDMMaterialStage> GetStageWidget() const { return StageWidgetWeak.Pin(); }

	DYNAMICMATERIALEDITOR_API UDMMaterialSlot* GetSlot() const;

	DYNAMICMATERIALEDITOR_API UDynamicMaterialModelBase* GetPreviewModelBase() const;

	DYNAMICMATERIALEDITOR_API UDynamicMaterialModel* GetPreviewModel() const;

	DYNAMICMATERIALEDITOR_API UDMMaterialStage* GetStage() const;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageSource* GetStageSource() const;

	DYNAMICMATERIALEDITOR_API UDMMaterialStageBlend* GetStageSourceAsBlend() const;

	DYNAMICMATERIALEDITOR_API UDMMaterialLayerObject* GetLayer() const;

protected:
	TWeakPtr<SDMMaterialEditor> EditorWidgetWeak;
	TWeakPtr<SDMMaterialStage> StageWidgetWeak;
	TWeakObjectPtr<UDMMaterialLayerObject> LayerObjectWeak;

	static UDMMenuContext* Create(const TWeakPtr<SDMMaterialEditor>& InEditorWidget, const TWeakPtr<SDMMaterialStage>& InStageWidget,
		UDMMaterialLayerObject* InLayerObject);

	static UToolMenu* GenerateContextMenu(const FName InMenuName, const TWeakPtr<SDMMaterialEditor>& InEditorWidget, 
		const TWeakPtr<SDMMaterialStage>& InStageWidget, UDMMaterialLayerObject* InLayerObject);
};
