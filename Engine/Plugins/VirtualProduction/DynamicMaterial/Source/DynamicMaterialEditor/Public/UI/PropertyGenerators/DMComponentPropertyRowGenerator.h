// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "UI/Utils/IDMWidgetLibrary.h"

class FName;
class FNotifyHook;
class FProperty;
class SDMMaterialComponentEditor;
class UDMMaterialComponent;
enum class EDMPropertyHandlePriority : uint8;
struct FDMComponentPropertyRowGeneratorParams;
struct FDMPropertyHandle;

/** Used to generate editable properties for objects editable in the Material Designer. */
class FDMComponentPropertyRowGenerator
{
public:
	DYNAMICMATERIALEDITOR_API static const TSharedRef<FDMComponentPropertyRowGenerator>& Get();

	virtual ~FDMComponentPropertyRowGenerator() = default;

	/**
	 * Generate properties for the given component.
	 * @param InComponentEditorWidget The edit widget generating the properties.
	 * @param InComponent The component being edited.
	 * @param InOutPropertyRows The generated rows.
	 * @param InOutProcessedObjects The already processed objects - add to this to avoid possible recursive generation.
	 */
	DYNAMICMATERIALEDITOR_API virtual void AddComponentProperties(FDMComponentPropertyRowGeneratorParams& InParams);

	/**
	 * Add the rows needed for a specific property.
	 * @param InProperty The name of the property.
	 */
	DYNAMICMATERIALEDITOR_API virtual void AddPropertyEditRows(FDMComponentPropertyRowGeneratorParams& InParams,
		const FName& InProperty);

	/**
	 * Adds the rows needed for a specific property.
	 * @param InProperty The FProperty for this property.
	 * @param InMemoryPtr The pointer to the property's value in memory
	 */
	DYNAMICMATERIALEDITOR_API virtual void AddPropertyEditRows(FDMComponentPropertyRowGeneratorParams& InParams,
		FProperty* InProperty, void* InMemoryPtr);

	DYNAMICMATERIALEDITOR_API virtual bool AllowKeyframeButton(UDMMaterialComponent* InComponent, FProperty* InProperty);

protected:
	/** Returns true if the component edit widget is editing a Material Designer Dynamic. */
	DYNAMICMATERIALEDITOR_API static bool IsDynamic(FDMComponentPropertyRowGeneratorParams& InParams);
};
