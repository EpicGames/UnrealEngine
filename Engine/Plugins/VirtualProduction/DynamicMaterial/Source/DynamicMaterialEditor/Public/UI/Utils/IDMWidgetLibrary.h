// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "UObject/NameTypes.h"

class FNotifyHook;
class SDMMaterialComponentEditor;
class SWidget;
class UDynamicMaterialModelBase;
class UObject;
struct FDMPropertyHandle;

struct FDMPropertyHandleGenerateParams
{
	const SWidget* Widget;
	FNotifyHook* NotifyHook;
	UDynamicMaterialModelBase* PreviewMaterialModelBase;
	UDynamicMaterialModelBase* OriginalMaterialModelBase;
	UObject* Object;
	FName PropertyName;
};

class IDMWidgetLibrary
{
public:
	virtual ~IDMWidgetLibrary() = default;

	DYNAMICMATERIALEDITOR_API static IDMWidgetLibrary& Get();

	// Creates a DM property handle for use in the details panel section of the Material Designer
	virtual FDMPropertyHandle GetPropertyHandle(const FDMPropertyHandleGenerateParams& InParams) = 0;
};
