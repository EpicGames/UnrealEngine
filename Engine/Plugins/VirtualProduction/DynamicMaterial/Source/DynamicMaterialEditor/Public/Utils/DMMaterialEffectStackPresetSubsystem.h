// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "DMMaterialEffectStackPresetSubsystem.generated.h"

class UDMMaterialEffect;
class UDMMaterialEffectStack;
struct FDMMaterialEffectStackJson;

UCLASS(MinimalAPI, BlueprintType)
class UDMMaterialEffectStackPresetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static UDMMaterialEffectStackPresetSubsystem* Get();

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool SavePreset(const FString& InPresetName, const FDMMaterialEffectStackJson& InPreset) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool LoadPreset(const FString& InPresetName, FDMMaterialEffectStackJson& OutPreset) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API bool RemovePreset(const FString& InPresetName) const;

	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	DYNAMICMATERIALEDITOR_API TArray<FString> GetPresetNames() const;
};
