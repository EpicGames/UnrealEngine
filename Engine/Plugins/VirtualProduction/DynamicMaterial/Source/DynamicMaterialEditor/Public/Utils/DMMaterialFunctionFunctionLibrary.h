// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Templates/SharedPointerFwd.h"

#include "DMMaterialFunctionFunctionLibrary.generated.h"

class IPropertyHandle;
class UDMMaterialValue;
class UDMMaterialValueTexture;
class UMaterialExpressionFunctionInput;
enum class EDMValueType : uint8;
struct FFunctionExpressionInput;

/**
 * Material Function Function Library
 */
UCLASS()
class UDMMaterialFunctionFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static void ApplyMetaData(const FFunctionExpressionInput& InFunctionInput, 
		const TSharedRef<IPropertyHandle>& InPropertyHandle);

	DYNAMICMATERIALEDITOR_API static EDMValueType GetInputValueType(UMaterialExpressionFunctionInput* InFunctionInput);

	DYNAMICMATERIALEDITOR_API static void SetInputDefault(UMaterialExpressionFunctionInput* InFunctionInput, UDMMaterialValue* InValue);

private:
	static void SetInputDefault_Texture(UMaterialExpressionFunctionInput* InFunctionInput, UDMMaterialValueTexture* InTextureValue);
};
