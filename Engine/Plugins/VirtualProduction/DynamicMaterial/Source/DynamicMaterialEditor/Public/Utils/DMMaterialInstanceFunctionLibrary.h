// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Containers/ContainersFwd.h"
#include "DMEDefs.h"

#include "DMMaterialInstanceFunctionLibrary.generated.h"

class AActor;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
class UDynamicMaterialModelBase;
class UMaterial;
struct FDMObjectMaterialProperty;

/**
 * Material Instance Function Library
 */
UCLASS()
class UDMMaterialInstanceFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static TArray<FDMObjectMaterialProperty> GetActorMaterialProperties(AActor* InActor);

	DYNAMICMATERIALEDITOR_API static bool SetMaterialInObject(FDMObjectMaterialProperty& InMaterialProperty, UDynamicMaterialInstance* InInstance);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModel* CreateMaterialInObject(FDMObjectMaterialProperty& InMaterialProperty);
};
