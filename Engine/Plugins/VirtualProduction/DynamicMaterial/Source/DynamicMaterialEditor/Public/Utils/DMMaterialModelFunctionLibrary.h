// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "DMEDefs.h"

#include "DMMaterialModelFunctionLibrary.generated.h"

class FString;
class UDynamicMaterialModelBase;
class UDynamicMaterialModelDynamic;
class UMaterial;

/**
 * Material / Model Function Library
 */
UCLASS()
class UDMMaterialModelFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static UDynamicMaterialInstance* ExportMaterial(UDynamicMaterialModelBase* InMaterialModelBase);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialInstance* ExportMaterial(UDynamicMaterialModelBase* InMaterialModel, const FString& InSavePath);

	DYNAMICMATERIALEDITOR_API static UMaterial* ExportGeneratedMaterial(UDynamicMaterialModelBase* InMaterialModelBase);

	DYNAMICMATERIALEDITOR_API static UMaterial* ExportGeneratedMaterial(UDynamicMaterialModelBase* InMaterialModelBase, const FString& InSavePath);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModel* ExportToTemplateMaterialModel(UDynamicMaterialModelDynamic* InMaterialModelDynamic);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModel* ExportToTemplateMaterialModel(UDynamicMaterialModelDynamic* InMaterialModelDynamic, const FString& InSavePath);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialInstance* ExportToTemplateMaterial(UDynamicMaterialModelDynamic* InMaterialModelDynamic);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialInstance* ExportToTemplateMaterial(UDynamicMaterialModelDynamic* InMaterialModelDynamic, const FString& InSavePath);

	DYNAMICMATERIALEDITOR_API static bool IsModelValid(UDynamicMaterialModelBase* InMaterialModelBase);

	DYNAMICMATERIALEDITOR_API static bool DuplicateModelBetweenMaterials(UDynamicMaterialModel* InFromModel, UDynamicMaterialInstance* InToInstance);

	DYNAMICMATERIALEDITOR_API static bool CreateModelInstanceInMaterial(UDynamicMaterialModel* InFromModel, UDynamicMaterialInstance* InToInstance);

	DYNAMICMATERIALEDITOR_API static FString RemoveAssetPrefix(const FString& InAssetName);

	DYNAMICMATERIALEDITOR_API static UDynamicMaterialModelBase* CreatePreviewModel(UDynamicMaterialModelBase* InOriginalModelBase);

	/**
	 * Ensures that the source model is entirely mirrored to the target model by updating
	 * objects, rather than recreating them. Additions and removals are also propagated.
	 */
	DYNAMICMATERIALEDITOR_API static void MirrorMaterialModel(UDynamicMaterialModelBase* InSource, UDynamicMaterialModelBase*& InTarget);

	DYNAMICMATERIALEDITOR_API static UObject* FindSubobject(UObject* InOuter, FStringView InPath);

	template<typename InClassName>
	static InClassName* FindSubobject(UObject* InOuter, FStringView InPath)
	{
		return Cast<InClassName>(FindSubobject(InOuter, InPath));
	}
};
