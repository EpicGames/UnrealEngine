// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Templates/SubclassOf.h"

#include "DMMaterialSlotFunctionLibrary.generated.h"

class SDMMaterialSlot;
class UDMMaterialLayerObject;
class UDMMaterialSlot;
class UDMMaterialStage;
class UDMMaterialStageBlend;
class UDMMaterialStageExpression;
class UDMMaterialStageGradient;
class UDMMaterialValue;
class UDMRenderTargetRenderer;
class UMaterialFunctionInterface;
class UTexture;
enum class EDMMaterialLayerStage : uint8;
enum class EDMMaterialPropertyType : uint8;
enum class EDMValueType : uint8;

/**
 * Material Slot Function Library
 */
UCLASS()
class UDMMaterialSlotFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_NewLocalValue(UDMMaterialSlot* InSlot, EDMValueType InValueType);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_NewLocalValue(UDMMaterialSlot* InSlot, 
		TSubclassOf<UDMMaterialValue> InValueClass);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_GlobalValue(UDMMaterialSlot* InSlot, UDMMaterialValue* InValue);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_NewGlobalValue(UDMMaterialSlot* InSlot, EDMValueType InValueType);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_NewGlobalValue(UDMMaterialSlot* InSlot, 
		TSubclassOf<UDMMaterialValue> InValueClass);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_Slot(UDMMaterialSlot* InTargetSlot, UDMMaterialSlot* InSourceSlot, 
		EDMMaterialPropertyType InMaterialProperty);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_Expression(UDMMaterialSlot* InSlot, 
		TSubclassOf<UDMMaterialStageExpression> InExpressionClass);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_Blend(UDMMaterialSlot* InSlot, 
		TSubclassOf<UDMMaterialStageBlend> InBlendClass);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_Gradient(UDMMaterialSlot* InSlot, 
		TSubclassOf<UDMMaterialStageGradient> InGradientClass);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_UV(UDMMaterialSlot* InSlot);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_MaterialFunction(UDMMaterialSlot* InSlot, 
		UMaterialFunctionInterface* InFunction = nullptr);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_SceneTexture(UDMMaterialSlot* InSlot);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer_Renderer(UDMMaterialSlot* InSlot, 
		TSubclassOf<UDMRenderTargetRenderer> InRendererClass);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddNewLayer(UDMMaterialSlot* InSlot, UDMMaterialStage* InNewBaseStage = nullptr, 
		UDMMaterialStage* InNewMaskStage = nullptr);

	DYNAMICMATERIALEDITOR_API static UDMMaterialLayerObject* AddTextureLayer(UDMMaterialSlot* InSlot, UTexture* InTexture, 
		EDMMaterialPropertyType InPropertyType, bool bInReplaceSlot);
};
