// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Templates/SubclassOf.h"

#include "DMMaterialStageFunctionLibrary.generated.h"

class UDMMaterialStage;
class UDMMaterialStageInputValue;
class UDMRenderTargetRenderer;

/**
 * Material Stage Function Library
 */
UCLASS()
class UDMMaterialStageFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	DYNAMICMATERIALEDITOR_API static UDMMaterialStageInputValue* FindDefaultStageOpacityInputValue(UDMMaterialStage* InStage);

	DYNAMICMATERIALEDITOR_API static void SetStageInputToRenderer(UDMMaterialStage* InStage, TSubclassOf<UDMRenderTargetRenderer> InRendererClass, int32 InInputIndex);
};
