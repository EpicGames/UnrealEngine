// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "DMTextureSetFunctionLibrary.generated.h"

class UDynamicMaterialModelEditorOnlyData;
class UDMTextureSet;

/**
 * Material Stage Function Library
 */
UCLASS()
class UDMTextureSetFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Integrates a Texture Set with the given model's editor only data.
	 * @param InEditorOnlyData The editor only data for the model.
	 * @param InTextureSet The set to integrate.
	 * @param bInReplaceSlots Whether to add to or completely replace slots.
	 * @return True on success.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALEDITOR_API bool AddTextureSetToModel(UDynamicMaterialModelEditorOnlyData* InEditorOnlyData, UDMTextureSet* InTextureSet,
		bool bInReplaceSlots);
};
