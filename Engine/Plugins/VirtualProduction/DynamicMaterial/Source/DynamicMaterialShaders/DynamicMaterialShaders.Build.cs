// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterialShaders : ModuleRules
{
	public DynamicMaterialShaders(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"RenderCore"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Projects",
				"Renderer",
				"RHI"
			}
		);

		ShortName = "DynMatEdShdrs";
	}
}
