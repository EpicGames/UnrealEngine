// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMAlphaOneMinusPS.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DynamicMaterialShadersModule.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "UObject/Package.h"

const FString FDMAlphaOneMinusPS::ShaderPath = FString(UE::DynamicMaterialShaders::Internal::VirtualShaderMountPoint) + TEXT("/DMAlphaOneMinusPS.usf");

bool FDMAlphaOneMinusPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
{
	return IsFeatureLevelSupported(InParameters.Platform, ERHIFeatureLevel::ES3_1);
}

FDMAlphaOneMinusPS::FParameters* FDMAlphaOneMinusPS::AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, FRDGTextureRef InRGBATexture, FRDGTextureRef InOutputTexture)
{
	FParameters* Parameters = InGraphBuilder.AllocParameters<FParameters>();
	Parameters->InputTexture = InRGBATexture;
	Parameters->InputSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Parameters->RenderTargets[0] = FRenderTargetBinding{InOutputTexture, ERenderTargetLoadAction::ENoAction};
	return Parameters;
}

IMPLEMENT_SHADER_TYPE(, FDMAlphaOneMinusPS, *FDMAlphaOneMinusPS::ShaderPath, TEXT("MainPS"), SF_Pixel);
