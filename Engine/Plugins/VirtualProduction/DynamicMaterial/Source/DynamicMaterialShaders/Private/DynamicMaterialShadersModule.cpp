// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialShadersModule.h"
#include "DMAlphaOneMinusPS.h"
#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleManager.h"

void FDynamicMaterialShadersModule::StartupModule()
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	const FString PluginShaderDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(UE::DynamicMaterialShaders::Internal::VirtualShaderMountPoint, PluginShaderDir);

	// Force init
	FDMAlphaOneMinusPS::GetStaticType();
}

IMPLEMENT_MODULE(FDynamicMaterialShadersModule, DynamicMaterialShaders)
