// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::DynamicMaterialShaders::Internal
{
	static inline const TCHAR* VirtualShaderMountPoint = TEXT("/Plugin/MaterialDesigner");
}

class FDynamicMaterialShadersModule : public IModuleInterface
{
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	//~ End IModuleInterface interface
};
