// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

class FRDGBuilder;
struct FGlobalShaderPermutationParameters;
using FRDGTextureRef = class FRDGTexture*;

class FDMAlphaOneMinusPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FDMAlphaOneMinusPS, DYNAMICMATERIALSHADERS_API);
	SHADER_USE_PARAMETER_STRUCT(FDMAlphaOneMinusPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static const FString ShaderPath;

	static DYNAMICMATERIALSHADERS_API bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters);

	DYNAMICMATERIALSHADERS_API FParameters* AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, FRDGTextureRef InRGBATexture, FRDGTextureRef InOutputTexture);
};
