// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterialTextureSet : ModuleRules
{
    public DynamicMaterialTextureSet(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
				"Core"
            }
        );

        ShortName = "DynMatTexSet";
    }
}
