// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSet.h"

#include "DMTextureSetMaterialProperty.h"
#include "Engine/Texture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSet)

UDMTextureSet::UDMTextureSet()
{
	UEnum* MaterialPropertyEnum = StaticEnum<EDMTextureSetMaterialProperty>();

	Textures.Reserve(MaterialPropertyEnum->NumEnums());

	for (int32 Index = 0; Index < MaterialPropertyEnum->NumEnums(); ++Index)
	{
		const EDMTextureSetMaterialProperty MaterialProperty = static_cast<EDMTextureSetMaterialProperty>(MaterialPropertyEnum->GetValueByIndex(Index));

		if (MaterialProperty == EDMTextureSetMaterialProperty::None)
		{
			break;
		}

		Textures.Emplace(MaterialProperty);
	}
}

bool UDMTextureSet::HasMaterialProperty(EDMTextureSetMaterialProperty InMaterialProperty) const
{
	return Textures.Contains(InMaterialProperty);
}

const TMap<EDMTextureSetMaterialProperty, FDMMaterialTexture>& UDMTextureSet::GetTextures() const
{
	return Textures;
}

bool UDMTextureSet::HasMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty) const
{
	if (const FDMMaterialTexture* MaterialTexture = Textures.Find(InMaterialProperty))
	{
		return !MaterialTexture->Texture.IsNull();
	}

	return false;
}

bool UDMTextureSet::GetMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty, FDMMaterialTexture& OutMaterialTexture) const
{
	if (const FDMMaterialTexture* MaterialTexture = Textures.Find(InMaterialProperty))
	{
		OutMaterialTexture = *MaterialTexture;
		return true;
	}

	return false;
}

const FDMMaterialTexture* UDMTextureSet::GetMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty) const
{
	return Textures.Find(InMaterialProperty);
}

void UDMTextureSet::SetMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty, const FDMMaterialTexture& InMaterialTexture)
{
	if (FDMMaterialTexture* MaterialTexture = Textures.Find(InMaterialProperty))
	{
		*MaterialTexture = InMaterialTexture;
	}
}

bool UDMTextureSet::ContainsTexture(UTexture* InTexture) const
{
	if (!IsValid(InTexture))
	{
		return false;
	}

	for (const TPair<EDMTextureSetMaterialProperty, FDMMaterialTexture>& TexturePair : Textures)
	{
		if (TexturePair.Value.Texture == InTexture)
		{
			return true;
		}
	}

	return false;
}
