// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMTextureChannelMask.h"
#include "UObject/SoftObjectPtr.h"

#include "DMMaterialTexture.generated.h"

class UTexture;

USTRUCT(BlueprintType)
struct FDMMaterialTexture
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material Designer")
	TSoftObjectPtr<UTexture> Texture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material Designer")
	EDMTextureChannelMask TextureChannel = EDMTextureChannelMask::RGBA;
};
