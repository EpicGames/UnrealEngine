// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "DMTextureChannelMask.generated.h"

UENUM(BlueprintType)
enum class EDMTextureChannelMask : uint8
{
	None = 0 UMETA(Hidden),
	Red = 1 << 0,
	Green = 1 << 1,
	Blue = 1 << 2,
	Alpha = 1 << 3,
	RGB = Red|Green|Blue,
	RGBA = Red|Green|Blue|Alpha,
};
ENUM_CLASS_FLAGS(EDMTextureChannelMask)
