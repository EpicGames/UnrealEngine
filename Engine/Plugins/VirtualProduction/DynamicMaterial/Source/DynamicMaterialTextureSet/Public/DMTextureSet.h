// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "DMMaterialTexture.h"

#include "DMTextureSet.generated.h"

#define UE_API DYNAMICMATERIALTEXTURESET_API

enum class EDMTextureSetMaterialProperty : uint8;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = "Material Designer")
class UDMTextureSet : public UObject
{
	GENERATED_BODY()

public:
	UE_API UDMTextureSet();

	virtual ~UDMTextureSet() override = default;

	/**
	 * Checks whether a given Material Property exists in the Texture Map. Does not check whether
	 * that a Texture is assigned to it.
	 * @param InMaterialProperty The Material Property to check.
	 * @return True if the property exists in the Texture Map.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UE_API bool HasMaterialProperty(EDMTextureSetMaterialProperty InMaterialProperty) const;

	/**
	 * @return Gets the entire Texture Map.
	 */
	UE_API const TMap<EDMTextureSetMaterialProperty, FDMMaterialTexture>& GetTextures() const;

	/**
	 * Checks whether a given Material Property has a Texture assigned to it.
	 * @param InMaterialProperty The Material Property to check.
	 * @return True if the Material Property has an assigned Texture.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UE_API bool HasMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty) const;

	/**
	 * Gets the Material Texture associated with a Material Property. Does not check whether a Texture
	 * is assigned to it.
	 * @param InMaterialProperty The Material Property to check.
	 * @param OutMaterialTexture The found Material Texture.
	 * @return True if the Material Property exists within the Texture Map.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UE_API bool GetMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty, FDMMaterialTexture& OutMaterialTexture) const;

	/**
	 * Gets the Material Texture associated with a Material Property. Does not check whether a Texture
	 * is assigned to it.
	 * @param InMaterialProperty The Material Property to check.
	 * @return A pointer to the found Material Texture, if it exists, or nullptr.
	 */
	UE_API const FDMMaterialTexture* GetMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty) const;

	/**
	 * Sets the Material Texture for a given Material Property. Can be used to unset Textures.
	 * @param InMaterialProperty The Material Property to check.
	 * @param InMaterialTexture The Material Texture to set on the given Material Property.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	UE_API void SetMaterialTexture(EDMTextureSetMaterialProperty InMaterialProperty, const FDMMaterialTexture& InMaterialTexture);

	/**
	 * Checks the Texture Map to see if a given Texture exists within it.
	 * @param InTexture The Texture to search for.
	 * @return True if the Texture exits in the Texture Map.
	 */
	UFUNCTION(BlueprintPure, Category = "Material Designer")
	UE_API bool ContainsTexture(UTexture* InTexture) const;

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material Designer", EditFixedSize, meta = (ReadOnlyKeys, AllowPrivateAccess))
	TMap<EDMTextureSetMaterialProperty, FDMMaterialTexture> Textures;
};

#undef UE_API
