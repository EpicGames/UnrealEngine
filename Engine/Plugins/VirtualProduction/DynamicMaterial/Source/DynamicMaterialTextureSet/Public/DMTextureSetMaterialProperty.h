// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "SceneTypes.h"
#include "UObject/ObjectMacros.h"
#include "DMTextureSetMaterialProperty.generated.h"

/**
 * Copying the values from EMaterialProperty to change required meta data,
 * such as DisplayName and Hidden.
 */
UENUM(BlueprintType)
enum class EDMTextureSetMaterialProperty : uint8
{
	BaseColor = EMaterialProperty::MP_BaseColor,
	Normal = EMaterialProperty::MP_Normal,
	Metallic = EMaterialProperty::MP_Metallic,
	Roughness = EMaterialProperty::MP_Roughness,
	AmbientOcclusion = EMaterialProperty::MP_AmbientOcclusion,
	Displacement = EMaterialProperty::MP_Displacement,
	Specular = EMaterialProperty::MP_Specular,
	SubsurfaceColor = EMaterialProperty::MP_SubsurfaceColor,

	EmissiveColor = EMaterialProperty::MP_EmissiveColor,
	Opacity = EMaterialProperty::MP_Opacity,
	OpacityMask = EMaterialProperty::MP_OpacityMask,
	Anisotropy = EMaterialProperty::MP_Anisotropy,
	Refraction = EMaterialProperty::MP_Refraction,
	Tangent = EMaterialProperty::MP_Tangent,
	WorldPositionOffset = EMaterialProperty::MP_WorldPositionOffset,
	PixelDepthOffset = EMaterialProperty::MP_PixelDepthOffset,

	SurfaceThickness = EMaterialProperty::MP_SurfaceThickness,

	None = EMaterialProperty::MP_MAX UMETA(Hidden)
};
