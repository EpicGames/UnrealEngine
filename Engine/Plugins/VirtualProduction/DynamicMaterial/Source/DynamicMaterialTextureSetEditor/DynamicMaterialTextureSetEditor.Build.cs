// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DynamicMaterialTextureSetEditor : ModuleRules
{
	public DynamicMaterialTextureSetEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"UnrealEd"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"Core",
				"DeveloperSettings",
				"DynamicMaterialTextureSet",
				"EditorWidgets",
				"Engine",
				"InputCore",
				"PropertyEditor",
				"Slate",
				"SlateCore",
			}
		);

		ShortName = "DynMatTexSetEd";
	}
}
