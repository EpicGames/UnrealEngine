// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DMTextureSet.h"

#include "DMTextureSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DMTextureSet)

#define LOCTEXT_NAMESPACE "AssetDefinition_DMTextureSet"

FText UAssetDefinition_DMTextureSet::GetAssetDisplayName() const
{
	return LOCTEXT("DMTextureSet", "Material Designer Texture Set");
}

FText UAssetDefinition_DMTextureSet::GetAssetDisplayName(const FAssetData& InAssetData) const
{
	return GetAssetDisplayName();
}

TSoftClassPtr<> UAssetDefinition_DMTextureSet::GetAssetClass() const
{
	return UDMTextureSet::StaticClass();
}

FLinearColor UAssetDefinition_DMTextureSet::GetAssetColor() const
{
	return FLinearColor(FColor(192, 64, 64));
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DMTextureSet::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Texture, LOCTEXT("AssetDefinition_DMTextureSetSubMenu", "Other"), ECategoryMenuType::Section)
		};
	return Categories;
}

#undef LOCTEXT_NAMESPACE
