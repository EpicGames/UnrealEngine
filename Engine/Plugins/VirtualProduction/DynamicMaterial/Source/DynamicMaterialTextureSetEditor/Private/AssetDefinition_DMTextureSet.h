// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_DMTextureSet.generated.h"

class UAssetDefinition_DMTextureSetInterface;

UCLASS()
class UAssetDefinition_DMTextureSet : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FText GetAssetDisplayName(const FAssetData& InAssetData) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~ End UAssetDefinition End
};
