// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetBlueprintFunctionLibrary.h"

#include "CoreGlobals.h"
#include "DMTextureSet.h"
#include "DMTextureSetFactory.h"
#include "DMTextureSetSettings.h"
#include "Engine/Texture.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SDMTextureSetBuilder.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSetBlueprintFunctionLibrary)

#define LOCTEXT_NAMESPACE "UDMTextureSetBlueprintFunctionLibrary"

UDMTextureSet* UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssets(const TArray<FAssetData>& InAssets)
{
	if (InAssets.IsEmpty())
	{
		return nullptr;
	}

	const UDMTextureSetSettings* TextureSetSettings = GetDefault<UDMTextureSetSettings>();

	if (!TextureSetSettings || TextureSetSettings->Filters.IsEmpty())
	{
		return nullptr;
	}

	UDMTextureSet* TextureSet = Cast<UDMTextureSet>(GetMutableDefault<UDMTextureSetFactory>()->FactoryCreateNew(
		UDMTextureSet::StaticClass(),
		GetTransientPackage(),
		NAME_None,
		RF_Transactional,
		/* Context */ nullptr,
		GWarn
	));

	if (!TextureSet)
	{
		return nullptr;
	}

	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (!AssetClass || !AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			continue;
		}

		UTexture* Texture = Cast<UTexture>(Asset.GetAsset());

		if (!Texture)
		{
			continue;
		}

		const FString AssetName = Asset.AssetName.ToString();

		for (const FDMTextureSetFilter& Filter : TextureSetSettings->Filters)
		{
			if (Filter.MaterialProperties.IsEmpty())
			{
				continue;
			}

			if (!Filter.MatchesFilter(AssetName))
			{
				continue;
			}

			for (const TPair<EDMTextureSetMaterialProperty, EDMTextureChannelMask>& Pair : Filter.MaterialProperties)
			{
				if (TextureSet->HasMaterialTexture(Pair.Key))
				{
					continue;
				}

				TextureSet->SetMaterialTexture(Pair.Key, {Texture, Pair.Value});
			}
		}
	}

	return TextureSet;
}

void UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssetsInteractive(const TArray<FAssetData>& InAssets, 
	FDMTextureSetBuilderOnComplete InOnComplete)
{
	UDMTextureSet* TextureSet = CreateTextureSetFromAssets(InAssets);

	if (!TextureSet)
	{
		InOnComplete.ExecuteIfBound(nullptr, /* Was Accepted */ false);
		return;
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.SizingRule(ESizingRule::Autosized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.Title(LOCTEXT("TextureSetBuilder", "Material Designer Texture Set Builder"))
		[
			SNew(SDMTextureSetBuilder, TextureSet, InAssets, InOnComplete)
		];

	FSlateApplication::Get().AddWindow(Window, /* Show Immediately */ true);
}

#undef LOCTEXT_NAMESPACE
