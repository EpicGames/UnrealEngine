// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetContentBrowserIntegration.h"

FDMTextureSetContentBrowserIntegration::FOnPopulateMenu FDMTextureSetContentBrowserIntegration::PopulateMenuDelegate;

TMulticastDelegate<void(FMenuBuilder&, const TArray<FAssetData>&)>::RegistrationType& FDMTextureSetContentBrowserIntegration::GetPopulateExtenderDelegate()
{
	return PopulateMenuDelegate;
}
