// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetContentBrowserIntegrationPrivate.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "DMTextureSet.h"
#include "DMTextureSetBlueprintFunctionLibrary.h"
#include "DMTextureSetStyle.h"
#include "Engine/Texture.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "FDMTextureSetContentBrowserIntegrationPrivate"

FDelegateHandle FDMTextureSetContentBrowserIntegrationPrivate::ContentBrowserHandle;

void FDMTextureSetContentBrowserIntegrationPrivate::Integrate()
{
	FDMTextureSetStyle::Get();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FDMTextureSetContentBrowserIntegrationPrivate::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

void FDMTextureSetContentBrowserIntegrationPrivate::Disintegrate()
{
	if (!ContentBrowserHandle.IsValid())
	{
		return;
	}

	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule->GetAllAssetViewContextMenuExtenders();

		CBMenuExtenderDelegates.RemoveAll(
			[](const FContentBrowserMenuExtender_SelectedAssets& InElement)
			{
				return InElement.GetHandle() == ContentBrowserHandle;
			}
		);
	}

	ContentBrowserHandle.Reset();
}

TSharedRef<FExtender> FDMTextureSetContentBrowserIntegrationPrivate::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	bool bHasTexture = false;

	for (const FAssetData& SelectedAsset : InSelectedAssets)
	{
		if (UClass* AssetClass = SelectedAsset.GetClass(EResolveClass::Yes))
		{
			if (AssetClass->IsChildOf<UTexture>())
			{
				bHasTexture = true;
				break;
			}
		}
	}

	if (!bHasTexture)
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		"GetAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateLambda(
			[InSelectedAssets](FMenuBuilder& InMenuBuilder)
			{
				InMenuBuilder.AddSubMenu(
					LOCTEXT("TextureSetMenu", "Material Designer Texture Set"),
					LOCTEXT("TextureSetMenuTooltip", "Create and use Material Designer Texture Sets"),
					FNewMenuDelegate::CreateStatic(&FDMTextureSetContentBrowserIntegrationPrivate::CreateMenu, InSelectedAssets),
					/* Close on click */ false,
					FSlateIconFinder::FindIconForClass(UTexture::StaticClass())
				);
			}
		)
	);	

	return Extender;
}

void FDMTextureSetContentBrowserIntegrationPrivate::CreateMenu(FMenuBuilder& InMenuBuilder, TArray<FAssetData> InSelectedAssets)
{
	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CreateTextureSet", "Create From Selected Textures"),
		LOCTEXT("CreateTextureSetTooltip", "Creates an asset listing a group of textures and the material properties they are associated with."),
		FSlateIconFinder::FindIconForClass(UTexture::StaticClass()),
		FUIAction(FExecuteAction::CreateStatic(&FDMTextureSetContentBrowserIntegrationPrivate::CreateTextureSet, InSelectedAssets))
	);

	PopulateMenuDelegate.Broadcast(InMenuBuilder, InSelectedAssets);
}

void FDMTextureSetContentBrowserIntegrationPrivate::CreateTextureSet(TArray<FAssetData> InSelectedAssets)
{
	if (InSelectedAssets.IsEmpty())
	{
		return;
	}

	UDMTextureSetBlueprintFunctionLibrary::CreateTextureSetFromAssetsInteractive(
		InSelectedAssets,
		FDMTextureSetBuilderOnComplete::CreateStatic(
			&FDMTextureSetContentBrowserIntegrationPrivate::OnCreateTextureSetComplete,
			InSelectedAssets[0].PackagePath.ToString()
		)
	);
}

void FDMTextureSetContentBrowserIntegrationPrivate::OnCreateTextureSetComplete(UDMTextureSet* InTextureSet, bool bInAccepted, FString InPath)
{
	if (!InTextureSet || !bInAccepted)
	{
		return;
	}

	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	FString UniquePackageName;
	FString UniqueAssetName;

	const FString BasePackageName = InPath / TEXT("MDTS_NewTextureSet");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), UniquePackageName, UniqueAssetName);

	UPackage* Package = CreatePackage(*UniquePackageName);

	if (!Package)
	{
		return;
	}

	InTextureSet->SetFlags(RF_Standalone | RF_Public);
	InTextureSet->Rename(*UniqueAssetName, Package, REN_DontCreateRedirectors);

	FAssetRegistryModule::AssetCreated(InTextureSet);
}

#undef LOCTEXT_NAMESPACE
