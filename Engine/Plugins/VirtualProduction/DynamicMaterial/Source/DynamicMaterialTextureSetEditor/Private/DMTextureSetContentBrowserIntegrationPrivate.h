// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMTextureSetContentBrowserIntegration.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

class FExtender;
class FMenuBuilder;
class UDMTextureSet;

class FDMTextureSetContentBrowserIntegrationPrivate : public FDMTextureSetContentBrowserIntegration
{
public:
	static void Integrate();

	static void Disintegrate();

protected:
	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets);

	static void CreateMenu(FMenuBuilder& InMenuBuilder, TArray<FAssetData> InSelectedAssets);

	static void CreateTextureSet(TArray<FAssetData> InSelectedAssets);

	static void OnCreateTextureSetComplete(UDMTextureSet* InTextureSet, bool bInAccepted, FString InPath);

	static FDelegateHandle ContentBrowserHandle;
};
