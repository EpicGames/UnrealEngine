// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetFactory.h"

#include "DMTextureSet.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSetFactory)

#define LOCTEXT_NAMESPACE "MaterialDesignerInstanceFactory"

UDMTextureSetFactory::UDMTextureSetFactory()
{
	SupportedClass = UDMTextureSet::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;
	bText = false;
}

UObject* UDMTextureSetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name,
	EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UDMTextureSet::StaticClass()));

	UDMTextureSet* NewInstance = NewObject<UDMTextureSet>(InParent, Class, Name, Flags | RF_Transactional);
	check(NewInstance);

	return NewInstance;
}

FText UDMTextureSetFactory::GetDisplayName() const
{
	return LOCTEXT("UDMTextureSet", "Material Designer Texture Set");
}

FText UDMTextureSetFactory::GetToolTip() const
{
	return LOCTEXT("UDMTextureSetTooltip", "A set of textures which are associated with a material property.");
}

#undef LOCTEXT_NAMESPACE
