// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetFilter.h"

#include "DMTextureSetSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSetFilter)

FDMTextureSetFilter::FDMTextureSetFilter()
	: FilterStrings({TEXT("_")})
	, MaterialProperties({{EDMTextureSetMaterialProperty::BaseColor, EDMTextureChannelMask::RGBA}})
{
}

bool FDMTextureSetFilter::MatchesFilter(const FString& InAssetName) const
{
	bool bOnlyMatchEndOfAssetName = false;

	if (UDMTextureSetSettings* Settings = UDMTextureSetSettings::Get())
	{
		bOnlyMatchEndOfAssetName = Settings->bOnlyMatchEndOfAssetName;
	}

	for (const FString& FilterString : FilterStrings)
	{
		if (bOnlyMatchEndOfAssetName || FilterString.StartsWith(TEXT("_")))
		{
			if (InAssetName.EndsWith(FilterString))
			{
				return true;
			}
		}
		else if (InAssetName.Find(FilterString) != INDEX_NONE)
		{
			return true;
		}
	}

	return false;
}
