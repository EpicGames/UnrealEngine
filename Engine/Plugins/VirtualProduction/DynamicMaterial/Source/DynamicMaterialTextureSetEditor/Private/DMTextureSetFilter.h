// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "DMTextureChannelMask.h"
#include "DMTextureSetMaterialProperty.h"

#include "DMTextureSetFilter.generated.h"

USTRUCT()
struct FDMTextureSetFilter
{
	GENERATED_BODY()

	FDMTextureSetFilter();

	/**
	 * Portion of the name of the Texture asset to search for. For instance _Normal or _ORB.
	 * Will match any of the given strings.
	 * Filters starting with _ will only match if they are at the end of an asset name.
	 */
	UPROPERTY(EditAnywhere, Category = "Material Designer")
	TArray<FString> FilterStrings;

	/**
	 * Where the matched Texture assets should be placed into the Texture Set. Links to the channel for the given asset.
	 */
	UPROPERTY(EditAnywhere, Category = "Material Designer")
	TMap<EDMTextureSetMaterialProperty, EDMTextureChannelMask> MaterialProperties;

	/**
	 * Checks the given asset name against the filter strings.
	 * @param InAssetName The name of the asset
	 * @return True if any filter string was matched.
	 */
	bool MatchesFilter(const FString& InAssetName) const;
};
