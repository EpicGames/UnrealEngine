// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetSettings.h"

#include "DMTextureSetMaterialProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSetSettings)

UDMTextureSetSettings::UDMTextureSetSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("Material Designer Texture Set");

	int32 FilterCount = 0;

	FDMTextureSetFilter BaseColor;
	BaseColor.FilterStrings = {TEXT("Base_Color"), TEXT("BaseColor"), TEXT("Base_Colour"), TEXT("BaseColour"), TEXT("_BC"), TEXT("Diffuse"), TEXT("Albedo"), TEXT("_Diff"), TEXT("_D")};
	BaseColor.MaterialProperties = {{EDMTextureSetMaterialProperty::BaseColor, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Roughness;
	Roughness.FilterStrings = {TEXT("Roughness"), TEXT("Rough"), TEXT("_R")};
	Roughness.MaterialProperties = {{EDMTextureSetMaterialProperty::Roughness, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Normal;
	Normal.FilterStrings = {TEXT("Normal"), TEXT("Norm"), TEXT("_N"), TEXT("_Nor")};
	Normal.MaterialProperties = {{EDMTextureSetMaterialProperty::Normal, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Metallic;
	Metallic.FilterStrings = {TEXT("Metallic"), TEXT("Metal"), TEXT("_M")};
	Metallic.MaterialProperties = {{EDMTextureSetMaterialProperty::Metallic, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter AmbientOcclusion;
	AmbientOcclusion.FilterStrings = {TEXT("AmbientOcclusion"), TEXT("Ambient_Occlusion"), TEXT("_AO")};
	AmbientOcclusion.MaterialProperties = {{EDMTextureSetMaterialProperty::AmbientOcclusion, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Specular;
	Specular.FilterStrings = {TEXT("Specular"), TEXT("_S")};
	Specular.MaterialProperties = {{EDMTextureSetMaterialProperty::Specular, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Emissive;
	Emissive.FilterStrings = {TEXT("Emissive"), TEXT("Emission"), TEXT("_E")};
	Emissive.MaterialProperties = {{EDMTextureSetMaterialProperty::EmissiveColor, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Opacity;
	Opacity.FilterStrings = {TEXT("Opacity"), TEXT("_O"), TEXT("Alpha"), TEXT("_A")};
	Opacity.MaterialProperties = {{EDMTextureSetMaterialProperty::Opacity, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter OpacityMask;
	OpacityMask.FilterStrings = {TEXT("OpacityMask"), TEXT("Opacity_Mask"), TEXT("_OM"), TEXT("AlphaMask"), TEXT("Alpha_Mask"), TEXT("_AM")};
	OpacityMask.MaterialProperties = {{EDMTextureSetMaterialProperty::OpacityMask, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Displacement;
	Displacement.FilterStrings = {TEXT("Displacement"), TEXT("_Disp"), TEXT("Height")};
	Displacement.MaterialProperties = {{EDMTextureSetMaterialProperty::Displacement, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter WorldPositionOffset;
	WorldPositionOffset.FilterStrings = {TEXT("WorldPositionOffset"), TEXT("World_Position_Offset"), TEXT("_WPO")};
	WorldPositionOffset.MaterialProperties = {{EDMTextureSetMaterialProperty::WorldPositionOffset, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter PixelDepthOffset;
	PixelDepthOffset.FilterStrings = {TEXT("PixelDepthOffset"), TEXT("Pixel_Depth_Offset"), TEXT("_PDO")};
	PixelDepthOffset.MaterialProperties = {{EDMTextureSetMaterialProperty::PixelDepthOffset, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter SubsurfaceColor;
	SubsurfaceColor.FilterStrings = {TEXT("SubsurfaceColor"), TEXT("SubsurfaceColour"), TEXT("Subsurface_Color"), TEXT("Subsurface_Coluor"), TEXT("_SC"), TEXT("_SS"), TEXT("_SSC")};
	SubsurfaceColor.MaterialProperties = {{EDMTextureSetMaterialProperty::SubsurfaceColor, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter SurfaceThickness;
	SubsurfaceColor.FilterStrings = {TEXT("SurfaceThickness"), TEXT("Surface_Thickness"), TEXT("_ST")};
	SubsurfaceColor.MaterialProperties = {{EDMTextureSetMaterialProperty::SurfaceThickness, EDMTextureChannelMask::Red}};
	++FilterCount;

	FDMTextureSetFilter Anisotropy;
	Anisotropy.FilterStrings = {TEXT("Anisotropy"), TEXT("_Ani")};
	Anisotropy.MaterialProperties = {{EDMTextureSetMaterialProperty::Anisotropy, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Refraction;
	Refraction.FilterStrings = {TEXT("Refraction"), TEXT("_Ref")};
	Refraction.MaterialProperties = {{EDMTextureSetMaterialProperty::Refraction, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter Tangent;
	Tangent.FilterStrings = {TEXT("Tangent"), TEXT("_T")};
	Tangent.MaterialProperties = {{EDMTextureSetMaterialProperty::Tangent, EDMTextureChannelMask::RGBA}};
	++FilterCount;

	FDMTextureSetFilter ORM;
	ORM.FilterStrings = {TEXT("_ORM"), TEXT("OcclusionRoughnessMetallic")};
	ORM.MaterialProperties = {
		{EDMTextureSetMaterialProperty::Opacity, EDMTextureChannelMask::Red},
		{EDMTextureSetMaterialProperty::Roughness, EDMTextureChannelMask::Green},
		{EDMTextureSetMaterialProperty::Metallic, EDMTextureChannelMask::Blue}
	};
	++FilterCount;

	FDMTextureSetFilter ORDp;
	ORDp.FilterStrings = {TEXT("_ORDp"), TEXT("AmbientOcclusionRoughnessDisplacement")};
	ORDp.MaterialProperties = {
		{EDMTextureSetMaterialProperty::AmbientOcclusion, EDMTextureChannelMask::Red},
		{EDMTextureSetMaterialProperty::Roughness, EDMTextureChannelMask::Green},
		{EDMTextureSetMaterialProperty::Displacement, EDMTextureChannelMask::Blue}
	};
	++FilterCount;

	FDMTextureSetFilter RM;
	RM.FilterStrings = {TEXT("_RM"), TEXT("RoughnessMetallic")};
	RM.MaterialProperties = {
		{EDMTextureSetMaterialProperty::Roughness, EDMTextureChannelMask::Red},
		{EDMTextureSetMaterialProperty::Metallic, EDMTextureChannelMask::Green}
	};
	++FilterCount;

	FDMTextureSetFilter RMA;
	RMA.FilterStrings = {TEXT("_RMA"), TEXT("RoughnessMetallicAmbientOcclusion")};
	RMA.MaterialProperties = {
		{EDMTextureSetMaterialProperty::Roughness, EDMTextureChannelMask::Red},
		{EDMTextureSetMaterialProperty::Metallic, EDMTextureChannelMask::Green},
		{EDMTextureSetMaterialProperty::AmbientOcclusion, EDMTextureChannelMask::Blue}
	};
	++FilterCount;

	Filters.Reserve(FilterCount);

	Filters.Add(MoveTemp(BaseColor));
	Filters.Add(MoveTemp(Roughness));
	Filters.Add(MoveTemp(Normal));
	Filters.Add(MoveTemp(Metallic));
	Filters.Add(MoveTemp(AmbientOcclusion));
	Filters.Add(MoveTemp(Specular));
	Filters.Add(MoveTemp(Emissive));
	Filters.Add(MoveTemp(Opacity));
	Filters.Add(MoveTemp(OpacityMask));
	Filters.Add(MoveTemp(Displacement));
	Filters.Add(MoveTemp(WorldPositionOffset));
	Filters.Add(MoveTemp(PixelDepthOffset));
	Filters.Add(MoveTemp(SubsurfaceColor));
	Filters.Add(MoveTemp(SurfaceThickness));
	Filters.Add(MoveTemp(Anisotropy));
	Filters.Add(MoveTemp(Refraction));
	Filters.Add(MoveTemp(Tangent));
	Filters.Add(MoveTemp(ORM));
	Filters.Add(MoveTemp(ORDp));
	Filters.Add(MoveTemp(RM));
	Filters.Add(MoveTemp(RMA));
}

UDMTextureSetSettings* UDMTextureSetSettings::Get()
{
	return GetMutableDefault<UDMTextureSetSettings>();
}
