// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "DMTextureSetFilter.h"

#include "DMTextureSetSettings.generated.h"

/**
 * DM Texture Set Settings
 */
UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "Material Designer Texture Set"))
class UDMTextureSetSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UDMTextureSetSettings();
	virtual ~UDMTextureSetSettings() override = default;

	static UDMTextureSetSettings* Get();

	UPROPERTY(EditAnywhere, Category = "Material Designer")
	bool bOnlyMatchEndOfAssetName = false;

	UPROPERTY(EditAnywhere, Category = "Material Designer", meta = (TitleProperty=FilterStrings))
	TArray<FDMTextureSetFilter> Filters;
};
