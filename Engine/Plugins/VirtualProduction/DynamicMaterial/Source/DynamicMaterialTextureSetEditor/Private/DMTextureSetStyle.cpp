// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMTextureSetStyle.h"

#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/StyleColors.h"

const ISlateStyle& FDMTextureSetStyle::Get()
{
	static FDMTextureSetStyle Instance;
	return Instance;
}

FDMTextureSetStyle::FDMTextureSetStyle()
	: FSlateStyleSet(TEXT("DMTextureSetStyle"))
{
	Set("TextureSetConfig.Window.Background", new FSlateColorBrush(
		FStyleColors::Panel.GetSpecifiedColor()));

	Set("TextureSetConfig.Cell.Background", new FSlateRoundedBoxBrush(
		FStyleColors::Recessed.GetSpecifiedColor(), 6.0f,
		FStyleColors::Header.GetSpecifiedColor(), 2.0f));

	Set("TextureSetConfig.TextureNameHighlight.Background", new FSlateColorBrush(FStyleColors::Black.GetSpecifiedColor() * FLinearColor(1.f, 1.f, 1.f, 0.5f)));

	Set("TextureSetConfig.TextureNameHighlight.Color", FStyleColors::AccentWhite);
}
