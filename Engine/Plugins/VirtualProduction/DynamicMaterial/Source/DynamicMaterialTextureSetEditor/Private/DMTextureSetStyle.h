// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FDMTextureSetStyle final : public FSlateStyleSet
{
public:
	static const ISlateStyle& Get();

	FDMTextureSetStyle();

	virtual ~FDMTextureSetStyle() override = default;
};
