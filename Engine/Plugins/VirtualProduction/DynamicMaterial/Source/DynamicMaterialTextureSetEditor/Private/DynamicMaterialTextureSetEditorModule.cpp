// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMaterialTextureSetEditorModule.h"

#include "DMTextureSetContentBrowserIntegrationPrivate.h"
#include "DMTextureSetStyle.h"
#include "Modules/ModuleManager.h"

void FDynamicMaterialTextureSetEditorModule::StartupModule()
{
	FDMTextureSetStyle::Get();
	FDMTextureSetContentBrowserIntegrationPrivate::Integrate();
}

void FDynamicMaterialTextureSetEditorModule::ShutdownModule()
{
	FDMTextureSetContentBrowserIntegrationPrivate::Disintegrate();
}

IMPLEMENT_MODULE(FDynamicMaterialTextureSetEditorModule, DynamicMaterialTextureSetEditor)
