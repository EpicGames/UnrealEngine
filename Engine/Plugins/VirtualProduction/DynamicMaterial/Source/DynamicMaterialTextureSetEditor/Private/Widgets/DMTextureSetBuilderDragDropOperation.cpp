// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DMTextureSetBuilderDragDropOperation.h"

#include "Framework/Application/SlateApplication.h"

TSharedRef<FDMTextureSetBuilderDragDropOperation> FDMTextureSetBuilderDragDropOperation::New(const FAssetData& InAssetData, int32 InIndex, bool bInIsMaterialProperty)
{
	TSharedRef<FDMTextureSetBuilderDragDropOperation> Operation = MakeShared<FDMTextureSetBuilderDragDropOperation>();

	Operation->Init({InAssetData}, {}, nullptr);
	Operation->Index = InIndex;
	Operation->bIsMaterialProperty = bInIsMaterialProperty;

	Operation->Construct();
	return Operation;
}

int32 FDMTextureSetBuilderDragDropOperation::GetIndex() const
{
	return Index;
}

bool FDMTextureSetBuilderDragDropOperation::IsMaterialProperty() const
{
	return bIsMaterialProperty;
}

FCursorReply FDMTextureSetBuilderDragDropOperation::OnCursorQuery()
{
	if (bIsMaterialProperty)
	{
		const bool bOverwriteTexture = FSlateApplication::Get().GetModifierKeys().IsShiftDown();

		if (bOverwriteTexture)
		{
			return FCursorReply::Cursor(EMouseCursor::Hand);
		}
	}

	return FAssetDragDropOp::OnCursorQuery();
}
