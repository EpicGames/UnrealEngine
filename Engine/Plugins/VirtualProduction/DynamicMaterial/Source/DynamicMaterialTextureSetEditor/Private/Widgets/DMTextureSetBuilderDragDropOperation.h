// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/AssetDragDropOp.h"

class FDMTextureSetBuilderDragDropOperation : public FAssetDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMTextureSetBuilderDragDropOperation, FAssetDragDropOp)

	static TSharedRef<FDMTextureSetBuilderDragDropOperation> New(const FAssetData& InAssetData, int32 InIndex, bool bInIsMaterialProperty);

	virtual ~FDMTextureSetBuilderDragDropOperation() override = default;

	int32 GetIndex() const;

	bool IsMaterialProperty() const;

	//~ Begin FDragDropOperation
	virtual FCursorReply OnCursorQuery();
	//~ End FDragDropOperation

protected:
	int32 Index = -1;
	bool bIsMaterialProperty = false;
};
