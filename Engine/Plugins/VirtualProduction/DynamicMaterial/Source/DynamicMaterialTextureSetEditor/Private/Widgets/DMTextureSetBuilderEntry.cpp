// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DMTextureSetBuilderEntry.h"

#include "DMTextureSetMaterialProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMTextureSetBuilderEntry)

FDMTextureSetBuilderEntry::FDMTextureSetBuilderEntry()
	: FDMTextureSetBuilderEntry(EDMTextureSetMaterialProperty::BaseColor, nullptr, EDMTextureChannelMask::RGBA)
{
}

FDMTextureSetBuilderEntry::FDMTextureSetBuilderEntry(EDMTextureSetMaterialProperty InMaterialProperty, UTexture* InTexture, EDMTextureChannelMask InMask)
	: MaterialProperty(InMaterialProperty)
	, Texture(InTexture)
	, ChannelMask(InMask)
{
}

FDMTextureSetBuilderEntryProvider::FDMTextureSetBuilderEntryProvider(const TSharedRef<FDMTextureSetBuilderEntry>& InEntry)
	: Entry(InEntry)
{
}

bool FDMTextureSetBuilderEntryProvider::IsValid() const
{
	return true;
}

const UStruct* FDMTextureSetBuilderEntryProvider::GetBaseStructure() const
{
	return FDMTextureSetBuilderEntry::StaticStruct();
}

void FDMTextureSetBuilderEntryProvider::GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const
{
	OutInstances.Add(MakeShared<FStructOnScope>(
		FDMTextureSetBuilderEntry::StaticStruct(),
		reinterpret_cast<uint8*>(&Entry.Get())
	));
}

bool FDMTextureSetBuilderEntryProvider::IsPropertyIndirection() const
{
	return false;
}
