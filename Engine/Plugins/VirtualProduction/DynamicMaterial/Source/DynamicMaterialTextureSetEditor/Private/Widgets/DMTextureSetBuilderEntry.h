// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStructureDataProvider.h"

#include "DMTextureChannelMask.h"
#include "DMTextureSetMaterialProperty.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

#include "DMTextureSetBuilderEntry.generated.h"

class UTexture;

USTRUCT()
struct FDMTextureSetBuilderEntry
{
	GENERATED_BODY()

	FDMTextureSetBuilderEntry();

	FDMTextureSetBuilderEntry(EDMTextureSetMaterialProperty InMaterialProperty, UTexture* InTexture, EDMTextureChannelMask InMask);

	UPROPERTY(EditAnywhere, Category = "Material Designer")
	EDMTextureSetMaterialProperty MaterialProperty = EDMTextureSetMaterialProperty::BaseColor;

	UPROPERTY(EditAnywhere, Category = "Material Designer")
	TObjectPtr<UTexture> Texture;

	UPROPERTY(EditAnywhere, Category = "Material Designer")
	EDMTextureChannelMask ChannelMask = EDMTextureChannelMask::RGBA;
};

class FDMTextureSetBuilderEntryProvider : public IStructureDataProvider
{
public:
	explicit FDMTextureSetBuilderEntryProvider(const TSharedRef<FDMTextureSetBuilderEntry>& InEntry);

	virtual ~FDMTextureSetBuilderEntryProvider() override = default;

	//~ Begin IStructureDataProvider
	virtual bool IsValid() const override;
	virtual const UStruct* GetBaseStructure() const override;
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override;
	virtual bool IsPropertyIndirection() const override;
	//~ End IStructureDataProvider

protected:
	TSharedRef<FDMTextureSetBuilderEntry> Entry;
};
