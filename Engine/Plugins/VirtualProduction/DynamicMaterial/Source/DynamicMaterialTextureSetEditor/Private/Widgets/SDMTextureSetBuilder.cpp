// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMTextureSetBuilder.h"

#include "DMTextureSet.h"
#include "DMTextureSetMaterialProperty.h"
#include "DMTextureSetStyle.h"
#include "Engine/Texture.h"
#include "SAssetDropTarget.h"
#include "Widgets/DMTextureSetBuilderDragDropOperation.h"
#include "Widgets/DMTextureSetBuilderEntry.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SDMTextureSetBuilderCellBase.h"
#include "Widgets/SDMTextureSetBuilderMaterialPropertyCell.h"
#include "Widgets/SDMTextureSetBuilderUnassignedTextureCell.h"

#define LOCTEXT_NAMESPACE "SDMTextureSetBuilder"

namespace UE::DynamicMaterialEditor::TextureSet::Private
{
	constexpr int32 WidthMax = 8;
}

void SDMTextureSetBuilder::Construct(const FArguments& InArgs, UDMTextureSet* InTextureSet, const TArray<FAssetData>& InAssets, 
	FDMTextureSetBuilderOnComplete InOnComplete)
{
	TextureSet.Reset(InTextureSet);
	Assets = InAssets;
	OnComplete = InOnComplete;

	if (!IsValid(InTextureSet))
	{
		OnComplete.ExecuteIfBound(nullptr, /* Was Accepted */ false);
		return;
	}

	using namespace UE::DynamicMaterialEditor::TextureSet::Private;

	MaterialPropertyGrid = SNew(SGridPanel);
	int32 WidthIndex = 0;
	int32 HeightIndex = 0;

	MaterialProperties.Reserve(InTextureSet->GetTextures().Num());

	for (const TPair<EDMTextureSetMaterialProperty, FDMMaterialTexture>& SetElement : InTextureSet->GetTextures())
	{
		TSharedRef<FDMTextureSetBuilderEntry> Entry = MakeShared<FDMTextureSetBuilderEntry>(
			SetElement.Key,
			SetElement.Value.Texture.LoadSynchronous(),
			SetElement.Value.TextureChannel
		);

		MaterialPropertyGrid->AddSlot(WidthIndex, HeightIndex)
			.Padding(5.f)
			[
				GenerateMaterialPropertyCell(Entry, MaterialProperties.Num())
			];

		MaterialProperties.Add(Entry);

		++WidthIndex;

		if (WidthIndex == WidthMax)
		{
			WidthIndex = 0;
			++HeightIndex;
		}
	}

	UnassignedTextureGrid = SNew(SGridPanel);
	
	CreateUnassignedTextureSlots();

	ChildSlot
	[
		SNew(SBorder)
		.Padding(5.f)
		.BorderImage(FDMTextureSetStyle::Get().GetBrush("TextureSetConfig.Window.Background"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 5.f, 5.f, 0.f)
			[
				MaterialPropertyGrid.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 5.f, 5.f, 0.f)
			[
				SNew(SBorder)
				.Padding(5.f)
				.BorderImage(FDMTextureSetStyle::Get().GetBrush("TextureSetConfig.Cell.Background"))
				[
					SNew(SAssetDropTarget)
					.OnAreAssetsAcceptableForDrop(this, &SDMTextureSetBuilder::OnAssetDraggedOver)
					.OnAssetsDropped(this, &SDMTextureSetBuilder::OnAssetsDropped)
					[
						UnassignedTextureGrid.ToSharedRef()
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.f, 5.f, 5.f, 5.f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.Padding(10.f, 5.f)
					[
						SNew(SButton)
						.Text(LOCTEXT("Accept", "Accept"))
						.ContentPadding(FMargin(5.f, 3.f))
						.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
						.OnClicked(this, &SDMTextureSetBuilder::OnAcceptClicked)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.Padding(10.f, 5.f)
					[
						SNew(SButton)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.ContentPadding(FMargin(5.f, 3.f))
						.ButtonStyle(FAppStyle::Get(), "Button")
						.OnClicked(this, &SDMTextureSetBuilder::OnCancelClicked)
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				]
			]
		]
	];
}

void SDMTextureSetBuilder::SwapTexture(int32 InFromIndex, bool bInIsFromMaterialProperty, int32 InToIndex, bool bInIsToMaterialProperty)
{
	if (InFromIndex == InToIndex && bInIsFromMaterialProperty == bInIsToMaterialProperty)
	{
		return;
	}

	const bool bSwapTextures = !FSlateApplication::Get().GetModifierKeys().IsShiftDown();

	TSharedPtr<SDMTextureSetBuilderCellBase> FromCell = GetCell(InFromIndex, bInIsFromMaterialProperty);
	TSharedPtr<SDMTextureSetBuilderCellBase> ToCell = GetCell(InToIndex, bInIsToMaterialProperty);

	if (!FromCell.IsValid() || !ToCell.IsValid())
	{
		return;
	}

	UTexture* FromTexture = FromCell->GetTexture();

	if (bSwapTextures)
	{
		UTexture* ToTexture = ToCell->GetTexture();

		FromCell->SetTexture(ToTexture);

		if (bInIsFromMaterialProperty)
		{
			MaterialProperties[InFromIndex]->Texture = ToTexture;
		}
	}

	ToCell->SetTexture(FromTexture);

	if (bInIsToMaterialProperty)
	{
		MaterialProperties[InToIndex]->Texture = FromTexture;
	}

	if (!bInIsFromMaterialProperty)
	{
		UnassignedTextures.Remove(FromTexture);
	}

	if (!bInIsToMaterialProperty)
	{
		if (UnassignedTextures.Contains(FromTexture))
		{
			UnassignedTextures.Add(FromTexture);
		}
	}

	if (!bInIsFromMaterialProperty || !bInIsToMaterialProperty || !bSwapTextures)
	{
		CreateUnassignedTextureSlots();
	}
}

void SDMTextureSetBuilder::SetTexture(int32 InIndex, bool bInIsMaterialProperty, UTexture* InTexture)
{
	if (!IsValid(InTexture))
	{
		return;
	}

	TSharedPtr<SDMTextureSetBuilderCellBase> Cell = GetCell(InIndex, bInIsMaterialProperty);

	if (!Cell.IsValid())
	{
		return;
	}

	Cell->SetTexture(InTexture);

	if (bInIsMaterialProperty)
	{
		MaterialProperties[InIndex]->Texture = InTexture;
	}

	Assets.AddUnique(FAssetData(InTexture));

	CreateUnassignedTextureSlots();
}

TSharedRef<SWidget> SDMTextureSetBuilder::GenerateMaterialPropertyCell(const TSharedRef<FDMTextureSetBuilderEntry>& InListItem, int32 InIndex)
{
	return SNew(SDMTextureSetBuilderMaterialPropertyCell, SharedThis(this), InListItem, InIndex);
}

TSharedRef<SWidget> SDMTextureSetBuilder::GenerateUnusedTextureCell(UTexture* InTexture, int32 InIndex)
{
	return SNew(SDMTextureSetBuilderUnassignedTextureCell, SharedThis(this), InTexture, InIndex);
}

FReply SDMTextureSetBuilder::OnAcceptClicked()
{
	UDMTextureSet* TextureSetObject = TextureSet.Get();

	if (!IsValid(TextureSetObject))
	{
		return FReply::Handled();
	}

	for (const TPair<EDMTextureSetMaterialProperty, FDMMaterialTexture>& SetElement : TextureSet->GetTextures())
	{
		TextureSet->SetMaterialTexture(SetElement.Key, {nullptr, EDMTextureChannelMask::RGBA});
	}

	for (const TSharedRef<FDMTextureSetBuilderEntry>& Entry : MaterialProperties)
	{
		if (!TextureSet->HasMaterialProperty(Entry->MaterialProperty))
		{
			continue;
		}

		TextureSet->SetMaterialTexture(Entry->MaterialProperty, {Entry->Texture, Entry->ChannelMask});
	}

	bAccepted = true;

	Close();

	return FReply::Handled();
}

FReply SDMTextureSetBuilder::OnCancelClicked()
{
	Close();

	return FReply::Handled();
}

void SDMTextureSetBuilder::Close()
{
	if (TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()))
	{
		CurrentWindow->RequestDestroyWindow();
	}

	OnComplete.ExecuteIfBound(TextureSet.Get(), /* Was Accepted */ bAccepted);
}

void SDMTextureSetBuilder::CreateUnassignedTextureSlots()
{
	using namespace UE::DynamicMaterialEditor::TextureSet::Private;

	UnassignedTextures.Empty(TextureSet->GetTextures().Num());
	UnassignedTextureGrid->ClearChildren();

	int32 WidthIndex = 0;
	int32 HeightIndex = 0;

	for (const FAssetData& Asset : Assets)
	{
		UTexture* Texture = Cast<UTexture>(Asset.GetAsset());

		if (!Texture || IsTextureAssignedToMaterialProperty(Texture))
		{
			continue;
		}

		UnassignedTextureGrid->AddSlot(WidthIndex, HeightIndex)
			.Padding(5.f)
			[
				GenerateUnusedTextureCell(Texture, UnassignedTextures.Num())
			];

		UnassignedTextures.Add(Texture);

		++WidthIndex;

		if (WidthIndex == WidthMax)
		{
			WidthIndex = 0;
			++HeightIndex;
		}
	}

	if (UnassignedTextures.IsEmpty())
	{
		UnassignedTextureGrid->AddSlot(WidthIndex, HeightIndex)
			.Padding(5.f)
			[
				GenerateUnusedTextureCell(nullptr, UnassignedTextures.Num())
			];

		UnassignedTextures.Add(nullptr);
	}
}

bool SDMTextureSetBuilder::OnAssetDraggedOver(TArrayView<FAssetData> InAssets)
{
	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (AssetClass && AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			return true;
		}
	}

	return false;
}

void SDMTextureSetBuilder::OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets)
{
	if (TSharedPtr<FDMTextureSetBuilderDragDropOperation> BuilderOperation = InDragDropEvent.GetOperationAs<FDMTextureSetBuilderDragDropOperation>())
	{
		if (BuilderOperation->IsMaterialProperty())
		{
			if (TSharedPtr<SDMTextureSetBuilderCellBase> FromCell = GetCell(BuilderOperation->GetIndex(), /* Is Material Property */ true))
			{
				if (UTexture* Texture = FromCell->GetTexture())
				{
					FromCell->SetTexture(nullptr);

					if (!UnassignedTextures.Contains(Texture))
					{
						CreateUnassignedTextureSlots();
					}
				}
			}
		}
	}
}

TSharedPtr<SDMTextureSetBuilderCellBase> SDMTextureSetBuilder::GetCell(int32 InIndex, bool bInIsMaterialProperty) const
{
	FChildren* Children = bInIsMaterialProperty
		? MaterialPropertyGrid->GetChildren()
		: UnassignedTextureGrid->GetChildren();;

	if (InIndex >= Children->Num())
	{
		return nullptr;
	}

	return StaticCastSharedRef<SDMTextureSetBuilderCellBase>(Children->GetChildAt(InIndex));
}

UTexture* SDMTextureSetBuilder::GetTexture(int32 InIndex, bool bInIsMaterialProperty) const
{
	if (TSharedPtr<SDMTextureSetBuilderCellBase> Cell = GetCell(InIndex, bInIsMaterialProperty))
	{
		return Cell->GetTexture();
	}

	return nullptr;
}

void SDMTextureSetBuilder::SetTexture(int32 InIndex, bool bInIsMaterialProperty, UTexture* InTexture) const
{
	if (TSharedPtr<SDMTextureSetBuilderCellBase> Cell = GetCell(InIndex, bInIsMaterialProperty))
	{
		Cell->SetTexture(InTexture);
	}
}

bool SDMTextureSetBuilder::IsTextureAssignedToMaterialProperty(UTexture* InTexture) const
{
	FChildren* Children = MaterialPropertyGrid->GetChildren();
	const int32 ChildCount = Children->Num();

	for (int32 Index = 0; Index < ChildCount; ++Index)
	{
		TSharedRef<SDMTextureSetBuilderCellBase> Cell = StaticCastSharedRef<SDMTextureSetBuilderCellBase>(Children->GetChildAt(Index));

		if (Cell->GetTexture() == InTexture)
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
