// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMTextureSetBlueprintFunctionLibrary.h"
#include "Widgets/SCompoundWidget.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"

class SDMTextureSetBuilderCellBase;
class SGridPanel;
class UDMTextureSet;
class UTexture;
struct FAssetData;
struct FDMTextureSetBuilderEntry;

class SDMTextureSetBuilder : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDMTextureSetBuilder) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UDMTextureSet* InTextureSet, const TArray<FAssetData>& InAssets, FDMTextureSetBuilderOnComplete InOnComplete);

	bool WasAccepted() const { return bAccepted; }

	void SwapTexture(int32 InFromIndex, bool bInIsFromMaterialProperty, int32 InToIndex, bool bInIsToMaterialProperty);

	void SetTexture(int32 InIndex, bool bInIsMaterialProperty, UTexture* InTexture);

protected:
	TStrongObjectPtr<UDMTextureSet> TextureSet;
	TArray<FAssetData> Assets;
	TArray<TSharedRef<FDMTextureSetBuilderEntry>> MaterialProperties;
	TArray<UTexture*> UnassignedTextures;
	bool bAccepted = false;
	FDMTextureSetBuilderOnComplete OnComplete;

	TSharedPtr<SGridPanel> MaterialPropertyGrid;
	TSharedPtr<SGridPanel> UnassignedTextureGrid;

	TSharedRef<SWidget> GenerateMaterialPropertyCell(const TSharedRef<FDMTextureSetBuilderEntry>& InListItem, int32 InIndex);

	TSharedRef<SWidget> GenerateUnusedTextureCell(UTexture* InTexture, int32 InIndex);

	FReply OnAcceptClicked();

	FReply OnCancelClicked();

	void Close();

	void CreateUnassignedTextureSlots();

	bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets);

	void OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets);

	TSharedPtr<SDMTextureSetBuilderCellBase> GetCell(int32 InIndex, bool bInIsMaterialProperty) const;

	UTexture* GetTexture(int32 InIndex, bool bInIsMaterialProperty) const;

	void SetTexture(int32 InIndex, bool bInIsMaterialProperty, UTexture* InTexture) const;

	bool IsTextureAssignedToMaterialProperty(UTexture* InTexture) const;
};
