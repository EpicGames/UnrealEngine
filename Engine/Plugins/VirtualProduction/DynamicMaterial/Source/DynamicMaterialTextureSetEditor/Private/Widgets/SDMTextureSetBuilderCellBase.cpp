// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMTextureSetBuilderCellBase.h"

#include "AssetRegistry/AssetData.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Engine/Texture.h"
#include "Widgets/DMTextureSetBuilderDragDropOperation.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SDMTextureSetBuilder.h"

#define LOCTEXT_NAMESPACE "SDMTextureSetBuilderCellBase"

SDMTextureSetBuilderCellBase::SDMTextureSetBuilderCellBase()
	: Index(-1)
	, bIsMaterialProperty(false)
{
}

void SDMTextureSetBuilderCellBase::Construct(const FArguments& InArgs, const TSharedRef<SDMTextureSetBuilder>& InTextureSetBuilder,
	UTexture* InTexture, int32 InIndex, bool bInIsMaterialProperty)
{
	TextureSetBuilderWeak = InTextureSetBuilder;
	Texture.Reset(InTexture);
	Index = InIndex;
	bIsMaterialProperty = bInIsMaterialProperty;
}

UTexture* SDMTextureSetBuilderCellBase::GetTexture() const
{
	return Texture.Get();
}

void SDMTextureSetBuilderCellBase::SetTexture(UTexture* InTexture)
{
	Texture.Reset(InTexture);
}

FReply SDMTextureSetBuilderCellBase::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && Texture.IsValid())
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SDMTextureSetBuilderCellBase::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedRef<FDMTextureSetBuilderDragDropOperation> Operation = FDMTextureSetBuilderDragDropOperation::New(
		FAssetData(Texture.Get()),
		Index,
		bIsMaterialProperty
	);

	return FReply::Handled().BeginDragDrop(Operation);
}

bool SDMTextureSetBuilderCellBase::OnAssetDraggedOver(TArrayView<FAssetData> InAssets)
{
	for (const FAssetData& Asset : InAssets)
	{
		UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

		if (AssetClass && AssetClass->IsChildOf(UTexture::StaticClass()))
		{
			return true;
		}
	}

	return false;
}

void SDMTextureSetBuilderCellBase::OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets)
{
	if (TSharedPtr<SDMTextureSetBuilder> TextureSetBuilder = TextureSetBuilderWeak.Pin())
	{
		if (TSharedPtr<FDMTextureSetBuilderDragDropOperation> BuilderOperation = InDragDropEvent.GetOperationAs<FDMTextureSetBuilderDragDropOperation>())
		{
			TextureSetBuilder->SwapTexture(
				BuilderOperation->GetIndex(),
				BuilderOperation->IsMaterialProperty(),
				Index,
				bIsMaterialProperty
			);
		}
		else if (TSharedPtr<FAssetDragDropOp> AssetOperation = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
		{
			for (const FAssetData& Asset : AssetOperation->GetAssets())
			{
				UClass* AssetClass = Asset.GetClass(EResolveClass::Yes);

				if (AssetClass && AssetClass->IsChildOf(UTexture::StaticClass()))
				{
					if (UTexture* AssetTexture = Cast<UTexture>(Asset.GetAsset()))
					{
						TextureSetBuilder->SetTexture(
							Index,
							bIsMaterialProperty,
							AssetTexture
						);

						// Only set the first texture.
						break;
					}
				}
			}
		}
	}
}

EVisibility SDMTextureSetBuilderCellBase::GetImageVisibility() const
{
	return Texture.IsValid() ? EVisibility::Visible : EVisibility::Hidden;
}

FText SDMTextureSetBuilderCellBase::GetToolTipText() const
{
	UTexture* TextureObject = Texture.Get();

	if (!TextureObject)
	{
		return LOCTEXT("NoTexture", "Texture slot empty.");
	}

	const FText Format = LOCTEXT("TextureTooltipFormat", "{0}\n\nDrag to another slot to swap textures. Hold shift when dropping to overwrite.");

	return FText::Format(Format, FText::FromString(TextureObject->GetPathName()));
}

FText SDMTextureSetBuilderCellBase::GetTextureName() const
{
	UTexture* TextureObject = Texture.Get();

	if (!TextureObject)
	{
		return LOCTEXT("-", "-");
	}

	return FText::FromString(TextureObject->GetName());
}

#undef LOCTEXT_NAMESPACE
