// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"

class SDMTextureSetBuilder;
class SImage;
class UTexture;

class SDMTextureSetBuilderCellBase : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDMTextureSetBuilderCellBase) {}
	SLATE_END_ARGS()

public:
	SDMTextureSetBuilderCellBase();

	virtual ~SDMTextureSetBuilderCellBase() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMTextureSetBuilder>& InTextureSetBuilder,
		UTexture* InTexture, int32 InIndex, bool bInIsMaterialProperty);

	UTexture* GetTexture() const;

	virtual void SetTexture(UTexture* InTexture);

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget

protected:
	TWeakPtr<SDMTextureSetBuilder> TextureSetBuilderWeak;
	TStrongObjectPtr<UTexture> Texture;
	int32 Index;
	bool bIsMaterialProperty;

	bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets);

	void OnAssetsDropped(const FDragDropEvent& InDragDropEvent, TArrayView<FAssetData> InAssets);

	EVisibility GetImageVisibility() const;

	FText GetToolTipText() const;

	FText GetTextureName() const;
};
