// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMTextureSetBuilderMaterialPropertyCell.h"

#include "DMTextureSetBuilderEntry.h"
#include "DMTextureSetMaterialProperty.h"
#include "DMTextureSetStyle.h"
#include "Engine/Texture.h"
#include "ISinglePropertyView.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SAssetDropTarget.h"
#include "SDMTextureSetBuilder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMTextureSetBuilderMaterialPropertyCell"

namespace UE::DynamicMaterial::TextureSet::Private
{
	const TCHAR* Material_Red =   TEXT("/Script/Engine.Material'/DynamicMaterial/Materials/TextureSet_Red.TextureSet_Red'");
	const TCHAR* Material_Green = TEXT("/Script/Engine.Material'/DynamicMaterial/Materials/TextureSet_Green.TextureSet_Green'");
	const TCHAR* Material_Blue =  TEXT("/Script/Engine.Material'/DynamicMaterial/Materials/TextureSet_Blue.TextureSet_Blue'");
	const TCHAR* Material_Alpha = TEXT("/Script/Engine.Material'/DynamicMaterial/Materials/TextureSet_Alpha.TextureSet_Alpha'");
	const TCHAR* Material_RGB =   TEXT("/Script/Engine.Material'/DynamicMaterial/Materials/TextureSet_RGB.TextureSet_RGB'");
	const TCHAR* Material_All =   TEXT("/Script/Engine.Material'/DynamicMaterial/Materials/TextureSet_All.TextureSet_All'");

	const TCHAR* Parameter_Texture = TEXT("Texture");
}

SDMTextureSetBuilderMaterialPropertyCell::SDMTextureSetBuilderMaterialPropertyCell()
	: MaterialBrush(FSlateMaterialBrush(FVector2D(120.f)))
{
}

void SDMTextureSetBuilderMaterialPropertyCell::Construct(const FArguments& InArgs, const TSharedRef<SDMTextureSetBuilder>& InTextureSetBuilder, 
	const TSharedRef<FDMTextureSetBuilderEntry>& InEntry, int32 InIndex)
{
	SDMTextureSetBuilderCellBase::Construct(
		SDMTextureSetBuilderCellBase::FArguments(), 
		InTextureSetBuilder,
		InEntry->Texture,
		InIndex,
		/* Is Material Property */ true
	);

	Entry = InEntry;

	UEnum* MaterialPropertyEnum = StaticEnum<EDMTextureSetMaterialProperty>();

	if (!MaterialPropertyEnum)
	{
		return;
	}

	SetMaterialForChannelMask();

	EntryProvider = MakeShared<FDMTextureSetBuilderEntryProvider>(InEntry);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FSinglePropertyParams InitParams;
	InitParams.NotifyHook = this;
	InitParams.NamePlacement = EPropertyNamePlacement::Hidden;

	TSharedPtr<ISinglePropertyView> PropertyView = PropertyEditorModule.CreateSingleProperty(
		EntryProvider.ToSharedRef(),
		GET_MEMBER_NAME_CHECKED(FDMTextureSetBuilderEntry, ChannelMask),
		InitParams
	);

	if (!PropertyView.IsValid())
	{
		return;
	}

	PropertyView->SetEnabled(TAttribute<bool>::CreateSP(this, &SDMTextureSetBuilderMaterialPropertyCell::GetPropertyEnabled));

	ChildSlot
	[
		SNew(SBorder)
		.Padding(10.f)
		.BorderImage(FDMTextureSetStyle::Get().GetBrush("TextureSetConfig.Cell.Background"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0.f, 0.f, 0.f, 5.f)
			[
				SNew(STextBlock)
				.Text(MaterialPropertyEnum->GetDisplayNameTextByValue(static_cast<int64>(Entry->MaterialProperty)))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0.f, 0.f, 0.f, 5.f)
			[
				SNew(SAssetDropTarget)
				.OnAreAssetsAcceptableForDrop(this, &SDMTextureSetBuilderMaterialPropertyCell::OnAssetDraggedOver)
				.OnAssetsDropped(this, &SDMTextureSetBuilderMaterialPropertyCell::OnAssetsDropped)
				[
					SNew(SOverlay)
					.ToolTipText(this, &SDMTextureSetBuilderMaterialPropertyCell::GetToolTipText)

					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Image(&MaterialBrush)
						.DesiredSizeOverride(FVector2D(120.f))
						.Visibility(this, &SDMTextureSetBuilderMaterialPropertyCell::GetImageVisibility)
					]

					+ SOverlay::Slot()
					.Padding(5.f)
					[
						SNew(STextBlock)
						.Text(this, &SDMTextureSetBuilderMaterialPropertyCell::GetTextureName)
						.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
						.WrapTextAt(110.f)
						.Font(FAppStyle::GetFontStyle("TinyText"))
						.Visibility(this, &SDMTextureSetBuilderMaterialPropertyCell::GetTextureNameVisibility)
						.HighlightText(this, &SDMTextureSetBuilderMaterialPropertyCell::GetTextureName)
						.HighlightColor(FDMTextureSetStyle::Get().GetColor(TEXT("TextureSetConfig.TextureNameHighlight.Color")))
						.HighlightShape(FDMTextureSetStyle::Get().GetBrush(TEXT("TextureSetConfig.TextureNameHighlight.Background")))
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(0.f, 0.f, 0.f, 0.f)
			[
				PropertyView.ToSharedRef()
			]
		]
	];
}

void SDMTextureSetBuilderMaterialPropertyCell::SetTexture(UTexture* InTexture)
{
	SDMTextureSetBuilderCellBase::SetTexture(InTexture);

	using namespace UE::DynamicMaterial::TextureSet::Private;

	if (UMaterialInstanceDynamic* MIDObject = MID.Get())
	{
		MIDObject->SetTextureParameterValue(Parameter_Texture, InTexture);
	}	
}

bool SDMTextureSetBuilderMaterialPropertyCell::GetPropertyEnabled() const
{
	return Texture.IsValid();
}

void SDMTextureSetBuilderMaterialPropertyCell::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	if (InPropertyThatChanged && InPropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(FDMTextureSetBuilderEntry, ChannelMask))
	{
		SetMaterialForChannelMask();
	}
}

void SDMTextureSetBuilderMaterialPropertyCell::SetMaterialForChannelMask()
{
	using namespace UE::DynamicMaterial::TextureSet::Private;

	UMaterial* ParentMaterial = nullptr;

	switch (Entry->ChannelMask)
	{
		case EDMTextureChannelMask::Red:
			ParentMaterial = LoadObject<UMaterial>(GetTransientPackage(), Material_Red);
			break;

		case EDMTextureChannelMask::Green:
			ParentMaterial = LoadObject<UMaterial>(GetTransientPackage(), Material_Green);
			break;

		case EDMTextureChannelMask::Blue:
			ParentMaterial = LoadObject<UMaterial>(GetTransientPackage(), Material_Blue);
			break;

		case EDMTextureChannelMask::Alpha:
			ParentMaterial = LoadObject<UMaterial>(GetTransientPackage(), Material_Alpha);
			break;

		case EDMTextureChannelMask::RGB:
			ParentMaterial = LoadObject<UMaterial>(GetTransientPackage(), Material_RGB);
			break;

		case EDMTextureChannelMask::RGBA:
			ParentMaterial = LoadObject<UMaterial>(GetTransientPackage(), Material_All);
			break;
	}

	if (!ParentMaterial)
	{
		return;
	}

	UMaterialInstanceDynamic* MIDObject = MID.Get();

	if (!MIDObject || MIDObject->Parent != ParentMaterial)
	{
		MIDObject = UMaterialInstanceDynamic::Create(ParentMaterial, GetTransientPackage());
		MID.Reset(MIDObject);
	}

	if (MIDObject)
	{
		MIDObject->SetTextureParameterValue(Parameter_Texture, Texture.Get());
		MaterialBrush.SetMaterial(MIDObject);
	}
}

EVisibility SDMTextureSetBuilderMaterialPropertyCell::GetTextureNameVisibility() const
{
	return Texture.IsValid()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
