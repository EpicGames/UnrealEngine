// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SDMTextureSetBuilderCellBase.h"

#include "SlateMaterialBrush.h"
#include "Templates/SharedPointer.h"

class FDMTextureSetBuilderEntryProvider;
class SDMTextureSetBuilder;
class UMaterialInstanceDynamic;
struct FDMTextureSetBuilderEntry;

class SDMTextureSetBuilderMaterialPropertyCell : public SDMTextureSetBuilderCellBase, public FNotifyHook
{
	SLATE_BEGIN_ARGS(SDMTextureSetBuilderMaterialPropertyCell) {}
	SLATE_END_ARGS()

public:
	SDMTextureSetBuilderMaterialPropertyCell();

	virtual ~SDMTextureSetBuilderMaterialPropertyCell() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMTextureSetBuilder>& InTextureSetBuilder, 
		const TSharedRef<FDMTextureSetBuilderEntry>& InEntry, int32 InIndex);

	//~ Begin SDMTextureSetBuilderCellBase
	virtual void SetTexture(UTexture* InTexture) override;
	//~ End SDMTextureSetBuilderCellBase

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
	//~ End FNotifyHook

protected:
	TSharedPtr<FDMTextureSetBuilderEntry> Entry;
	TSharedPtr<FDMTextureSetBuilderEntryProvider> EntryProvider;
	TStrongObjectPtr<UMaterialInstanceDynamic> MID;
	FSlateMaterialBrush MaterialBrush;

	bool GetPropertyEnabled() const;

	void SetMaterialForChannelMask();

	EVisibility GetTextureNameVisibility() const;
};
