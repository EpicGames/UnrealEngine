// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMTextureSetBuilderUnassignedTextureCell.h"

#include "AssetRegistry/AssetData.h"
#include "DMTextureSetStyle.h"
#include "Engine/Texture.h"
#include "SDMTextureSetBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDMTextureSetBuilderUnassignedTextureCell"

SDMTextureSetBuilderUnassignedTextureCell::SDMTextureSetBuilderUnassignedTextureCell()
	: TextureBrush(FSlateImageBrush((UObject*)nullptr, FVector2D(120.f)))
{
}

void SDMTextureSetBuilderUnassignedTextureCell::Construct(const FArguments& InArgs, const TSharedRef<SDMTextureSetBuilder>& InTextureSetBuilder,
	UTexture* InTexture, int32 InIndex)
{
	SDMTextureSetBuilderCellBase::Construct(
		SDMTextureSetBuilderCellBase::FArguments(),
		InTextureSetBuilder,
		InTexture,
		InIndex,
		/* Material Property */ false
	);

	TextureSetBuilderWeak = InTextureSetBuilder;
	Index = InIndex;

	TextureBrush.SetResourceObject(InTexture);

	ChildSlot
	[
		SNew(SVerticalBox)
		.Visibility(this, &SDMTextureSetBuilderUnassignedTextureCell::GetImageVisibility)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(0.f, 0.f, 0.f, 5.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Unassigned", "Unassigned"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(0.f, 0.f, 0.f, 0.f)
		[
			SNew(SOverlay)
			.ToolTipText(this, &SDMTextureSetBuilderUnassignedTextureCell::GetToolTipText)

			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(&TextureBrush)
				.DesiredSizeOverride(FVector2D(120.f))
				.Visibility(this, &SDMTextureSetBuilderUnassignedTextureCell::GetImageVisibility)
			]

			+ SOverlay::Slot()
			.Padding(5.f)
			[
				SNew(STextBlock)
				.Text(this, &SDMTextureSetBuilderUnassignedTextureCell::GetTextureName)
				.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
				.WrapTextAt(110.f)
				.Font(FAppStyle::GetFontStyle("TinyText"))
				.HighlightText(this, &SDMTextureSetBuilderUnassignedTextureCell::GetTextureName)
				.HighlightColor(FDMTextureSetStyle::Get().GetColor(TEXT("TextureSetConfig.TextureNameHighlight.Color")))
				.HighlightShape(FDMTextureSetStyle::Get().GetBrush(TEXT("TextureSetConfig.TextureNameHighlight.Background")))
			]
		]
	];
}

void SDMTextureSetBuilderUnassignedTextureCell::SetTexture(UTexture* InTexture)
{
	SDMTextureSetBuilderCellBase::SetTexture(InTexture);

	TextureBrush.SetResourceObject(InTexture);
}

#undef LOCTEXT_NAMESPACE
