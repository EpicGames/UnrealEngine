// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SDMTextureSetBuilderCellBase.h"

#include "Brushes/SlateImageBrush.h"
#include "Templates/SharedPointer.h"

class SDMTextureSetBuilder;

class SDMTextureSetBuilderUnassignedTextureCell : public SDMTextureSetBuilderCellBase
{
	SLATE_BEGIN_ARGS(SDMTextureSetBuilderUnassignedTextureCell) {}
	SLATE_END_ARGS()

public:
	SDMTextureSetBuilderUnassignedTextureCell();

	virtual ~SDMTextureSetBuilderUnassignedTextureCell() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SDMTextureSetBuilder>& InTextureSetBuilder,
		UTexture* InTexture, int32 InIndex);

	//~ Begin SDMTextureSetBuilderCellBase
	virtual void SetTexture(UTexture* InTexture) override;
	//~ End SDMTextureSetBuilderCellBase

protected:
	FSlateImageBrush TextureBrush;
};
