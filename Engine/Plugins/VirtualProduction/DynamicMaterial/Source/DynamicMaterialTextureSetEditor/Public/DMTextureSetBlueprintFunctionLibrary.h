// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "AssetRegistry/AssetData.h"

#include "DMTextureSetBlueprintFunctionLibrary.generated.h"

class UDMTextureSet;

DECLARE_DELEGATE_TwoParams(FDMTextureSetBuilderOnComplete, UDMTextureSet*, /* Was Accepted */bool);

/**
 * Material Designer Texture Set Blueprint Function Library
 */
UCLASS(BlueprintType)
class UDMTextureSetBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Uses the filters from the Texture Set Settings to create a Texture Set based on the given assets.
	 * @param InAssets The texture assets to assign to the texture slot.
	 * @return A new texture set or nullptr if no textures were filtered.
	 */
	UFUNCTION(BlueprintCallable, Category = "Material Designer")
	static DYNAMICMATERIALTEXTURESETEDITOR_API UDMTextureSet* CreateTextureSetFromAssets(const TArray<FAssetData>& InAssets);

	/**
	 * Uses the filters from the Texture Set Settings to create a Texture Set based on the given assets
	 * and presents the user with a UI to confirm slot assignments.
	 * @param InAssets The texture assets to assign to the texture slot.
	 * @param InOnComplete Called when the interactive panel is closed, successful or not.
	 */
	static DYNAMICMATERIALTEXTURESETEDITOR_API void CreateTextureSetFromAssetsInteractive(const TArray<FAssetData>& InAssets, FDMTextureSetBuilderOnComplete InOnComplete);
};
