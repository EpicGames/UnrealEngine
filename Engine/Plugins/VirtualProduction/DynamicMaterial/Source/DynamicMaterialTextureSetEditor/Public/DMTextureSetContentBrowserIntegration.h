// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/IDelegateInstance.h"

class FMenuBuilder;

class FDMTextureSetContentBrowserIntegration
{
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPopulateMenu, FMenuBuilder& InMenuBuilder,
		const TArray<FAssetData>& InSelectedAssets)

	DYNAMICMATERIALTEXTURESETEDITOR_API static FOnPopulateMenu::RegistrationType& GetPopulateExtenderDelegate();

protected:
	static FOnPopulateMenu PopulateMenuDelegate;
};
