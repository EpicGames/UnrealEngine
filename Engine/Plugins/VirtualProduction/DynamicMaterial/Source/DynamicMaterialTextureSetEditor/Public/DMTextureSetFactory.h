// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "DMTextureSetFactory.generated.h"

UCLASS(MinimalAPI)
class UDMTextureSetFactory : public UFactory
{
	GENERATED_BODY()

public:
	UDMTextureSetFactory();

	//~ Begin UFactory
	DYNAMICMATERIALTEXTURESETEDITOR_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags,
		UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	//~ End UFactory
};
